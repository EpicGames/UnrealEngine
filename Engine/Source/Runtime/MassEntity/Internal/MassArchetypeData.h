// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityHandle.h"
#include "MassArchetypeTypes.h"
#include "Mass/ArchetypeGroup.h"
#include "Misc/MTTransactionallySafeAccessDetector.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityManager.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassRequirements.h"
#include "MassSparseElementsStorage.h"
#include "HAL/LowLevelMemTracker.h"

struct FMassEntityQuery;
struct FMassExecutionContext;
class FOutputDevice;
struct FMassArchetypeEntityCollection;
struct FMassFragmentRequirementDescription;
struct FMassFragmentRequirements;

namespace UE::Mass
{
	struct FSparseElementsStorage;
	struct FChunkSparseElements;

	uint32 SanitizeChunkMemorySize(const uint32 InChunkMemorySize, const bool bLogMismatch = true);

	struct FChunkSparseElements
	{
		FChunkSparseElements() = default;
		FChunkSparseElements(const uint32 NumEntitiesPerChunk)
		{
			Init(NumEntitiesPerChunk);
		}

		void Init(const uint32 NumEntitiesPerChunk)
		{
			PerEntityElements.AddDefaulted(NumEntitiesPerChunk);
		}

		/** @return whether it is the first instance of ElementType added to this chunk, i.e. SparseElementsPresent has been modified */
		bool Add(const FMassRawEntityInChunkData& EntityInChunkHandle, TNotNull<const UScriptStruct*> ElementType);
		/** @return whether it is the last instance of ElementType removes from this chunk, i.e. SparseElementsPresent has been modified */
		bool Remove(const FMassRawEntityInChunkData& EntityInChunkHandle, TNotNull<const UScriptStruct*> ElementType);
		/** @return whether SparseElementsPresent has been modified */
		bool UpdateElementsPresenceOnEntityRemoval(const int32 IndexWithinChunk);

		/** @return whether these are the first instances of ElementType added to this chunk, i.e. SparseElementsPresent has been modified */
		bool BatchAdd(TNotNull<const UScriptStruct*> ElementType, int32 SubchunkStart, int32 RangeLength);
		/** @return whether these are the last instances of ElementType removed this chunk, i.e. SparseElementsPresent has been modified */
		bool BatchRemove(TNotNull<const UScriptStruct*> ElementType, int32 SubchunkStart, int32 RangeLength);

		bool IsEmpty() const
		{
			return PerEntityElements.IsEmpty();
		}

		bool Contains(const FMassRawEntityInChunkData& EntityInChunkHandle, TNotNull<const UScriptStruct*> ElementType) const;
		/** @return whether anything in the chunk potentially has the element */
		bool Contains(TNotNull<const UScriptStruct*> ElementType) const
		{
			return SparseElementsPresent.Contains(ElementType);
		}

		bool HasElementsInRange(int32 IndexWithinChunk, int32 Num) const
		{
			for (int32 Index = 0; Index < Num; ++Index)
			{
				if (PerEntityElements.IsValidIndex(IndexWithinChunk + Index) 
					&& PerEntityElements[IndexWithinChunk + Index].IsEmpty() == false)
				{
					return true;
				}
			}
			return false;
		}

		/**
		 * Moves Num sparse elements bitsets from OriginalSparseElements, starting at index OriginalIndexWithinChunk
		 * to `this` FChunkSparseElements instances, starting at index NewIndexWithinChunk
		 * @return true if any bits have been added to or remove from SparseElementsPresent, both from `this` and OriginalSparseElements (if it's not the same).
		 */
		bool MoveElementsFrom(FChunkSparseElements& OriginalSparseElements, int32 OriginalIndexWithinChunk, int32 NewIndexWithinChunk, int32 Num);

		const FMassElementBitSet& GetSparseElementsPresent() const
		{
			return SparseElementsPresent;
		}

		FMassElementBitSet& GetElementsForEntity(const FMassRawEntityInChunkData& EntityInChunkHandle)
		{
			check(PerEntityElements.IsEmpty() == false);
			return PerEntityElements[EntityInChunkHandle.IndexWithinChunk];
		}

		const FMassElementBitSet& GetElementsForEntity(const FMassRawEntityInChunkData& EntityInChunkHandle) const
		{
			check(PerEntityElements.IsEmpty() == false);
			return PerEntityElements[EntityInChunkHandle.IndexWithinChunk];
		}

		FMassElementBitSet& GetElementsForEntityUnsafe(const int32 Index)
		{
			check(PerEntityElements.IsValidIndex(Index));
			return PerEntityElements[Index];
		}

		/**
		 * Reports the size of memory allocations owned by this instance, including the per-entity bitset payloads
		 * and the FElementCounter dense arrays. Returns 0 when the instance has not been initialized
		 * (i.e. IsEmpty() == true).
		 */
		SIZE_T GetAllocatedSize() const;

	private:
		FMassElementBitSet SparseElementsPresent;
		TArray<FMassElementBitSet> PerEntityElements;

		struct FElementCounter
		{
			uint16& Get(const int32 TypeIndex)
			{
				if (LIKELY(DenseIndexToElementIndex.IsValidIndex(LastAccessedDenseIndex)
					&& DenseIndexToElementIndex[LastAccessedDenseIndex] == TypeIndex))
				{
					return DenseCounters[LastAccessedDenseIndex];
				}
				if (UNLIKELY(DenseIndexToElementIndex.IsEmpty()))
				{
					// add a new one at the end
					LastAccessedDenseIndex = DenseIndexToElementIndex.Add(static_cast<uint16>(TypeIndex));
					return DenseCounters.Add_GetRef(0);
				}

				const int32 DenseIndex = Algo::LowerBound(DenseIndexToElementIndex, TypeIndex);
				LastAccessedDenseIndex = DenseIndex;

				if (DenseIndex == DenseIndexToElementIndex.Num()
					|| DenseIndexToElementIndex[DenseIndex] != TypeIndex)
				{
					// in this case DenseIndex indicates the new element insertion index
					DenseIndexToElementIndex.Insert(static_cast<uint16>(TypeIndex), DenseIndex);
					return DenseCounters.Insert_GetRef(0, DenseIndex);
				}

				return DenseCounters[DenseIndex];
			}

			SIZE_T GetAllocatedSize() const
			{
				return DenseCounters.GetAllocatedSize() + DenseIndexToElementIndex.GetAllocatedSize();
			}

		private:
			int32 LastAccessedDenseIndex = INDEX_NONE;
			TArray<uint16, TInlineAllocator<16>> DenseCounters;
			TArray<uint16, TInlineAllocator<16>> DenseIndexToElementIndex;
		};
		FElementCounter SparseElementTypesCount;
	};
} // namespace UE::Mass

/** This type represents a single chunk within an archetype */
struct FMassArchetypeChunk
{
private:
	uint8* RawMemory = nullptr;
	SIZE_T AllocSize = 0;
	int32 NumInstances = 0;
	int32 SerialModificationNumber = 0;
	TArray<FInstancedStruct> ChunkFragmentData;
	FMassArchetypeSharedFragmentValues SharedFragmentValues;
	TArray<uint16, TInlineAllocator<256>> ElementTypeCounts;
	UE::Mass::FChunkSparseElements SparseElements;
	
public:
	explicit FMassArchetypeChunk(const SIZE_T InAllocSize, TConstArrayView<FInstancedStruct> InChunkFragmentTemplates, const FMassArchetypeSharedFragmentValues& InSharedFragmentValues)
		: AllocSize(InAllocSize)
		, ChunkFragmentData(InChunkFragmentTemplates)
		, SharedFragmentValues(InSharedFragmentValues)
	{
		
		LLM_SCOPE_BYNAME(TEXT("Mass/ArchetypeChunk"));
		RawMemory = static_cast<uint8*>(FMemory::Malloc(AllocSize));
	}

	~FMassArchetypeChunk()
	{
		// Only release memory if it was not done already.
		if (RawMemory != nullptr)
		{
			FMemory::Free(RawMemory);
			RawMemory = nullptr;
		}
	}

	// Returns the Entity array element at the specified index
	FMassEntityHandle& GetEntityArrayElementRef(int32 ChunkBase, int32 IndexWithinChunk)
	{
		uint8* RawMemoryChunkBase = RawMemory + ChunkBase;
		checkSlow(ChunkBase + IndexWithinChunk * sizeof(FMassEntityHandle) < AllocSize
			&& (reinterpret_cast<SIZE_T>(RawMemoryChunkBase) % alignof(FMassEntityHandle)) == 0);
		return reinterpret_cast<FMassEntityHandle*>(RawMemoryChunkBase)[IndexWithinChunk];
	}

	const FMassEntityHandle* GetEntityArray(int32 ChunkBase) const
	{
		uint8* RawMemoryChunkBase = RawMemory + ChunkBase;
		checkSlow(ChunkBase < AllocSize
			&& (reinterpret_cast<SIZE_T>(RawMemoryChunkBase) % alignof(FMassEntityHandle)) == 0);
		return reinterpret_cast<const FMassEntityHandle*>(RawMemoryChunkBase);
	}

	uint8* GetRawMemory() const
	{
		return RawMemory;
	}

	int32 GetNumInstances() const
	{
		return NumInstances;
	}

	bool HasSparseElements() const
	{
		return SparseElements.IsEmpty() == false;
	}

	UE::Mass::FChunkSparseElements& GetSparseElements(const uint32 NumEntitiesPerChunk)
	{
		if (SparseElements.IsEmpty())
		{
			SparseElements.Init(NumEntitiesPerChunk);
		}
		return SparseElements;
	}

	/**
	 * Note that the returned FChunkSparseElements instance might not be initialized.
	 * It's caller's responsibility to call FChunkSparseElements.IsEmpty to verify the instance's state.
	 * Call GetSparseElements(uint32) if you need a guaranteed initialized instance.
	 */
	UE::Mass::FChunkSparseElements& GetSparseElementsUnsafe()
	{
		return SparseElements;
	}

	/** The caller is responsible for checking whether SparseElements.IsEmpty */
	const UE::Mass::FChunkSparseElements& GetSparseElementsUnsafe() const
	{
		return SparseElements;
	}

	/**
	 * Reports the size of memory allocations owned by this chunk, excluding the raw chunk buffer
	 * (which is accounted for separately via FMassArchetypeData::GetChunkAllocSize). Currently
	 * covers the per-chunk SparseElements bookkeeping.
	 */
	SIZE_T GetAllocatedSize() const
	{
		return SparseElements.GetAllocatedSize();
	}

	bool DoesMatchComposition(const FMassElementBitSet& RequiredAllSparseElements, const FMassElementBitSet& RequiredAnySparseElements) const
	{
		if (SparseElements.IsEmpty() == false)
		{
			return (RequiredAllSparseElements.IsEmpty() || SparseElements.GetSparseElementsPresent().HasAll(RequiredAllSparseElements))
				&& (RequiredAnySparseElements.IsEmpty() || SparseElements.GetSparseElementsPresent().HasAny(RequiredAnySparseElements));
		}
		return RequiredAllSparseElements.IsEmpty() && RequiredAnySparseElements.IsEmpty();
	}

	void AddMultipleInstances(uint32 Count)
	{
		NumInstances += Count;
		SerialModificationNumber++;
	}

	void RemoveMultipleInstances(uint32 Count)
	{
		NumInstances -= Count;
		check(NumInstances >= 0);
		SerialModificationNumber++;

		// Because we only remove trailing chunks to avoid messing up the absolute indices in the entities map,
		// We are freeing the memory here to save memory
		if (NumInstances == 0)
		{
			FMemory::Free(RawMemory);
			RawMemory = nullptr;
		}
	}

	void AddInstance()
	{
		AddMultipleInstances(1);
	}

	void RemoveInstance()
	{
		RemoveMultipleInstances(1);
	}

	int32 GetSerialModificationNumber() const
	{
		return SerialModificationNumber;
	}

	FStructView GetMutableChunkFragmentViewChecked(const int32 Index)
	{
		return FStructView(ChunkFragmentData[Index]);
	}

	FInstancedStruct* FindMutableChunkFragment(const UScriptStruct* Type)
	{
		return ChunkFragmentData.FindByPredicate([Type](const FInstancedStruct& Element)
			{
				return Element.GetScriptStruct()->IsChildOf(Type);
			});
	}

	void Recycle(TConstArrayView<FInstancedStruct> InChunkFragmentsTemplate, const FMassArchetypeSharedFragmentValues& InSharedFragmentValues)
	{
		checkf(NumInstances == 0, TEXT("Recycling a chunk that is not empty."));
		SerialModificationNumber++;
		ChunkFragmentData = InChunkFragmentsTemplate;
		SharedFragmentValues = InSharedFragmentValues;
		
		// If this chunk previously had entity and it does not anymore, we might have to reallocate the memory as it was freed to save memory
		if (RawMemory == nullptr)
		{
			RawMemory = static_cast<uint8*>(FMemory::Malloc(AllocSize));
		}
	}

	bool IsValidSubChunk(const int32 StartIndex, const int32 Length) const
	{
		return StartIndex >= 0 && StartIndex < NumInstances && (StartIndex + Length) <= NumInstances;
	}

#if WITH_MASSENTITY_DEBUG
	int32 DebugGetChunkFragmentCount() const
	{
		return ChunkFragmentData.Num();
	}
#endif // WITH_MASSENTITY_DEBUG

	FMassArchetypeSharedFragmentValues& GetMutableSharedFragmentValues()
	{
		return SharedFragmentValues;
	}
	const FMassArchetypeSharedFragmentValues& GetSharedFragmentValues() const
	{
		return SharedFragmentValues;
	}
};

// Information for a single fragment type in an archetype
struct FMassArchetypeFragmentConfig
{
	const UScriptStruct* FragmentType = nullptr;
	int32 ArrayOffsetWithinChunk = 0;

	void* GetFragmentData(uint8* ChunkBase, int32 IndexWithinChunk) const
	{
		return ChunkBase + ArrayOffsetWithinChunk + (IndexWithinChunk * FragmentType->GetStructureSize());
	}
};

// An archetype is defined by a collection of unique fragment types (no duplicates).
// Order doesn't matter, there will only ever be one FMassArchetypeData per unique set of fragment types per entity manager subsystem
struct FMassArchetypeData
{
private:
	// One-stop-shop variable describing composition of entities hosted by the archetype
	FMassElementBitSet CompositionBitSet;
	mutable FMassElementBitSet CachedSparseElementsBitSet;
	
	// Pre-created default chunk fragment templates
	TArray<FInstancedStruct> ChunkFragmentsTemplate;

	TArray<FMassArchetypeFragmentConfig, TInlineAllocator<16>> FragmentConfigs;
	
	TArray<FMassArchetypeChunk> Chunks;

	// Entity ID to index within archetype
	//@TODO: Could be folded into FEntityData in the entity manager at the expense of a bit
	// of loss of encapsulation and extra complexity during archetype changes
	TMap<int32, int32> EntityMap;
	
	TMap<const UScriptStruct*, int32> FragmentIndexMap;

	UE::Mass::FArchetypeGroups Groups;

	int32 NumEntitiesPerChunk;
	uint32 TotalBytesPerEntity = 0;
	int32 EntityListOffsetWithinChunk;

	/**
	 * Archetype version at which this archetype was created, useful for query to do incremental archetype matching.
	 * Note that it's set once and never changed afterward.
	 */
	uint32 CreatedArchetypeDataVersion = 0;
	
	/**
	 * The current version of this archetype, this value is incremented whenever an entity is added to or removed from this archetype and when
	 * any operation modifies the order of hosted entities (e.g compaction). This value can be checked to see if the archetype has changed in
	 * any way after an operation.
	 */
	uint32 ArchetypeVersion = 0;

	/**
	 * Incremented whenever an operation modifies the order of hosted entities, for example entity removal and compaction.
	 * Unlike ArchetypeVersion - this value is only incremented when the order changes. E.g adding an entity to the archetype will not
	 * increment this value since it does not modify entity order.
	 * This value is used to validate stored entity ranges, including FMassArchetypeEntityCollection.
	 */
	uint32 EntityOrderVersion = 0;

	/** Defaults to UMassEntitySettings.ChunkMemorySize. In near future will support being set via constructor. */
	const uint32 ChunkMemorySize = 0;

	/**
	 * Points to the EntityManager-owned SparseElementsStorage. This is why it's safe to use a raw pointer here,
	 * because that's the manager that the sole owner and creator of FMassArchetypeData instances
	 */
	UE::Mass::FSparseElementsStorage* SparseElementsStorage = nullptr;

	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(ChunkIterationDetector);

	mutable bool bSparseElementsBitSetDirty = false;

#if WITH_MASSENTITY_DEBUG
	/** Arrays of names the archetype is referred as. */
	TArray<FName> DebugNames;

	/**
	 * Color to be used when representing this archetype. If not set with FMassArchetypeCreationParams
	 * will be deterministically set based on archetype's composition. Can be overridden at any point 
	 * via SetDebugColor.
	 */
	FColor DebugColor;
#endif // WITH_MASSENTITY_DEBUG
	
	friend FMassEntityQuery;
	friend FMassArchetypeEntityCollection;
	friend FMassDebugger;

public:
	explicit FMassArchetypeData(const FMassArchetypeCreationParams& CreationParams = FMassArchetypeCreationParams());

	TConstArrayView<FMassArchetypeFragmentConfig> GetFragmentConfigs() const
	{
		return FragmentConfigs;
	}
	const FMassElementBitSet& GetCompositionBitSet() const;
	FMassArchetypeCompositionDescriptor GetCompositionDescriptor() const;

	FORCEINLINE const FMassArchetypeSharedFragmentValues& GetSharedFragmentValues(int32 EntityIndex) const
	{ 
		const int32 AbsoluteIndex = EntityMap.FindChecked(EntityIndex);
		const int32 ChunkIndex = AbsoluteIndex / NumEntitiesPerChunk;

		return Chunks[ChunkIndex].GetSharedFragmentValues();
	}
	FORCEINLINE const FMassArchetypeSharedFragmentValues& GetSharedFragmentValues(FMassEntityHandle Entity) const
	{
		return GetSharedFragmentValues(Entity.Index);
	}

	const UE::Mass::FArchetypeGroups& GetGroups() const;
	bool IsInGroup(const UE::Mass::FArchetypeGroupHandle GroupHandle) const;
	bool IsInGroupOfType(const UE::Mass::FArchetypeGroupType GroupType) const;

	/** Method to iterate on all the fragment types */
	void ForEachFragmentType(TFunction< void(const UScriptStruct* /*FragmentType*/)> Function) const;

	/**
	 * checks whether a given fragment type is part of this archetype's entities' composition
	 */
	bool HasFragmentType(const UScriptStruct* FragmentType) const
	{
		return UE::Mass::IsA<FMassFragment>(FragmentType) 
			&& CompositionBitSet.Contains(FragmentType);
	}

	/**
	 * checks whether a given tag type is part of this archetype's entities' composition
	 */
	bool HasTagType(const UScriptStruct* TagType) const
	{
		return UE::Mass::IsA<FMassTag>(TagType)
			&& CompositionBitSet.Contains(TagType);
	}

	/**
	 * checks whether a given element type is part of this archetype's entities' composition
	 */
	bool HasElement(TNotNull<const UScriptStruct*> ElementType) const
	{
		return CompositionBitSet.Contains(ElementType);
	}

	bool DoesContainEntitiesWithSparseElement(const UScriptStruct* ElementType) const
	{
		return UE::Mass::IsSparse(ElementType) && GetSparseElementsBitSet().Contains(ElementType);
	}

	bool IsEquivalent(const FMassElementBitSet& OtherCompositionBitSet, const UE::Mass::FArchetypeGroups& OtherGroups) const;

	void Initialize(const FMassEntityManager& EntityManager, const FMassElementBitSet& InCompositionBitSet
		, const uint32 ArchetypeDataVersion, UE::Mass::FSparseElementsStorage& InSparseElementsStorage);

	/** 
	 * A special way of initializing an archetype resulting in a copy of BaseArchetype's setup with OverrideTags
	 * replacing original tags of BaseArchetype
	 */
	void InitializeWithSimilar(const FMassEntityManager& EntityManager, const FMassArchetypeData& BaseArchetype
		, FMassElementBitSet&& InCompositionBitSet, const UE::Mass::FArchetypeGroups& InGroups, const uint32 ArchetypeDataVersion
		, UE::Mass::FSparseElementsStorage& InSparseElementsStorage);

	void AddEntity(FMassEntityHandle Entity, const FMassArchetypeSharedFragmentValues& InSharedFragmentValues);
	void RemoveEntity(FMassEntityHandle Entity);

	/**
	 * @return if ElementType is a tag the function will return an empty struct view. Otherwise, the function will
	 *	return a valid view to the element owned by the entity, provided it does own one, or an empty view otherwise.
	 */
	FStructView AddSparseElementToEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType);
	/** Works with all sparse element types (fragments and tags). */
	void RemoveSparseElementFromEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType);
	/** Removes all sparse elements indicated by SparseElementsBitSet from the given entity. Works with all sparse element types (fragments and tags). */
	void RemoveSparseElementFromEntity(FMassEntityHandle EntityHandle, const FMassElementBitSet& SparseElementsBitSet);
	/** Expects non-tag sparse element type. Will assert if the assumption is broken.
	 * @todo this method is const but returns mutable data (const doesn't propagate through the SparseElementsStorage
	 * raw pointer indirection). Consider making non-const to match GetMutableFragmentDataForEntity's convention. */
	FStructView GetMutableSparseElementDataForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle) const;
	/** Expects non-tag sparse element type. Will assert if the assumption is broken. */
	FConstStructView GetSparseElementDataForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle) const;
	/** @return a bitset indicating all sparse elements (both tags and fragments) owned by the given entity */
	FMassElementBitSet GetSparseElementsBitSetForEntity(const FMassEntityHandle EntityHandle) const;

	void* GetMutableFragmentDataForEntityChecked(TNotNull<const UScriptStruct*> FragmentType, const FMassEntityHandle EntityHandle);
	void* GetMutableFragmentDataForEntity(TNotNull<const UScriptStruct*> FragmentType, const FMassEntityHandle EntityHandle);
	const void* GetFragmentDataForEntityChecked(TNotNull<const UScriptStruct*> FragmentType, const FMassEntityHandle EntityHandle) const;
	const void* GetFragmentDataForEntity(TNotNull<const UScriptStruct*> FragmentType, const FMassEntityHandle EntityHandle) const;

	bool HasSparseElementForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle) const;
	bool HasSparseElementForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityInChunkDataHandle& EntityInChunkHandle) const;
	bool DoesEntityMatchSparseComposition(const FMassEntityInChunkDataHandle& EntityInChunkHandle
		, const FMassElementBitSet& RequiredAllSparseElements
		, const FMassElementBitSet& RequiredAnySparseElements
		, const FMassElementBitSet& RequiredNoneSparseElements) const;
	bool ContainsAnySparseData() const;
	bool ContainsSparseElement(TNotNull<const UScriptStruct*> ElementType) const;
	const FMassElementBitSet& GetSparseElementsBitSet() const;

	FORCEINLINE const int32* GetInternalIndexForEntity(const int32 EntityIndex) const
	{
		return EntityMap.Find(EntityIndex);
	}
	FORCEINLINE int32 GetInternalIndexForEntityChecked(const int32 EntityIndex) const
	{
		return EntityMap.FindChecked(EntityIndex);
	}
	int32 GetNumEntitiesPerChunk() const
	{
		return NumEntitiesPerChunk;
	}
	SIZE_T GetBytesPerEntity() const
	{
		return TotalBytesPerEntity;
	}

	int32 GetNumEntities() const
	{
		return EntityMap.Num();
	}

	SIZE_T GetChunkAllocSize() const
	{
		return ChunkMemorySize;
	}

	int32 GetChunkCount() const
	{
		return Chunks.Num();
	}
	int32 GetNonEmptyChunkCount() const;

	FORCEINLINE static int32 CalculateRangeLength(FMassArchetypeEntityCollection::FArchetypeEntityRange EntityRange, const FMassArchetypeChunk& Chunk)
	{
		return EntityRange.Length > 0
			? EntityRange.Length
			: (Chunk.GetNumInstances() - EntityRange.SubchunkStart);	
	}

	int32 CalculateRangeLength(FMassArchetypeEntityCollection::FArchetypeEntityRange EntityRange) const
	{
		check(Chunks.IsValidIndex(EntityRange.ChunkIndex));
		const FMassArchetypeChunk& Chunk = Chunks[EntityRange.ChunkIndex];
		return CalculateRangeLength(EntityRange, Chunk);
	}

	uint32 GetCreatedArchetypeDataVersion() const;
	uint32 GetEntityOrderVersion() const;
	uint32 GetArchetypeVersion() const;

	void ExecuteFunction(FMassExecutionContext& RunContext, const FMassExecuteFunction& Function, const FMassQueryRequirementIndicesMapping& RequirementMapping
		, FMassArchetypeEntityCollection::FConstEntityRangeArrayView EntityRangeContainer, const FMassChunkConditionFunction& ChunkCondition);
	void ExecuteFunction(FMassExecutionContext& RunContext, const FMassExecuteFunction& Function, const FMassQueryRequirementIndicesMapping& RequirementMapping
		, const FMassChunkConditionFunction& ChunkCondition, UE::Mass::FExecutionLimiter* ExecutionLimiter = nullptr);

	void ExecutionFunctionForChunk(FMassExecutionContext& RunContext, const FMassExecuteFunction& Function, const FMassQueryRequirementIndicesMapping& RequirementMapping
		, const FMassArchetypeEntityCollection::FArchetypeEntityRange& EntityRange, const FMassChunkConditionFunction& ChunkCondition = FMassChunkConditionFunction());

	/**
	 * Compacts entities to fill up chunks as much as possible
	 * @return number of entities moved around
	 */
	int32 CompactEntities(const double TimeAllowed);

	/**
	 * Moves the entity from this archetype to another, will only copy all matching fragment types.
	 * Uses an internally-cached fragment index mapping for the source->target archetype pair.
	 * @param Entity is the entity to move
	 * @param NewArchetype the archetype to move to
	 * @param SharedFragmentValuesOverride if provided will override all given Entity's shared fragment values
	 */
	void MoveEntityToAnotherArchetype(FMassEntityHandle Entity, FMassArchetypeData& NewArchetype, const FMassArchetypeSharedFragmentValues* SharedFragmentValuesOverride = nullptr);

	/**
	 * Set all fragment sources data on specified entity, will check if there are fragment sources type that does not exist in the archetype
	 * @param Entity is the entity to set the data of all fragments
	 * @param FragmentSources are the fragments to copy the data from
	 */
	template<typename TStruct>
	void SetFragmentsData(const FMassEntityHandle Entity, TConstArrayView<TStruct> FragmentSources);

	/**
	 * For all entities indicated by EntityCollection the function sets the value of fragment of type
	 *  FragmentSource.GetScriptStruct to the value represented by FragmentSource.GetMemory
	 */
	template<typename TStruct>
	void SetFragmentData(FMassArchetypeEntityCollection::FConstEntityRangeArrayView EntityRangeContainer, const TStruct& FragmentSource);

	/** Returns conversion from given Requirements to archetype's fragment indices */
	void GetRequirementsFragmentMapping(TConstArrayView<FMassFragmentRequirementDescription> Requirements, FMassFragmentIndicesMapping& OutFragmentIndices) const;

	/** Returns conversion from given ChunkRequirements to archetype's chunk fragment indices */
	void GetRequirementsChunkFragmentMapping(TConstArrayView<FMassFragmentRequirementDescription> ChunkRequirements, FMassFragmentIndicesMapping& OutFragmentIndices) const;

	/** Returns conversion from given const shared requirements to archetype's const shared fragment indices */
	void GetRequirementsConstSharedFragmentMapping(TConstArrayView<FMassFragmentRequirementDescription> Requirements, FMassFragmentIndicesMapping& OutFragmentIndices) const;

	/** Returns conversion from given shared requirements to archetype's shared fragment indices */
	void GetRequirementsSharedFragmentMapping(TConstArrayView<FMassFragmentRequirementDescription> Requirements, FMassFragmentIndicesMapping& OutFragmentIndices) const;

	SIZE_T GetAllocatedSize() const;

	/** Sum of FChunkSparseElements::GetAllocatedSize across all chunks; subset of GetAllocatedSize. */
	SIZE_T GetSparseElementsBookkeepingAllocatedSize() const;

	void ExportEntityHandles(const TConstArrayView<FMassArchetypeEntityCollection::FArchetypeEntityRange> Ranges, TArray<FMassEntityHandle>& InOutHandles) const;

	void ExportEntityHandles(TArray<FMassEntityHandle>& InOutHandles) const;

	void ListEntitiesForEachChunk(TFunctionRef<void(TConstArrayView<FMassEntityHandle>)> Function) const;

	// Converts the list of fragments into a user-readable debug string
	FString DebugGetDescription() const;

	/** Copies debug names from another archetype data. */
	void CopyDebugNamesFrom(const FMassArchetypeData& Other)
	{ 
#if WITH_MASSENTITY_DEBUG
		DebugNames = Other.DebugNames;
#endif // WITH_MASSENTITY_DEBUG
	}

#if WITH_MASSENTITY_DEBUG
	/** Fetches how much memory is allocated for active chunks, and how much of that memory is actually occupied */
	void DebugGetEntityMemoryNumbers(SIZE_T& OutActiveChunksMemorySize, SIZE_T& OutActiveEntitiesMemorySize) const;

	/** Adds new debug name associated with the archetype. */
	void AddUniqueDebugName(const FName& Name)
	{
		DebugNames.AddUnique(Name);
	}

	/** Whether the archetype was created using the default name (NAME_None) */
	bool IsUnnamedArchetype() const
	{
		return DebugNames.Num() >= 1 && DebugNames[0].IsNone();
	}

	/** @return array of debug names associated with this archetype. */
	TConstArrayView<FName> GetDebugNames() const
	{
		// For unnamed archetype we chop the "None" entry so the first name will be the unique identifier from the composition hash
		const TConstArrayView<FName> AllNames = DebugNames;
		return IsUnnamedArchetype() ? AllNames.RightChop(1) : AllNames;
	}

	/** @return string of all debug names combined */
	FString GetCombinedDebugNamesAsString() const;

	/**
	 * Prints out debug information about the archetype
	 */
	void DebugPrintArchetype(FOutputDevice& Ar);

	/**
	 * Prints out fragment's values for the specified entity. 
	 * @param Entity The entity for which we want to print fragment values
	 * @param Ar The output device
	 * @param InPrefix Optional prefix to remove from fragment names
	 */
	void DebugPrintEntity(FMassEntityHandle Entity, FOutputDevice& Ar, const TCHAR* InPrefix = TEXT("")) const;

	MASSENTITY_API const FMassArchetypeChunk& DebugGetChunk(const int32 Index) const;
#endif // WITH_MASSENTITY_DEBUG

	void SetDebugColor(const FColor InDebugColor);

	void REMOVEME_GetArrayViewForFragmentInChunk(int32 ChunkIndex, const UScriptStruct* FragmentType, void*& OutChunkBase, int32& OutNumEntities);

	//////////////////////////////////////////////////////////////////////
	// low level api
	FORCEINLINE const int32* GetFragmentIndex(const UScriptStruct* FragmentType) const
	{
		return FragmentIndexMap.Find(FragmentType);
	}
	FORCEINLINE int32 GetFragmentIndexChecked(const UScriptStruct* FragmentType) const
	{
		return FragmentIndexMap.FindChecked(FragmentType);
	}

	FORCEINLINE void* GetFragmentData(const int32 FragmentIndex, const FMassRawEntityInChunkData RawEntityInChunkHandle) const
	{
		return FragmentConfigs[FragmentIndex].GetFragmentData(RawEntityInChunkHandle.ChunkRawMemory, RawEntityInChunkHandle.IndexWithinChunk);
	}

	FORCEINLINE bool IsValidHandle(const FMassEntityInChunkDataHandle Handle) const
	{
		return Handle.IsSet() && Chunks.IsValidIndex(Handle.ChunkIndex) && Chunks[Handle.ChunkIndex].GetSerialModificationNumber() == Handle.ChunkSerialNumber;
	}

	FORCEINLINE void* GetFragmentData(const int32 FragmentIndex, const FMassEntityInChunkDataHandle EntityInChunkHandle) const
	{
		checkf(IsValidHandle(EntityInChunkHandle), TEXT("Input FMassRawEntityInChunkData is out of date."));
		return FragmentConfigs[FragmentIndex].GetFragmentData(EntityInChunkHandle.ChunkRawMemory, EntityInChunkHandle.IndexWithinChunk);
	}

	FORCEINLINE FMassRawEntityInChunkData MakeRawEntityHandle(int32 EntityIndex) const
	{
		const int32 AbsoluteIndex = EntityMap.FindChecked(EntityIndex);
		const int32 ChunkIndex = AbsoluteIndex / NumEntitiesPerChunk;
	
		return FMassRawEntityInChunkData(Chunks[ChunkIndex].GetRawMemory(), AbsoluteIndex - (NumEntitiesPerChunk * ChunkIndex));
	}

	FORCEINLINE FMassRawEntityInChunkData MakeRawEntityHandle(const FMassEntityHandle Entity) const
	{
		return MakeRawEntityHandle(Entity.Index); 
	}

	FORCEINLINE FMassEntityInChunkDataHandle MakeEntityHandle(int32 EntityIndex) const
	{
		const int32 AbsoluteIndex = EntityMap.FindChecked(EntityIndex);
		const int32 ChunkIndex = AbsoluteIndex / NumEntitiesPerChunk;
		checkf(Chunks.IsValidIndex(ChunkIndex), TEXT("Provided Entity Index %d is not hosted by this archetype"), EntityIndex);
		const FMassArchetypeChunk& Chunk = Chunks[ChunkIndex];

		return FMassEntityInChunkDataHandle(Chunk.GetRawMemory(), AbsoluteIndex - (NumEntitiesPerChunk * ChunkIndex)
			, ChunkIndex, Chunk.GetSerialModificationNumber());
	}

	FORCEINLINE FMassEntityInChunkDataHandle MakeEntityHandle(const FMassEntityHandle Entity) const
	{
		return MakeEntityHandle(Entity.Index); 
	}

	bool IsInitialized() const
	{
		return TotalBytesPerEntity > 0 && FragmentConfigs.IsEmpty() == false;
	}

	//////////////////////////////////////////////////////////////////////
	// batched api
	void BatchDestroyEntityChunks(FMassArchetypeEntityCollection::FConstEntityRangeArrayView EntityRangeContainer, TArray<FMassEntityHandle>& OutEntitiesRemoved);
	void BatchAddEntities(TConstArrayView<FMassEntityHandle> Entities, const FMassArchetypeSharedFragmentValues& SharedFragmentValues
		, TArray<FMassArchetypeEntityCollection::FArchetypeEntityRange>& OutNewRanges);
	/** 
	 * @param SharedFragmentValuesOverride if provided will override shared fragment values for the entities being moved
	 */
	void BatchMoveEntitiesToAnotherArchetype(const FMassArchetypeEntityCollection& EntityCollection, FMassArchetypeData& NewArchetype
		, TArray<FMassEntityHandle>& OutEntitiesBeingMoved, TArray<FMassArchetypeEntityCollection::FArchetypeEntityRange>* OutNewChunks = nullptr
		, const FMassArchetypeSharedFragmentValues* SharedFragmentValuesToAdd = nullptr
		, const FMassSharedFragmentBitSet* SharedFragmentToRemoveBitSet = nullptr
		, const FMassConstSharedFragmentBitSet* ConstSharedFragmentToRemoveBitSet = nullptr);
	void BatchSetFragmentValues(TConstArrayView<FMassArchetypeEntityCollection::FArchetypeEntityRange> EntityCollection, const FMassGenericPayloadViewSlice& Payload);

	/** Works with all sparse element types (fragments and tags). */
	void BatchAddSparseElementToEntities(const FMassArchetypeEntityCollection& EntityCollection, TNotNull<const UScriptStruct*> ElementType);
	/** Works with all sparse element types (fragments and tags). */
	void BatchRemoveSparseElementFromEntities(const FMassArchetypeEntityCollection& EntityCollection, TNotNull<const UScriptStruct*> ElementType);

	template<typename TSharedStruct>
	requires std::is_same_v<typename TDecay<TSharedStruct>::Type, FSharedStruct> || std::is_same_v<typename TDecay<TSharedStruct>::Type, FConstSharedStruct>
	void SetSharedFragmentsData(const FMassEntityHandle Entity, const TArray<TSharedStruct>& SharedFragmentValueOverrides)
	{
		SetSharedFragmentsData(Entity, MakeArrayView(SharedFragmentValueOverrides));
	}

	/**
	 * The function first creates new FMassArchetypeSharedFragmentValues instance combining existing values
	 * and the contents of SharedFragmentValueOverrides. Then that is used to find the target chunk for Entity,
	 * and if one cannot be found a new one will be created.
	 * Note that if SharedFragmentValueOverrides contains types not already present in archetype's composition,
	 * then those elements will be ignored (there will be an ensure failing).
	 * @param SharedFragmentValueOverrides is expected to contain only instance of types already
	 *    present in given archetypes FMassArchetypeSharedFragmentValues
	 */
	template<typename TSharedStruct>
	requires std::is_same_v<typename TDecay<TSharedStruct>::Type, FSharedStruct> || std::is_same_v<typename TDecay<TSharedStruct>::Type, FConstSharedStruct>
	void SetSharedFragmentsData(const FMassEntityHandle Entity, TArrayView<TSharedStruct> SharedFragmentValueOverrides)
	{
		FMassEntityInChunkDataHandle EntityInChunkHandle = MakeEntityHandle(Entity.Index);

		// Gets or adds a new chunk that will hold the new entity with the new shared values
		FMassArchetypeSharedFragmentValues NewSharedFragmentValues(Chunks[EntityInChunkHandle.ChunkIndex].GetSharedFragmentValues());
		NewSharedFragmentValues.ReplaceSharedFragments<TSharedStruct>(SharedFragmentValueOverrides);
		NewSharedFragmentValues.Sort();

		SetSharedFragmentsData(Entity, EntityInChunkHandle, MoveTemp(NewSharedFragmentValues));
	}

protected:
	MASSENTITY_API void SetSharedFragmentsData(FMassEntityHandle Entity, FMassEntityInChunkDataHandle EntityInChunkHandle, FMassArchetypeSharedFragmentValues&& SharedFragmentValues);

	FMassArchetypeEntityCollection::FArchetypeEntityRange PrepareNextEntitiesSpanInternal(TConstArrayView<FMassEntityHandle> Entities, const FMassArchetypeSharedFragmentValues& InSharedFragmentValues, const int32 StartingChunk = 0);
	void BatchRemoveEntitiesInternal(const int32 ChunkIndex, const int32 StartIndexWithinChunk, const int32 NumberToRemove);

	struct FTransientChunkLocation
	{
		uint8* RawChunkMemory;
		int32 IndexWithinChunk;
	};

	void MoveFragmentsToAnotherArchetypeInternal(FMassArchetypeData& TargetArchetype, FMassArchetypeChunk& TargetChunk, int32 NewIndexWithinChunk
		, FMassArchetypeChunk& SourceChunk, int32 OriginalIndexWithinChunk, int32 ElementsNum);
	void MoveFragmentsToNewLocationInternal(FTransientChunkLocation Target, const FTransientChunkLocation Source, const int32 NumberToMove);
	void ConfigureFragments(const FMassEntityManager& EntityManager);

	FORCEINLINE void* GetFragmentData(const int32 FragmentIndex, uint8* ChunkRawMemory, const int32 IndexWithinChunk) const
	{
		return FragmentConfigs[FragmentIndex].GetFragmentData(ChunkRawMemory, IndexWithinChunk);
	}

	void BindEntityRequirements(FMassExecutionContext& RunContext, const FMassFragmentIndicesMapping& EntityFragmentsMapping, FMassArchetypeChunk& Chunk, const int32 SubchunkStart, const int32 SubchunkLength);
	void BindChunkFragmentRequirements(FMassExecutionContext& RunContext, const FMassFragmentIndicesMapping& ChunkFragmentsMapping, FMassArchetypeChunk& Chunk);
	void BindConstSharedFragmentRequirements(FMassExecutionContext& RunContext, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, const FMassFragmentIndicesMapping& ChunkFragmentsMapping);
	void BindSharedFragmentRequirements(FMassExecutionContext& RunContext, FMassArchetypeSharedFragmentValues& SharedFragmentValues, const FMassFragmentIndicesMapping& ChunkFragmentsMapping);

	FMassArchetypeChunk& GetOrAddChunk(const FMassArchetypeSharedFragmentValues& SharedFragmentValues, int32& OutAbsoluteIndex, int32& OutIndexWithinChunk);
	
private:
	int32 AddEntityInternal(FMassEntityHandle Entity, const FMassArchetypeSharedFragmentValues& InSharedFragmentValues);
	void RemoveEntityInternal(const int32 AbsoluteIndex);
};


struct FMassArchetypeHelper
{
	FORCEINLINE static FMassArchetypeData* ArchetypeDataFromHandle(const FMassArchetypeHandle& ArchetypeHandle)
	{
		return ArchetypeHandle.DataPtr.Get();
	}
	FORCEINLINE static FMassArchetypeData& ArchetypeDataFromHandleChecked(const FMassArchetypeHandle& ArchetypeHandle)
	{
		check(ArchetypeHandle.IsValid());
		return *ArchetypeHandle.DataPtr.Get();
	}
	FORCEINLINE static FMassArchetypeHandle ArchetypeHandleFromData(const TSharedPtr<FMassArchetypeData>& Archetype)
	{
		return FMassArchetypeHandle(Archetype);
	}

	/**
	 * Determines whether given Archetype matches given Requirements. In case of failure to match and if WITH_MASSENTITY_DEBUG
	 * the function will also log the reasons for said failure (at VeryVerbose level).
	 * @param bBailOutOnFirstFail if true will skip the remaining tests as soon as a single mismatch is detected. This option
	 *	is used when looking for matching archetypes. For debugging purposes use `false` to list all the mismatching elements.
	 */
#if WITH_MASSENTITY_DEBUG
	MASSENTITY_API static bool DoesArchetypeMatchRequirements(const FMassArchetypeData& Archetype, const FMassFragmentRequirements& Requirements
		, const bool bBailOutOnFirstFail = true, FOutputDevice* OutputDevice = nullptr);
#endif // WITH_MASSENTITY_DEBUG

	MASSENTITY_API static bool DoesArchetypeMatchRequirements(const FMassArchetypeData& Archetype, const FMassFragmentRequirements& Requirements);
	MASSENTITY_API static bool DoesArchetypeMatchRequirements(const FMassArchetypeCompositionDescriptor& ArchetypeComposition, const FMassFragmentRequirements& Requirements);
	MASSENTITY_API static bool DoesArchetypeMatchRequirements(const FMassElementBitSet& ArchetypeCompositionBitSet, const FMassFragmentRequirements& Requirements);
};

//-----------------------------------------------------------------------------
// INLINES
//-----------------------------------------------------------------------------
inline const UE::Mass::FArchetypeGroups& FMassArchetypeData::GetGroups() const
{
	return Groups;
}

inline bool FMassArchetypeData::IsInGroup(const UE::Mass::FArchetypeGroupHandle GroupHandle) const
{
	if (GroupHandle.IsValid())
	{
		UE::Mass::FArchetypeGroupID FoundGroupID = Groups.GetID(GroupHandle.GetGroupType());
		return FoundGroupID.IsValid() && FoundGroupID == GroupHandle.GetGroupID();
	}
	return false;
}

inline bool FMassArchetypeData::IsInGroupOfType(const UE::Mass::FArchetypeGroupType GroupType) const
{
	return Groups.ContainsType(GroupType);
}

inline uint32 FMassArchetypeData::GetCreatedArchetypeDataVersion() const
{
	return CreatedArchetypeDataVersion;
}

inline uint32 FMassArchetypeData::GetEntityOrderVersion() const
{
	return EntityOrderVersion;
}

inline uint32 FMassArchetypeData::GetArchetypeVersion() const
{
	return ArchetypeVersion;
}

inline const FMassElementBitSet& FMassArchetypeData::GetCompositionBitSet() const
{
	return CompositionBitSet;
}

inline FMassArchetypeCompositionDescriptor FMassArchetypeData::GetCompositionDescriptor() const
{
	return FMassArchetypeCompositionDescriptor(CompositionBitSet);
}

template<typename TStruct>
void FMassArchetypeData::SetFragmentsData(const FMassEntityHandle Entity, TConstArrayView<TStruct> FragmentSources)
{
	FMassEntityInChunkDataHandle EntityInChunkHandle = MakeEntityHandle(Entity);

	for (const TStruct& Instance : FragmentSources)
	{
		const UScriptStruct* FragmentType = Instance.GetScriptStruct();
		check(FragmentType);

		if (UE::Mass::IsSparse(FragmentType) == false)
		{
			const int32 FragmentIndex = FragmentIndexMap.FindChecked(FragmentType);
			void* FragmentMemory = GetFragmentData(FragmentIndex, EntityInChunkHandle);
			FragmentType->CopyScriptStruct(FragmentMemory, Instance.GetMemory());
		}
		else
		{
			bSparseElementsBitSetDirty = true;
			UE::Mass::FChunkSparseElements& SparseElements = Chunks[EntityInChunkHandle.ChunkIndex].GetSparseElements(NumEntitiesPerChunk);
			SparseElements.Add(EntityInChunkHandle, FragmentType);
			FStructView _ = SparseElementsStorage->AddElementInstanceToEntity(Entity, Instance);
		}
	}
}

template<typename TStruct>
void FMassArchetypeData::SetFragmentData(FMassArchetypeEntityCollection::FConstEntityRangeArrayView EntityRangeContainer, const TStruct& FragmentSource)
{
	check(FragmentSource.IsValid());
	const UScriptStruct* FragmentType = FragmentSource.GetScriptStruct();
	check(FragmentType);

	if (UE::Mass::IsSparse(FragmentType) == false)
	{
		const int32* FragmentIndex = FragmentIndexMap.Find(FragmentType);
		if (LIKELY(FragmentIndex))
		{
			const int32 FragmentTypeSize = FragmentType->GetStructureSize();
			const uint8* FragmentSourceMemory = FragmentSource.GetMemory();
			check(FragmentSourceMemory);
	
			for (FMassArchetypeChunkIterator ChunkIterator(EntityRangeContainer); ChunkIterator; ++ChunkIterator)
			{
				uint8* FragmentMemory = static_cast<uint8*>(FragmentConfigs[*FragmentIndex].GetFragmentData(Chunks[ChunkIterator->ChunkIndex].GetRawMemory(), ChunkIterator->SubchunkStart));
				// Note that we cannot handle the copying with a single FragmentType->CopyScriptStruct call
				// because we have a single FragmentSource instance we're using as a copy source (i.e. we don't have a source array).
				for (int i = ChunkIterator->Length; i; --i, FragmentMemory += FragmentTypeSize)
				{
					FragmentType->CopyScriptStruct(FragmentMemory, FragmentSourceMemory);
				}
			}
		}
		else
		{
			UE_LOGF(LogMass, Warning
				, "Attempting to set value of fragment of type %ls, while it's not part of the archetype's composition"
				, *FragmentType->GetName());
		}
	}
	else
	{
		bSparseElementsBitSetDirty = true;
		for (FMassArchetypeChunkIterator ChunkIterator(EntityRangeContainer); ChunkIterator; ++ChunkIterator)
		{
			FMassArchetypeChunk& Chunk = Chunks[ChunkIterator->ChunkIndex];
			const FMassArchetypeEntityCollection::FArchetypeEntityRange& EntityRange = *ChunkIterator;

			UE::Mass::FChunkSparseElements& SparseElements = Chunk.GetSparseElements(NumEntitiesPerChunk);
			SparseElements.BatchAdd(FragmentType, EntityRange.SubchunkStart, EntityRange.Length);

			const FMassEntityHandle* EntitiesArray = Chunk.GetEntityArray(EntityListOffsetWithinChunk);
			TArrayView<const FMassEntityHandle> EntityHandlesView(&EntitiesArray[EntityRange.SubchunkStart], EntityRange.Length);
			SparseElementsStorage->BatchAddElementInstancesToEntities(EntityHandlesView, FragmentSource);
		}
	}
}

inline bool FMassArchetypeData::ContainsAnySparseData() const
{
	return !GetSparseElementsBitSet().IsEmpty();
}

inline bool FMassArchetypeData::ContainsSparseElement(TNotNull<const UScriptStruct*> ElementType) const
{
	return GetSparseElementsBitSet().Contains(ElementType);
}

inline const FMassElementBitSet& FMassArchetypeData::GetSparseElementsBitSet() const
{
	UE_MT_SCOPED_READ_ACCESS(ChunkIterationDetector);

	if (bSparseElementsBitSetDirty)
	{
		CachedSparseElementsBitSet.Reset();
		for (const FMassArchetypeChunk& Chunk : Chunks)
		{
			if (Chunk.HasSparseElements())
			{
				CachedSparseElementsBitSet += Chunk.GetSparseElementsUnsafe().GetSparseElementsPresent();
			}
		}
		bSparseElementsBitSetDirty = false;
	}
	return CachedSparseElementsBitSet;
}
