// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MassEntity : ModuleRules
{
	public MassEntity(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"DeveloperSettings",
				"MassCore",
				"TraceLog",
			}
		);

		if (Target.bBuildEditor || Target.bCompileAgainstEditor)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
			PrivateDependencyModuleNames.Add("EditorSubsystem");
		}

		if (Target.bBuildDeveloperTools)
		{
			DynamicallyLoadedModuleNames.Add("MassEntityTestSuite");
		}
	}
}
