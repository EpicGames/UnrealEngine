<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!--
  *
  * MassEntity Visualizers
  *
  * Conventions:
  *   - Use "(*ptr).member" in unquoted intrinsic expressions
  *   - Use "ptr->member" in quoted DisplayString expressions
  *   - Call intrinsics with "this->" for cross-platform compatibility
  *   - First matching DisplayString wins (most specific conditions first)
  *
  -->

  <!-- ===================================================================== -->
  <!-- FMassEntityHandle                                                      -->
  <!-- ===================================================================== -->
  <Type Name="FMassEntityHandle">
    <DisplayString Condition="Index == 0 &amp;&amp; SerialNumber == 0">Entity[null]</DisplayString>
    <DisplayString>Entity[{Index}:{SerialNumber}]</DisplayString>
    <Expand>
      <Item Name="Index">Index</Item>
      <Item Name="SerialNumber">SerialNumber</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassArchetypeHandle                                                   -->
  <!-- ===================================================================== -->
  <Type Name="FMassArchetypeHandle">
    <DisplayString Condition="DataPtr.Object == 0">Archetype[null]</DisplayString>
    <DisplayString>Archetype @ {(void*)DataPtr.Object}</DisplayString>
    <Expand>
      <ExpandedItem Condition="DataPtr.Object != 0">*DataPtr.Object</ExpandedItem>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassArchetypeVersionedHandle                                          -->
  <!-- ===================================================================== -->
  <Type Name="FMassArchetypeVersionedHandle">
    <DisplayString Condition="ArchetypeHandle.DataPtr.Object == 0">VersionedArchetype[null]</DisplayString>
    <DisplayString>VersionedArchetype[v{HandleVersion}] @ {(void*)ArchetypeHandle.DataPtr.Object}</DisplayString>
    <Expand>
      <Item Name="ArchetypeHandle">ArchetypeHandle</Item>
      <Item Name="HandleVersion">HandleVersion</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassArchetypeData                                                     -->
  <!-- ===================================================================== -->
  <Type Name="FMassArchetypeData">
    <DisplayString>{Chunks.ArrayNum} chunks, {NumEntitiesPerChunk}/chunk, {TotalBytesPerEntity} bytes/entity</DisplayString>
    <Expand>
      <Item Name="NumEntitiesPerChunk">NumEntitiesPerChunk</Item>
      <Item Name="TotalBytesPerEntity">TotalBytesPerEntity</Item>
      <Item Name="ChunkMemorySize">ChunkMemorySize</Item>
      <Item Name="EntityOrderVersion">EntityOrderVersion</Item>
      <Item Name="CreatedArchetypeDataVersion">CreatedArchetypeDataVersion</Item>
      <Item Name="[Composition]">CompositionBitSet</Item>
      <Item Name="[FragmentConfigs]">FragmentConfigs</Item>
      <Item Name="[Chunks]">Chunks</Item>
      <Item Name="[EntityMap]">EntityMap</Item>
      <Item Name="[FragmentIndexMap]">FragmentIndexMap</Item>
      <Item Name="[Groups]">Groups</Item>
      <Item Name="[DebugNames]" Optional="true">DebugNames</Item>
      <Item Name="[DebugColor]" Optional="true">DebugColor</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassArchetypeChunk                                                    -->
  <!-- ===================================================================== -->
  <Type Name="FMassArchetypeChunk">
    <DisplayString Condition="NumInstances == 0">Chunk[empty, {AllocSize} bytes]</DisplayString>
    <DisplayString>{NumInstances} entities, {AllocSize} bytes</DisplayString>
    <Expand>
      <Item Name="NumInstances">NumInstances</Item>
      <Item Name="AllocSize">AllocSize</Item>
      <Item Name="SerialModificationNumber">SerialModificationNumber</Item>
      <Item Name="RawMemory">RawMemory</Item>
      <Item Name="[ChunkFragmentData]">ChunkFragmentData</Item>
      <Item Name="[SharedFragmentValues]">SharedFragmentValues</Item>
      <Item Name="[SparseElements]">SparseElements</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassArchetypeFragmentConfig                                           -->
  <!-- ===================================================================== -->
  <Type Name="FMassArchetypeFragmentConfig">
    <DisplayString Condition="FragmentType != 0">{FragmentType->NamePrivate} @ offset {ArrayOffsetWithinChunk}</DisplayString>
    <DisplayString>null</DisplayString>
    <Expand>
      <Item Name="FragmentType">FragmentType</Item>
      <Item Name="ArrayOffsetWithinChunk">ArrayOffsetWithinChunk</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassArchetypeEntityCollection                                         -->
  <!-- ===================================================================== -->
  <Type Name="FMassArchetypeEntityCollection">
    <DisplayString Condition="Ranges.ArrayNum == 0 &amp;&amp; Archetype.ArchetypeHandle.DataPtr.Object == 0">EntityCollection[empty]</DisplayString>
    <DisplayString>{Ranges.ArrayNum} ranges</DisplayString>
    <Expand>
      <Item Name="[Ranges]">Ranges</Item>
      <Item Name="Archetype">Archetype</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassArchetypeEntityCollection::FArchetypeEntityRange                  -->
  <!-- ===================================================================== -->
  <Type Name="FMassArchetypeEntityCollection::FArchetypeEntityRange">
    <DisplayString Condition="ChunkIndex == -1">Range[invalid]</DisplayString>
    <DisplayString Condition="Length &lt;= 0">Chunk[{ChunkIndex}] [{SubchunkStart}..end)</DisplayString>
    <DisplayString>Chunk[{ChunkIndex}] [{SubchunkStart}..{SubchunkStart + Length})</DisplayString>
    <Expand>
      <Item Name="ChunkIndex">ChunkIndex</Item>
      <Item Name="SubchunkStart">SubchunkStart</Item>
      <Item Name="Length">Length</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassArchetypeChunkIterator                                            -->
  <!-- ===================================================================== -->
  <Type Name="FMassArchetypeChunkIterator">
    <DisplayString>ChunkIter[{CurrentChunkIndex}/{EntityRanges.ArrayNum}]</DisplayString>
    <Expand>
      <Item Name="CurrentChunkIndex">CurrentChunkIndex</Item>
      <Item Name="EntityRanges">EntityRanges</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassRawEntityInChunkData                                              -->
  <!-- ===================================================================== -->
  <Type Name="FMassRawEntityInChunkData">
    <DisplayString Condition="ChunkRawMemory == 0 || IndexWithinChunk == -1">ChunkData[null]</DisplayString>
    <DisplayString>ChunkData[idx:{IndexWithinChunk}]</DisplayString>
    <Expand>
      <Item Name="ChunkRawMemory">ChunkRawMemory</Item>
      <Item Name="IndexWithinChunk">IndexWithinChunk</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassEntityInChunkDataHandle                                           -->
  <!-- ===================================================================== -->
  <Type Name="FMassEntityInChunkDataHandle">
    <DisplayString Condition="ChunkRawMemory == 0 || IndexWithinChunk == -1">ChunkHandle[null]</DisplayString>
    <DisplayString>ChunkHandle[chunk:{ChunkIndex}, idx:{IndexWithinChunk}, sn:{ChunkSerialNumber}]</DisplayString>
    <Expand>
      <Item Name="ChunkIndex">ChunkIndex</Item>
      <Item Name="IndexWithinChunk">IndexWithinChunk</Item>
      <Item Name="ChunkSerialNumber">ChunkSerialNumber</Item>
      <Item Name="ChunkRawMemory">ChunkRawMemory</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassEntityView                                                        -->
  <!-- ===================================================================== -->
  <Type Name="FMassEntityView">
    <DisplayString Condition="Entity.Index == 0 &amp;&amp; Entity.SerialNumber == 0">EntityView[null]</DisplayString>
    <DisplayString>EntityView[{Entity.Index}:{Entity.SerialNumber}]</DisplayString>
    <Expand>
      <Item Name="Entity">Entity</Item>
      <Item Name="Archetype">Archetype</Item>
      <Item Name="EntityDataHandle">EntityDataHandle</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassFragmentRequirementDescription                                    -->
  <!-- ===================================================================== -->
  <Type Name="FMassFragmentRequirementDescription">
    <DisplayString Condition="StructType != 0">{StructType->NamePrivate} [{AccessMode}|{Presence}]</DisplayString>
    <DisplayString>null</DisplayString>
    <Expand>
      <Item Name="StructType">StructType</Item>
      <Item Name="AccessMode">AccessMode</Item>
      <Item Name="Presence">Presence</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassEntityQuery                                                       -->
  <!-- ===================================================================== -->
  <Type Name="FMassEntityQuery">
    <DisplayString>{ValidArchetypes.ArrayNum} archetypes, {FragmentRequirements.ArrayNum} requirements</DisplayString>
    <Expand>
      <Item Name="[ValidArchetypes]">ValidArchetypes</Item>
      <Item Name="[FragmentRequirements]">FragmentRequirements</Item>
      <Item Name="[ChunkFragmentRequirements]">ChunkFragmentRequirements</Item>
      <Item Name="[SharedFragmentRequirements]">SharedFragmentRequirements</Item>
      <Item Name="[ConstSharedFragmentRequirements]">ConstSharedFragmentRequirements</Item>
      <Item Name="[RequiredAllFragments]">RequiredAllFragments</Item>
      <Item Name="[RequiredAllTags]">RequiredAllTags</Item>
      <Item Name="[RequiredNoneFragments]">RequiredNoneFragments</Item>
      <Item Name="[RequiredNoneTags]">RequiredNoneTags</Item>
      <Item Name="EntitySubsystemHash">EntitySubsystemHash</Item>
      <Item Name="LastUpdatedArchetypeDataVersion">LastUpdatedArchetypeDataVersion</Item>
      <Item Name="bRequiresMutatingWorldAccess">bRequiresMutatingWorldAccess</Item>
      <Item Name="[ArchetypeFragmentMapping]">ArchetypeFragmentMapping</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassFragmentRequirements                                              -->
  <!-- ===================================================================== -->
  <Type Name="FMassFragmentRequirements">
    <DisplayString>{FragmentRequirements.ArrayNum} fragments, {ChunkFragmentRequirements.ArrayNum} chunk frags</DisplayString>
    <Expand>
      <Item Name="[FragmentRequirements]">FragmentRequirements</Item>
      <Item Name="[ChunkFragmentRequirements]">ChunkFragmentRequirements</Item>
      <Item Name="[SharedFragmentRequirements]">SharedFragmentRequirements</Item>
      <Item Name="[ConstSharedFragmentRequirements]">ConstSharedFragmentRequirements</Item>
      <Item Name="[RequiredAllFragments]">RequiredAllFragments</Item>
      <Item Name="[RequiredAnyFragments]">RequiredAnyFragments</Item>
      <Item Name="[RequiredOptionalFragments]">RequiredOptionalFragments</Item>
      <Item Name="[RequiredNoneFragments]">RequiredNoneFragments</Item>
      <Item Name="[RequiredAllTags]">RequiredAllTags</Item>
      <Item Name="[RequiredAnyTags]">RequiredAnyTags</Item>
      <Item Name="[RequiredNoneTags]">RequiredNoneTags</Item>
      <Item Name="[RequiredOptionalTags]">RequiredOptionalTags</Item>
      <Item Name="[RequiredAllSparseElements]">RequiredAllSparseElements</Item>
      <Item Name="[RequiredAnySparseElements]">RequiredAnySparseElements</Item>
      <Item Name="[RequiredNoneSparseElements]">RequiredNoneSparseElements</Item>
      <Item Name="bInitialized">bInitialized</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassSubsystemRequirements                                             -->
  <!-- ===================================================================== -->
  <Type Name="FMassSubsystemRequirements">
    <DisplayString>SubsystemReqs[const:{RequiredConstSubsystems}, mutable:{RequiredMutableSubsystems}]</DisplayString>
    <Expand>
      <Item Name="[RequiredConstSubsystems]">RequiredConstSubsystems</Item>
      <Item Name="[RequiredMutableSubsystems]">RequiredMutableSubsystems</Item>
      <Item Name="bRequiresGameThreadExecution">bRequiresGameThreadExecution</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassArchetypeCompositionDescriptor                                    -->
  <!-- ===================================================================== -->
  <Type Name="FMassArchetypeCompositionDescriptor">
    <DisplayString Condition="ElementsBitSet.StructTypesBitArray.NumBits == 0">Composition[empty]</DisplayString>
    <DisplayString>Composition[{ElementsBitSet.StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <Item Name="ElementsBitSet">ElementsBitSet</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassQueryRequirementIndicesMapping                                    -->
  <!-- ===================================================================== -->
  <Type Name="FMassQueryRequirementIndicesMapping">
    <DisplayString>Mapping[entity:{EntityFragments.ArrayNum}, chunk:{ChunkFragments.ArrayNum}, shared:{SharedFragments.ArrayNum}, const:{ConstSharedFragments.ArrayNum}]</DisplayString>
    <Expand>
      <Item Name="[EntityFragments]">EntityFragments</Item>
      <Item Name="[ChunkFragments]">ChunkFragments</Item>
      <Item Name="[ConstSharedFragments]">ConstSharedFragments</Item>
      <Item Name="[SharedFragments]">SharedFragments</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- Mass Bitset Types                                                      -->
  <!--                                                                        -->
  <!-- All inherit from TTypeBitSetBase which stores StructTypesBitArray       -->
  <!-- (a TBitArray). We show the bit count and delegate expansion to the      -->
  <!-- existing TBitArray natvis.                                              -->
  <!-- ===================================================================== -->

  <!-- FElementBitSet: fancy version — shows only set bits with type names     -->
  <!-- from the global StructTracker.DebugStructTypeNamesList.                 -->
  <!-- Requires WITH_STRUCTUTILS_DEBUG (always on in Debug/Development).       -->
  <!-- Falls through to MediumLow entry if the global symbol can't resolve.   -->
  <Type Name="UE::Mass::FElementBitSet" Priority="Medium">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">ElementBitSet[empty]</DisplayString>
    <DisplayString>ElementBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <CustomListItems>
        <Variable Name="i" InitialValue="0"/>
        <Variable Name="numBits" InitialValue="StructTypesBitArray.NumBits"/>
        <Variable Name="bitData" InitialValue="(unsigned int*)(StructTypesBitArray.AllocatorInstance.SecondaryData.Data != 0 ? (void*)StructTypesBitArray.AllocatorInstance.SecondaryData.Data : (void*)StructTypesBitArray.AllocatorInstance.InlineData)"/>
        <Variable Name="nameData" InitialValue="(FName*)(UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data != 0 ? (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data : (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.InlineData)"/>
        <Loop Condition="i &lt; numBits">
          <If Condition="(bitData[i/32] &gt;&gt; (i%32)) &amp; 1">
            <Item Name="[{i}]">nameData[i]</Item>
          </If>
          <Exec>++i</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>

  <!-- FElementBitSet: fallback — raw bit array expansion (no type names)     -->
  <Type Name="UE::Mass::FElementBitSet" Priority="MediumLow">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">ElementBitSet[empty]</DisplayString>
    <DisplayString>ElementBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <ExpandedItem>StructTypesBitArray</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::Mass::FFragmentBitSet" Priority="Medium">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">FragmentBitSet[empty]</DisplayString>
    <DisplayString>FragmentBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <CustomListItems>
        <Variable Name="i" InitialValue="0"/>
        <Variable Name="numBits" InitialValue="StructTypesBitArray.NumBits"/>
        <Variable Name="bitData" InitialValue="(unsigned int*)(StructTypesBitArray.AllocatorInstance.SecondaryData.Data != 0 ? (void*)StructTypesBitArray.AllocatorInstance.SecondaryData.Data : (void*)StructTypesBitArray.AllocatorInstance.InlineData)"/>
        <Variable Name="nameData" InitialValue="(FName*)(UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data != 0 ? (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data : (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.InlineData)"/>
        <Loop Condition="i &lt; numBits">
          <If Condition="(bitData[i/32] &gt;&gt; (i%32)) &amp; 1">
            <Item Name="[{i}]">nameData[i]</Item>
          </If>
          <Exec>++i</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>
  <Type Name="UE::Mass::FFragmentBitSet" Priority="MediumLow">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">FragmentBitSet[empty]</DisplayString>
    <DisplayString>FragmentBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <ExpandedItem>StructTypesBitArray</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::Mass::FTagBitSet" Priority="Medium">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">TagBitSet[empty]</DisplayString>
    <DisplayString>TagBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <CustomListItems>
        <Variable Name="i" InitialValue="0"/>
        <Variable Name="numBits" InitialValue="StructTypesBitArray.NumBits"/>
        <Variable Name="bitData" InitialValue="(unsigned int*)(StructTypesBitArray.AllocatorInstance.SecondaryData.Data != 0 ? (void*)StructTypesBitArray.AllocatorInstance.SecondaryData.Data : (void*)StructTypesBitArray.AllocatorInstance.InlineData)"/>
        <Variable Name="nameData" InitialValue="(FName*)(UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data != 0 ? (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data : (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.InlineData)"/>
        <Loop Condition="i &lt; numBits">
          <If Condition="(bitData[i/32] &gt;&gt; (i%32)) &amp; 1">
            <Item Name="[{i}]">nameData[i]</Item>
          </If>
          <Exec>++i</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>
  <Type Name="UE::Mass::FTagBitSet" Priority="MediumLow">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">TagBitSet[empty]</DisplayString>
    <DisplayString>TagBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <ExpandedItem>StructTypesBitArray</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::Mass::FChunkFragmentBitSet" Priority="Medium">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">ChunkFragmentBitSet[empty]</DisplayString>
    <DisplayString>ChunkFragmentBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <CustomListItems>
        <Variable Name="i" InitialValue="0"/>
        <Variable Name="numBits" InitialValue="StructTypesBitArray.NumBits"/>
        <Variable Name="bitData" InitialValue="(unsigned int*)(StructTypesBitArray.AllocatorInstance.SecondaryData.Data != 0 ? (void*)StructTypesBitArray.AllocatorInstance.SecondaryData.Data : (void*)StructTypesBitArray.AllocatorInstance.InlineData)"/>
        <Variable Name="nameData" InitialValue="(FName*)(UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data != 0 ? (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data : (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.InlineData)"/>
        <Loop Condition="i &lt; numBits">
          <If Condition="(bitData[i/32] &gt;&gt; (i%32)) &amp; 1">
            <Item Name="[{i}]">nameData[i]</Item>
          </If>
          <Exec>++i</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>
  <Type Name="UE::Mass::FChunkFragmentBitSet" Priority="MediumLow">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">ChunkFragmentBitSet[empty]</DisplayString>
    <DisplayString>ChunkFragmentBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <ExpandedItem>StructTypesBitArray</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::Mass::FSharedFragmentBitSet" Priority="Medium">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">SharedFragmentBitSet[empty]</DisplayString>
    <DisplayString>SharedFragmentBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <CustomListItems>
        <Variable Name="i" InitialValue="0"/>
        <Variable Name="numBits" InitialValue="StructTypesBitArray.NumBits"/>
        <Variable Name="bitData" InitialValue="(unsigned int*)(StructTypesBitArray.AllocatorInstance.SecondaryData.Data != 0 ? (void*)StructTypesBitArray.AllocatorInstance.SecondaryData.Data : (void*)StructTypesBitArray.AllocatorInstance.InlineData)"/>
        <Variable Name="nameData" InitialValue="(FName*)(UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data != 0 ? (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data : (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.InlineData)"/>
        <Loop Condition="i &lt; numBits">
          <If Condition="(bitData[i/32] &gt;&gt; (i%32)) &amp; 1">
            <Item Name="[{i}]">nameData[i]</Item>
          </If>
          <Exec>++i</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>
  <Type Name="UE::Mass::FSharedFragmentBitSet" Priority="MediumLow">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">SharedFragmentBitSet[empty]</DisplayString>
    <DisplayString>SharedFragmentBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <ExpandedItem>StructTypesBitArray</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::Mass::FConstSharedFragmentBitSet" Priority="Medium">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">ConstSharedFragmentBitSet[empty]</DisplayString>
    <DisplayString>ConstSharedFragmentBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <CustomListItems>
        <Variable Name="i" InitialValue="0"/>
        <Variable Name="numBits" InitialValue="StructTypesBitArray.NumBits"/>
        <Variable Name="bitData" InitialValue="(unsigned int*)(StructTypesBitArray.AllocatorInstance.SecondaryData.Data != 0 ? (void*)StructTypesBitArray.AllocatorInstance.SecondaryData.Data : (void*)StructTypesBitArray.AllocatorInstance.InlineData)"/>
        <Variable Name="nameData" InitialValue="(FName*)(UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data != 0 ? (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.SecondaryData.Data : (void*)UE::Mass::FElementBitSetBaseStructTrackerWrapper::StructTracker.DebugStructTypeNamesList.AllocatorInstance.InlineData)"/>
        <Loop Condition="i &lt; numBits">
          <If Condition="(bitData[i/32] &gt;&gt; (i%32)) &amp; 1">
            <Item Name="[{i}]">nameData[i]</Item>
          </If>
          <Exec>++i</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>
  <Type Name="UE::Mass::FConstSharedFragmentBitSet" Priority="MediumLow">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">ConstSharedFragmentBitSet[empty]</DisplayString>
    <DisplayString>ConstSharedFragmentBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <ExpandedItem>StructTypesBitArray</ExpandedItem>
    </Expand>
  </Type>

  <!-- FMassExternalSubsystemBitSet is a using alias for TStructTypeBitSet<USubsystem,...,UClass>.
       This natvis entry may not match if VS resolves the alias to the template type. -->
  <Type Name="FMassExternalSubsystemBitSet" Priority="MediumLow">
    <DisplayString Condition="StructTypesBitArray.NumBits == 0">SubsystemBitSet[empty]</DisplayString>
    <DisplayString>SubsystemBitSet[{StructTypesBitArray.NumBits} bits]</DisplayString>
    <Expand>
      <ExpandedItem>StructTypesBitArray</ExpandedItem>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- UE::Mass::FChunkSparseElements                                         -->
  <!-- ===================================================================== -->
  <Type Name="UE::Mass::FChunkSparseElements">
    <DisplayString Condition="PerEntityElements.ArrayNum == 0">SparseElements[empty]</DisplayString>
    <DisplayString>SparseElements[{PerEntityElements.ArrayNum} slots]</DisplayString>
    <Expand>
      <Item Name="[SparseElementsPresent]">SparseElementsPresent</Item>
      <Item Name="[PerEntityElements]">PerEntityElements</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- UE::Mass::FExecutionLimiter                                            -->
  <!-- ===================================================================== -->
  <Type Name="UE::Mass::FExecutionLimiter">
    <DisplayString>Limiter[limit={EntityLimit}, remaining={EntityCountRemaining}]</DisplayString>
    <Expand>
      <Item Name="EntityLimit">EntityLimit</Item>
      <Item Name="EntityCountRemaining">EntityCountRemaining</Item>
      <Item Name="ChunkIndex">ChunkIndex</Item>
      <Item Name="ArchetypeIndex">ArchetypeIndex</Item>
      <Item Name="MaxChunkIndex">MaxChunkIndex</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassExecutionContext                                                   -->
  <!-- ===================================================================== -->
  <Type Name="FMassExecutionContext">
    <DisplayString>{FragmentViews.ArrayNum} fragments, dt={DeltaTimeSeconds}</DisplayString>
    <Expand>
      <Item Name="DeltaTimeSeconds">DeltaTimeSeconds</Item>
      <Item Name="[FragmentViews]">FragmentViews</Item>
      <Item Name="[ChunkFragmentViews]">ChunkFragmentViews</Item>
      <Item Name="[SharedFragmentViews]">SharedFragmentViews</Item>
      <Item Name="[ConstSharedFragmentViews]">ConstSharedFragmentViews</Item>
      <Item Name="[EntityListView]">EntityListView</Item>
      <Item Name="[EntityCollection]">EntityCollection</Item>
      <Item Name="[CurrentArchetypeComposition]">CurrentArchetypeCompositionBitSet</Item>
      <Item Name="[DeferredCommandBuffer]">DeferredCommandBuffer</Item>
      <Item Name="[QueriesStack]">QueriesStack</Item>
      <Item Name="ExecutionType">ExecutionType</Item>
      <Item Name="bFlushDeferredCommands">bFlushDeferredCommands</Item>
      <Item Name="bHasSparseRequirements">bHasSparseRequirements</Item>
      <Item Name="ArchetypeDataPtr">ArchetypeDataPtr</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassArchetypeEntityCollectionWithPayload                              -->
  <!-- ===================================================================== -->
  <Type Name="FMassArchetypeEntityCollectionWithPayload">
    <DisplayString>{Entities}</DisplayString>
    <Expand>
      <Item Name="Entities">Entities</Item>
      <Item Name="PayloadSlice">PayloadSlice</Item>
    </Expand>
  </Type>

  <!-- ===================================================================== -->
  <!-- FMassChunkProcessingQueueParams                                        -->
  <!-- ===================================================================== -->
  <Type Name="FMassChunkProcessingQueueParams">
    <DisplayString>QueueParams[parallel={bParallelExecution}]</DisplayString>
    <Expand>
      <Item Name="SourceContext">SourceContext</Item>
      <Item Name="SourceQuery">SourceQuery</Item>
      <Item Name="Archetype">Archetype</Item>
      <Item Name="EntityRange">EntityRange</Item>
      <Item Name="bParallelExecution">bParallelExecution</Item>
    </Expand>
  </Type>

</AutoVisualizer>
