// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassArchetypeData.h"
#include "MassEntityTypes.h"
#include "MassExecutionContext.h"
#include "MassEntitySettings.h"
#include "MassDebugger.h"
#include "MassRequirements.h"
#include "Misc/StringBuilder.h"
#include "Misc/StringOutputDevice.h"

DECLARE_CYCLE_STAT(TEXT("Mass Archetype BatchAdd"), STAT_Mass_ArchetypeBatchAdd, STATGROUP_Mass);

namespace UE::Mass
{
	namespace Private
	{
		constexpr int32 UninitializedInt32 = -1;
		constexpr uint32 MinChunkMemorySize = 1024;
		constexpr uint32 MaxChunkMemorySize = 512 * 1024;
	}

	uint32 SanitizeChunkMemorySize(const uint32 InChunkMemorySize, const bool bLogMismatch)
	{
		const uint32 SanitizedSize = FMath::Clamp(InChunkMemorySize, Private::MinChunkMemorySize, Private::MaxChunkMemorySize);
		UE_CLOGF(bLogMismatch && SanitizedSize != InChunkMemorySize, LogMass, Warning
			, "ChunkMemorySize sanitization resulted in changing value. Old: %u, modified: %u"
			, InChunkMemorySize, SanitizedSize);
		return SanitizedSize;
	}

	//-----------------------------------------------------------------------------
	// FChunkSparseElements
	//-----------------------------------------------------------------------------
	bool FChunkSparseElements::Add(const FMassRawEntityInChunkData& EntityInChunkHandle, TNotNull<const UScriptStruct*> ElementType)
	{
		check(IsEmpty() == false);

		const int32 TypeIndex = FMassElementBitSet::GetTypeIndex(ElementType);

		FMassElementBitSet& EntitySparseElements = PerEntityElements[EntityInChunkHandle.IndexWithinChunk];
		if (EntitySparseElements.IsBitSet(TypeIndex) == false)
		{
			EntitySparseElements.AddAtIndex(TypeIndex);
			if (SparseElementTypesCount.Get(TypeIndex)++ == 0)
			{
				SparseElementsPresent.AddAtIndex(TypeIndex);
				return true;
			}
		}
		return false;
	}

	bool FChunkSparseElements::Remove(const FMassRawEntityInChunkData& EntityInChunkHandle, TNotNull<const UScriptStruct*> ElementType)
	{
		check(IsEmpty() == false);

		FMassElementBitSet& EntitySparseElements = PerEntityElements[EntityInChunkHandle.IndexWithinChunk];

		const int32 TypeIndex = FMassElementBitSet::GetTypeIndex(ElementType);
		if (EntitySparseElements.IsBitSet(TypeIndex))
		{
			EntitySparseElements.RemoveAtIndex(TypeIndex);
			if (--SparseElementTypesCount.Get(TypeIndex) == 0)
			{
				SparseElementsPresent.RemoveAtIndex(TypeIndex);
				return true;
			}
		}

		return false;
	}

	bool FChunkSparseElements::UpdateElementsPresenceOnEntityRemoval(const int32 IndexWithinChunk)
	{
		bool bSparseElementsPresentDirty = false;
		for (FMassElementBitSet::FIndexIterator It = PerEntityElements[IndexWithinChunk].GetIndexIterator(); It; ++It)
		{
			if (--SparseElementTypesCount.Get(*It) == 0)
			{
				SparseElementsPresent.RemoveAtIndex(*It);
				bSparseElementsPresentDirty = true;
			}
		}
		PerEntityElements[IndexWithinChunk].Reset();
		return bSparseElementsPresentDirty;
	}

	bool FChunkSparseElements::BatchAdd(TNotNull<const UScriptStruct*> ElementType, const int32 SubchunkStart, const int32 RangeLength)
	{
		check(IsEmpty() == false);
		check(SubchunkStart >= 0 && SubchunkStart + RangeLength <= PerEntityElements.Num());

		uint16 NumAdded = 0;
		const int32 TypeIndex = FMassElementBitSet::GetTypeIndex(ElementType);
		for (int32 EntityIndex = SubchunkStart; EntityIndex < SubchunkStart + RangeLength; ++EntityIndex)
		{
			NumAdded += static_cast<int>(PerEntityElements[EntityIndex].IsBitSet(TypeIndex) == false);
			PerEntityElements[EntityIndex].AddAtIndex(TypeIndex);
		}

		bool bSparseElementsPresentDirty = false;
		if (NumAdded)
		{
			uint16& TotalElementsCount = SparseElementTypesCount.Get(TypeIndex);
			if (TotalElementsCount == 0)
			{
				SparseElementsPresent.AddAtIndex(TypeIndex);
				bSparseElementsPresentDirty = true;
			}
			TotalElementsCount += NumAdded;
		}
		return bSparseElementsPresentDirty;
	}

	bool FChunkSparseElements::BatchRemove(TNotNull<const UScriptStruct*> ElementType, const int32 SubchunkStart, const int32 RangeLength)
	{
		check(IsEmpty() == false);
		check(SubchunkStart >= 0 && SubchunkStart + RangeLength <= PerEntityElements.Num());

		uint16 NumRemoved = 0;
		const int32 TypeIndex = FMassElementBitSet::GetTypeIndex(ElementType);
		for (int32 EntityIndex = SubchunkStart; EntityIndex < SubchunkStart + RangeLength; ++EntityIndex)
		{
			NumRemoved += static_cast<int>(PerEntityElements[EntityIndex].IsBitSet(TypeIndex));
			PerEntityElements[EntityIndex].RemoveAtIndex(TypeIndex);
		}

		bool bSparseElementsPresentDirty = false;
		if (NumRemoved)
		{
			uint16& TotalElementsCount = SparseElementTypesCount.Get(TypeIndex);
			check(TotalElementsCount >= NumRemoved);
			TotalElementsCount -= NumRemoved;
			if (TotalElementsCount == 0)
			{
				SparseElementsPresent.RemoveAtIndex(TypeIndex);
				bSparseElementsPresentDirty = true;
			}
		}
		return bSparseElementsPresentDirty;
	}

	bool FChunkSparseElements::Contains(const FMassRawEntityInChunkData& EntityInChunkHandle, TNotNull<const UScriptStruct*> ElementType) const
	{
		return PerEntityElements.IsValidIndex(EntityInChunkHandle.IndexWithinChunk)
			&& PerEntityElements[EntityInChunkHandle.IndexWithinChunk].Contains(ElementType);
	}

	SIZE_T FChunkSparseElements::GetAllocatedSize() const
	{
		// PerEntityElements is empty when this instance has not been initialized for a chunk yet,
		// in which case nothing has been allocated and there's nothing to count.
		if (PerEntityElements.IsEmpty())
		{
			return 0;
		}

		SIZE_T TotalSize = SparseElementsPresent.GetAllocatedSize()
			+ PerEntityElements.GetAllocatedSize()
			+ SparseElementTypesCount.GetAllocatedSize();

		// Each element of PerEntityElements may carry its own bitset payload allocation
		for (const FMassElementBitSet& EntityElements : PerEntityElements)
		{
			TotalSize += EntityElements.GetAllocatedSize();
		}

		return TotalSize;
	}

	bool FChunkSparseElements::MoveElementsFrom(FChunkSparseElements& OriginalSparseElements, const int32 OriginalIndexWithinChunk, const int32 NewIndexWithinChunk, const int32 Num)
	{
		check(OriginalSparseElements.PerEntityElements.IsValidIndex(OriginalIndexWithinChunk)
			&& OriginalIndexWithinChunk + Num <= OriginalSparseElements.PerEntityElements.Num()
			&& PerEntityElements.IsValidIndex(NewIndexWithinChunk)
			&& NewIndexWithinChunk + Num <= PerEntityElements.Num());

		bool bSparseElementsPresentDirty = false;
		if (&OriginalSparseElements == this)
		{
			// we're moving inside a single chunk, so we need to check the bitsets we're overriding, since they belong to entities
			// that are being moved out of the chunk
			// using NewIndexWithinChunk as the index reference because that's the data that will get overridden 
			for (int32 Index = NewIndexWithinChunk; Index < NewIndexWithinChunk + Num; ++Index)
			{
				bSparseElementsPresentDirty |= UpdateElementsPresenceOnEntityRemoval(Index);
			}
		}

		int32 OriginalIndex = OriginalIndexWithinChunk;
		int32 NewIndex = NewIndexWithinChunk;
		for (int32 ItemCount = 0; ItemCount < Num; ++ItemCount, ++OriginalIndex, ++NewIndex)
		{
			PerEntityElements[NewIndex] = MoveTemp(OriginalSparseElements.PerEntityElements[OriginalIndex]);
		}

		if (&OriginalSparseElements != this)
		{
			for (int32 Index = NewIndexWithinChunk; Index < NewIndexWithinChunk + Num; ++Index)
			{
				for (FMassElementBitSet::FIndexIterator It = PerEntityElements[Index].GetIndexIterator(); It; ++It)
				{
					if (SparseElementTypesCount.Get(*It)++ == 0)
					{
						SparseElementsPresent.AddAtIndex(*It);
						bSparseElementsPresentDirty = true;
					}
					if (--OriginalSparseElements.SparseElementTypesCount.Get(*It) == 0)
					{
						OriginalSparseElements.SparseElementsPresent.RemoveAtIndex(*It);
						bSparseElementsPresentDirty = true;
					}
				}
			}
		}
		return bSparseElementsPresentDirty;
	}
} // namespace UE::Mass

//-----------------------------------------------------------------------------
// FMassArchetypeData
//-----------------------------------------------------------------------------
FMassArchetypeData::FMassArchetypeData(const FMassArchetypeCreationParams& CreationParams)
	: NumEntitiesPerChunk(UE::Mass::Private::UninitializedInt32)
	, EntityListOffsetWithinChunk(UE::Mass::Private::UninitializedInt32)
	, ChunkMemorySize(UE::Mass::SanitizeChunkMemorySize(CreationParams.ChunkMemorySize ? CreationParams.ChunkMemorySize : GetDefault<UMassEntitySettings>()->ChunkMemorySize))
{
#if WITH_MASSENTITY_DEBUG
	DebugNames.Add(CreationParams.DebugName);
	DebugColor = CreationParams.DebugColor;
#endif // WITH_MASSENTITY_DEBUG
}

void FMassArchetypeData::ForEachFragmentType(TFunction< void(const UScriptStruct* /*Fragment*/)> Function) const
{
	for (const FMassArchetypeFragmentConfig& FragmentData : FragmentConfigs)
	{
		Function(FragmentData.FragmentType);
	}
}

void FMassArchetypeData::Initialize(const FMassEntityManager& EntityManager, const FMassElementBitSet& InCompositionBitSet
	, const uint32 ArchetypeDataVersion, UE::Mass::FSparseElementsStorage& InSparseElementsStorage)
{
	if (!ensureMsgf(Chunks.Num() == 0, TEXT("Trying to re-initialize non-empty Mass Archetype is not supported")))
	{
		return;
	}
	if (!ensureMsgf(CreatedArchetypeDataVersion == 0, TEXT("MassArchetype has already been initialized")))
	{
		return;
	}

	CreatedArchetypeDataVersion = ArchetypeDataVersion;
	SparseElementsStorage = &InSparseElementsStorage;

	CompositionBitSet = InCompositionBitSet;

	ConfigureFragments(EntityManager);

	// Create templates for chunk fragments, for quicker chunks creation.
	TArray<const UScriptStruct*, TInlineAllocator<16>> ChunkFragmentList;
	CompositionBitSet.Get<FMassChunkFragmentBitSet>().ExportTypes(ChunkFragmentList);
	ChunkFragmentList.Sort(FScriptStructSortOperator());
	for (const UScriptStruct* ChunkFragmentType : ChunkFragmentList)
	{
		check(ChunkFragmentType);
		ChunkFragmentsTemplate.Emplace(ChunkFragmentType);
	}

	EntityListOffsetWithinChunk = 0;

#if WITH_MASSENTITY_DEBUG
	// For unnamed archetype (DebugNames[0] == NAME_None) we push an extra identifier to allow debug tools to differentiate them
	if (IsUnnamedArchetype())
	{
		DebugNames.Add(FName(FString::Printf(TEXT("0x%X"), GetTypeHash(CompositionBitSet))));
	}
	SetDebugColor(DebugColor);
#endif // WITH_MASSENTITY_DEBUG
}

void FMassArchetypeData::InitializeWithSimilar(const FMassEntityManager& EntityManager, const FMassArchetypeData& BaseArchetype
	, FMassElementBitSet&& InCompositionBitSet, const UE::Mass::FArchetypeGroups& InGroups, const uint32 ArchetypeDataVersion
	, UE::Mass::FSparseElementsStorage& InSparseElementsStorage)
{
	checkf(IsInitialized() == false, TEXT("Trying to InitializeWithSimilar but this archetype has already been initialized"));

	// The composition delta between InCompositionBitSet and BaseArchetype may include tags, fragments,
	// and shared fragments. Shared fragment changes don't affect fragment configs or chunk data layout,
	// so the similar-archetype optimization (copying fragment configs from base) is still valid.

	CreatedArchetypeDataVersion = ArchetypeDataVersion;
	SparseElementsStorage = &InSparseElementsStorage;

	CompositionBitSet = MoveTemp(InCompositionBitSet);
	if (CompositionBitSet.Get<FMassFragmentBitSet>() != BaseArchetype.CompositionBitSet.Get<FMassFragmentBitSet>())
	{
		ConfigureFragments(EntityManager);
	}
	else
	{
		FragmentConfigs = BaseArchetype.FragmentConfigs;
		FragmentIndexMap = BaseArchetype.FragmentIndexMap;
		TotalBytesPerEntity = BaseArchetype.TotalBytesPerEntity;
		NumEntitiesPerChunk = BaseArchetype.NumEntitiesPerChunk;
	}
	ChunkFragmentsTemplate = BaseArchetype.ChunkFragmentsTemplate;

	Groups = InGroups;

	EntityListOffsetWithinChunk = 0;

#if WITH_MASSENTITY_DEBUG
	SetDebugColor(DebugColor);
#endif // WITH_MASSENTITY_DEBUG
}

void FMassArchetypeData::ConfigureFragments(const FMassEntityManager& EntityManager)
{
	TArray<const UScriptStruct*, TInlineAllocator<16>> SortedFragmentList;
	CompositionBitSet.Get<FMassFragmentBitSet>().ExportTypes(SortedFragmentList);

	SortedFragmentList.Sort(FScriptStructSortOperator());

	// Figure out how many bytes all of the individual fragments (and metadata) will cost per entity
	SIZE_T FragmentSizeTallyBytes = 0;

	// Alignment padding computation is currently very conservative and over-estimated.
	SIZE_T AlignmentPadding = 0;
	
	// Save room for the 'metadata' (entity array)
	FragmentSizeTallyBytes += sizeof(FMassEntityHandle);

	// Tally up the fragment sizes and place them in the index map
	FragmentConfigs.AddDefaulted(SortedFragmentList.Num());
	FragmentIndexMap.Reserve(SortedFragmentList.Num());

	for (int32 FragmentIndex = 0; FragmentIndex < SortedFragmentList.Num(); ++FragmentIndex)
	{
		const UScriptStruct* FragmentType = SortedFragmentList[FragmentIndex];
		check(FragmentType);
		FragmentConfigs[FragmentIndex].FragmentType = FragmentType;

		AlignmentPadding += SIZE_T(FragmentType->GetMinAlignment());
		FragmentSizeTallyBytes += SIZE_T(FragmentType->GetStructureSize());

		FragmentIndexMap.Add(FragmentType, FragmentIndex);
	}

	checkf(FragmentSizeTallyBytes < TNumericLimits<uint32>::Max(), TEXT("Single entity's size exceeds 2^32. This is not supported."));
	TotalBytesPerEntity = static_cast<uint32>(FragmentSizeTallyBytes);
	const SIZE_T ChunkAvailableSize = GetChunkAllocSize() - AlignmentPadding;
	checkf(TotalBytesPerEntity <= ChunkAvailableSize, TEXT("Single entity's size is larger than max chunk size - no entities will be created."));

	NumEntitiesPerChunk = static_cast<int32>(ChunkAvailableSize / TotalBytesPerEntity);

	// Set up the offsets for each fragment into the chunk data
	int32 CurrentOffset = NumEntitiesPerChunk * sizeof(FMassEntityHandle);
	for (FMassArchetypeFragmentConfig& FragmentData : FragmentConfigs)
	{
		CurrentOffset = Align(CurrentOffset, FragmentData.FragmentType->GetMinAlignment());
		FragmentData.ArrayOffsetWithinChunk = CurrentOffset;
		const int32 SizeOfThisFragmentArray = NumEntitiesPerChunk * FragmentData.FragmentType->GetStructureSize();
		CurrentOffset += SizeOfThisFragmentArray;
	}
}

void FMassArchetypeData::AddEntity(FMassEntityHandle Entity, const FMassArchetypeSharedFragmentValues& SharedFragmentValues)
{
	const int32 AbsoluteIndex = AddEntityInternal(Entity, SharedFragmentValues);

	// Initialize fragments
	const int32 ChunkIndex = AbsoluteIndex / NumEntitiesPerChunk;
	const int32 IndexWithinChunk = AbsoluteIndex - (NumEntitiesPerChunk * ChunkIndex);

	FMassArchetypeChunk& Chunk = Chunks[ChunkIndex];
	for (const FMassArchetypeFragmentConfig& FragmentConfig : FragmentConfigs)
	{
		void* FragmentPtr = FragmentConfig.GetFragmentData(Chunk.GetRawMemory(), IndexWithinChunk);
		FragmentConfig.FragmentType->InitializeStruct(FragmentPtr);
	}
}

int32 FMassArchetypeData::AddEntityInternal(FMassEntityHandle Entity, const FMassArchetypeSharedFragmentValues& SharedFragmentValues)
{
	checkf(SharedFragmentValues.IsSorted(), TEXT("Expecting shared fragment values to be previously sorted"));
	checkfSlow(SharedFragmentValues.DoesMatchComposition(CompositionBitSet)
		, TEXT("Expecting values for every specified shared fragment in the archetype and only those"));

	++ArchetypeVersion;
	
	int32 IndexWithinChunk = 0;
	int32 AbsoluteIndex = 0;

	FMassArchetypeChunk& DestinationChunk = GetOrAddChunk(SharedFragmentValues, AbsoluteIndex, IndexWithinChunk);
	DestinationChunk.AddInstance();

	// Add to the table and map
	EntityMap.Add(Entity.Index, AbsoluteIndex);
	DestinationChunk.GetEntityArrayElementRef(EntityListOffsetWithinChunk, IndexWithinChunk) = Entity;

	return AbsoluteIndex;
}

FStructView FMassArchetypeData::AddSparseElementToEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType)
{
	checkf(UE::Mass::IsSparse(ElementType), TEXT("%s is not a sparse element"), *ElementType->GetName());

	FMassEntityInChunkDataHandle EntityInChunkHandle = MakeEntityHandle(EntityHandle.Index);
	UE::Mass::FChunkSparseElements& SparseElements = Chunks[EntityInChunkHandle.ChunkIndex].GetSparseElements(NumEntitiesPerChunk);

	bSparseElementsBitSetDirty |= SparseElements.Add(EntityInChunkHandle, ElementType);

	// Sparse elements storage stores only fragments
	if (UE::Mass::IsA<FMassTag>(ElementType))
	{
		return {};
	}

	CA_ASSUME(SparseElementsStorage);
	return SparseElementsStorage->AddElementToEntity(EntityHandle, ElementType);
}

void FMassArchetypeData::RemoveSparseElementFromEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType)
{
	FMassEntityInChunkDataHandle EntityInChunkHandle = MakeEntityHandle(EntityHandle.Index);
	UE::Mass::FChunkSparseElements& SparseElements = Chunks[EntityInChunkHandle.ChunkIndex].GetSparseElementsUnsafe();

	if (SparseElements.IsEmpty() == false)
	{
		if (UE::Mass::IsA<FMassTag>(ElementType) == false)
		{
			CA_ASSUME(SparseElementsStorage);
			SparseElementsStorage->RemoveElementFromEntity(EntityHandle, ElementType);
		}
		bSparseElementsBitSetDirty |= SparseElements.Remove(EntityInChunkHandle, ElementType);
	}
}

void FMassArchetypeData::RemoveSparseElementFromEntity(FMassEntityHandle EntityHandle, const FMassElementBitSet& SparseElementsBitSet)
{
	FMassEntityInChunkDataHandle EntityInChunkHandle = MakeEntityHandle(EntityHandle.Index);
	UE::Mass::FChunkSparseElements& SparseElements = Chunks[EntityInChunkHandle.ChunkIndex].GetSparseElementsUnsafe();

	if (SparseElements.IsEmpty() == false)
	{
		for (FMassElementBitSet::FIndexIterator It = SparseElementsBitSet.GetIndexIterator(); It; ++It)
		{
			const UScriptStruct* ElementType = FMassElementBitSet::GetTypeAtIndex(*It);
			if (UE::Mass::IsA<FMassTag>(ElementType) == false)
			{
				CA_ASSUME(SparseElementsStorage);
				SparseElementsStorage->RemoveElementFromEntity(EntityHandle, ElementType);
			}
			bSparseElementsBitSetDirty |= SparseElements.Remove(EntityInChunkHandle, ElementType);
		}
	}
}

FStructView FMassArchetypeData::GetMutableSparseElementDataForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle) const
{
	check(UE::Mass::IsA<FMassTag>(ElementType) == false);
	CA_ASSUME(SparseElementsStorage);
	return SparseElementsStorage->GetMutableElementDataForEntity(EntityHandle, ElementType);
}

FConstStructView FMassArchetypeData::GetSparseElementDataForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle) const
{
	check(UE::Mass::IsA<FMassTag>(ElementType) == false);
	CA_ASSUME(SparseElementsStorage);
	return SparseElementsStorage->GetElementDataForEntity(EntityHandle, ElementType);
}

FMassElementBitSet FMassArchetypeData::GetSparseElementsBitSetForEntity(const FMassEntityHandle EntityHandle) const
{
	FMassEntityInChunkDataHandle EntityInChunkHandle = MakeEntityHandle(EntityHandle.Index);
	checkSlow(Chunks.IsValidIndex(EntityInChunkHandle.ChunkIndex));
	return Chunks[EntityInChunkHandle.ChunkIndex].HasSparseElements()
		? Chunks[EntityInChunkHandle.ChunkIndex].GetSparseElementsUnsafe().GetElementsForEntity(EntityInChunkHandle)
		: FMassElementBitSet();
}

FMassArchetypeChunk& FMassArchetypeData::GetOrAddChunk(const FMassArchetypeSharedFragmentValues& SharedFragmentValues, int32& OutAbsoluteIndex, int32& OutIndexWithinChunk)
{
	UE_MT_SCOPED_WRITE_ACCESS(ChunkIterationDetector);

	OutAbsoluteIndex = 0;
	OutIndexWithinChunk = 0;

	int32 ChunkIndex = 0;
	int32 EmptyChunkIndex = INDEX_NONE;
	int32 EmptyAbsoluteIndex = INDEX_NONE;

	FMassArchetypeChunk* DestinationChunk = nullptr;
	// Check chunks for a free spot (trying to reuse the earlier ones first so later ones might get freed up) 
	//@TODO: This could be accelerated to include a cached index to the first chunk with free spots or similar
	for (FMassArchetypeChunk& Chunk : Chunks)
	{
		if (Chunk.GetNumInstances() == 0)
		{
			// Remember first empty chunk but continue looking for a chunk that has space and same group tag
			if (EmptyChunkIndex == INDEX_NONE)
			{
				EmptyChunkIndex = ChunkIndex;
				EmptyAbsoluteIndex = OutAbsoluteIndex;
			}
		}
		else if (Chunk.GetNumInstances() < NumEntitiesPerChunk && Chunk.GetSharedFragmentValues().IsEquivalent(SharedFragmentValues))
		{
			OutIndexWithinChunk = Chunk.GetNumInstances();
			OutAbsoluteIndex += OutIndexWithinChunk;

			DestinationChunk = &Chunk;
			break;
		}
		OutAbsoluteIndex += NumEntitiesPerChunk;
		++ChunkIndex;
	}

	if (DestinationChunk == nullptr)
	{
		// Check if it is a recycled chunk
		if (EmptyChunkIndex != INDEX_NONE)
		{
			DestinationChunk = &Chunks[EmptyChunkIndex];
			DestinationChunk->Recycle(ChunkFragmentsTemplate, SharedFragmentValues);
			OutAbsoluteIndex = EmptyAbsoluteIndex;
		}
		else
		{
			DestinationChunk = &Chunks.Emplace_GetRef(GetChunkAllocSize(), ChunkFragmentsTemplate, SharedFragmentValues);
		}
	}

	check(DestinationChunk);
	return *DestinationChunk;
}

void FMassArchetypeData::RemoveEntity(FMassEntityHandle Entity)
{
	const int32 AbsoluteIndex = EntityMap.FindAndRemoveChecked(Entity.Index);

	// Destroy fragments
	const int32 ChunkIndex = AbsoluteIndex / NumEntitiesPerChunk;
	const int32 IndexWithinChunk = AbsoluteIndex - (NumEntitiesPerChunk * ChunkIndex);
	FMassArchetypeChunk& Chunk = Chunks[ChunkIndex];

	for (const FMassArchetypeFragmentConfig& FragmentConfig : FragmentConfigs)
	{
		// Destroy the fragment data
		void* DyingFragmentPtr = FragmentConfig.GetFragmentData(Chunk.GetRawMemory(), IndexWithinChunk);
		FragmentConfig.FragmentType->DestroyStruct(DyingFragmentPtr);
	}

	if (Chunk.HasSparseElements())
	{
		UE::Mass::FChunkSparseElements& SparseElements = Chunk.GetSparseElementsUnsafe();
		const FMassElementBitSet& ElementsBitSet = SparseElements.GetElementsForEntityUnsafe(IndexWithinChunk);
		SparseElementsStorage->RemoveEntity(Entity, ElementsBitSet);
		bSparseElementsBitSetDirty |= SparseElements.UpdateElementsPresenceOnEntityRemoval(IndexWithinChunk);
	}

	RemoveEntityInternal(AbsoluteIndex);
}

void FMassArchetypeData::RemoveEntityInternal(const int32 AbsoluteIndex)
{
	++EntityOrderVersion;
	++ArchetypeVersion;

	const int32 ChunkIndex = AbsoluteIndex / NumEntitiesPerChunk;
	const int32 IndexWithinChunk = AbsoluteIndex - (NumEntitiesPerChunk * ChunkIndex);

	FMassArchetypeChunk& Chunk = Chunks[ChunkIndex];

	const int32 IndexToSwapFrom = Chunk.GetNumInstances() - 1;

	// Remove and swap the last entry in the chunk to the location of the removed item (if it's not the same as the dying entry)
	if (IndexToSwapFrom != IndexWithinChunk)
	{
		for (const FMassArchetypeFragmentConfig& FragmentConfig : FragmentConfigs)
		{
			void* DyingFragmentPtr = FragmentConfig.GetFragmentData(Chunk.GetRawMemory(), IndexWithinChunk);
			void* MovingFragmentPtr = FragmentConfig.GetFragmentData(Chunk.GetRawMemory(), IndexToSwapFrom);

			// Move last entry
			FMemory::Memcpy(DyingFragmentPtr, MovingFragmentPtr, FragmentConfig.FragmentType->GetStructureSize());
		}

		if (Chunk.HasSparseElements())
		{
			UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();
			ChunkSparseElements.GetElementsForEntityUnsafe(IndexWithinChunk) = MoveTemp(ChunkSparseElements.GetElementsForEntityUnsafe(IndexToSwapFrom));
		}

		// Update the entity table and map
		const FMassEntityHandle EntityBeingSwapped = Chunk.GetEntityArrayElementRef(EntityListOffsetWithinChunk, IndexToSwapFrom);
		Chunk.GetEntityArrayElementRef(EntityListOffsetWithinChunk, IndexWithinChunk) = EntityBeingSwapped;
		EntityMap.FindChecked(EntityBeingSwapped.Index) = AbsoluteIndex;
	}
	
	Chunk.RemoveInstance();

	// If the chunk itself is empty now, see if we can remove it entirely
	// Note: This is only possible for trailing chunks, to avoid messing up the absolute indices in the entities map
	while ((Chunks.Num() > 0) && (Chunks.Last().GetNumInstances() == 0))
	{
		Chunks.RemoveAt(Chunks.Num() - 1, EAllowShrinking::No);
	}
}

void FMassArchetypeData::BatchDestroyEntityChunks(FMassArchetypeEntityCollection::FConstEntityRangeArrayView EntityRangeContainer, TArray<FMassEntityHandle>& OutEntitiesRemoved)
{
	const int32 InitialOutEntitiesCount = OutEntitiesRemoved.Num();

	// Sorting the subchunks info so that subchunks of a given chunk are processed "from the back". Otherwise removing 
	// a subchunk from the front of the chunk would inevitably invalidate following subchunks' information.
	FMassArchetypeEntityCollection::FEntityRangeArray SortedRangeCollection(EntityRangeContainer);
	SortedRangeCollection.Sort([](const FMassArchetypeEntityCollection::FArchetypeEntityRange& A, const FMassArchetypeEntityCollection::FArchetypeEntityRange& B) 
		{ 
			return A.ChunkIndex < B.ChunkIndex || (A.ChunkIndex == B.ChunkIndex && A.SubchunkStart > B.SubchunkStart);
		});

	for (const FMassArchetypeEntityCollection::FArchetypeEntityRange EntityRange : SortedRangeCollection)
	{ 
		FMassArchetypeChunk& Chunk = Chunks[EntityRange.ChunkIndex];
		const int32 RangeLength = CalculateRangeLength(EntityRange, Chunk);

		// gather entities we're about to remove
		FMassEntityHandle* DyingEntityPtr = &Chunk.GetEntityArrayElementRef(EntityListOffsetWithinChunk, EntityRange.SubchunkStart);
		OutEntitiesRemoved.Append(DyingEntityPtr, RangeLength);

		if (Chunk.HasSparseElements())
		{
			for (int32 RelativeIndex = 0; RelativeIndex < RangeLength; ++RelativeIndex)
			{
				UE::Mass::FChunkSparseElements& SparseElements = Chunk.GetSparseElementsUnsafe();
				FMassElementBitSet& ElementsBitSet = SparseElements.GetElementsForEntityUnsafe(EntityRange.SubchunkStart + RelativeIndex);
				SparseElementsStorage->RemoveEntity(DyingEntityPtr[RelativeIndex], ElementsBitSet);
				bSparseElementsBitSetDirty |= SparseElements.UpdateElementsPresenceOnEntityRemoval(EntityRange.SubchunkStart + RelativeIndex);
				ElementsBitSet.Reset();
			}
		}

		for (const FMassArchetypeFragmentConfig& FragmentConfig : FragmentConfigs)
		{
			// Destroy the fragment data
			void* DyingFragmentPtr = FragmentConfig.GetFragmentData(Chunk.GetRawMemory(), EntityRange.SubchunkStart);
			FragmentConfig.FragmentType->DestroyStruct(DyingFragmentPtr, RangeLength);
		}

		BatchRemoveEntitiesInternal(EntityRange.ChunkIndex, EntityRange.SubchunkStart, RangeLength);
	}

	for (int i = InitialOutEntitiesCount; i < OutEntitiesRemoved.Num(); ++i)
	{
		EntityMap.FindAndRemoveChecked(OutEntitiesRemoved[i].Index);
	}

	// If the chunk itself is empty now, see if we can remove it entirely
	// Note: This is only possible for trailing chunks, to avoid messing up the absolute indices in the entities map
	while ((Chunks.Num() > 0) && (Chunks.Last().GetNumInstances() == 0))
	{
		Chunks.RemoveAt(Chunks.Num() - 1, EAllowShrinking::No);
	}
}

bool FMassArchetypeData::HasSparseElementForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle) const
{
	const FMassEntityInChunkDataHandle EntityInChunkHandle = MakeEntityHandle(EntityHandle.Index);
	return HasSparseElementForEntity(ElementType, EntityInChunkHandle);
}

bool FMassArchetypeData::HasSparseElementForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityInChunkDataHandle& EntityInChunkHandle) const
{
	checkf(Chunks.IsValidIndex(EntityInChunkHandle.ChunkIndex), TEXT("%hs: Invalid chunk index (%d out of %d)")
		, __FUNCTION__, EntityInChunkHandle.ChunkIndex, Chunks.Num());

	return Chunks[EntityInChunkHandle.ChunkIndex].HasSparseElements()
		&& Chunks[EntityInChunkHandle.ChunkIndex].GetSparseElementsUnsafe().Contains(EntityInChunkHandle, ElementType);
}

void* FMassArchetypeData::GetMutableFragmentDataForEntityChecked(TNotNull<const UScriptStruct*> FragmentType, const FMassEntityHandle EntityHandle)
{
	void* ReturnPtr = GetMutableFragmentDataForEntity(FragmentType, EntityHandle);
	checkf(ReturnPtr, TEXT("Entity %s has neither a regular nor a sparse fragment of type %s"), *EntityHandle.DebugGetDescription(), *FragmentType->GetName());
	return ReturnPtr;
}

const void* FMassArchetypeData::GetFragmentDataForEntityChecked(TNotNull<const UScriptStruct*> FragmentType, const FMassEntityHandle EntityHandle) const
{
	// const delegates to mutable variant via const_cast to avoid code duplication. Safe because
	// the mutable accessor's return value is not written through in the const call path.
	return const_cast<FMassArchetypeData*>(this)->GetMutableFragmentDataForEntityChecked(FragmentType, EntityHandle);
}

void* FMassArchetypeData::GetMutableFragmentDataForEntity(TNotNull<const UScriptStruct*> FragmentType, const FMassEntityHandle EntityHandle)
{
	checkf(UE::Mass::IsA<FMassFragment>(FragmentType), TEXT("%s is not a fragment type"), *FragmentType->GetName());

	if (const int32* FragmentIndex = FragmentIndexMap.Find(FragmentType))
	{
		FMassRawEntityInChunkData InternalIndex = MakeEntityHandle(EntityHandle.Index);
		// failing the below Find means given entity's archetype is missing given FragmentType
		return GetFragmentData(*FragmentIndex, InternalIndex);
	}

	// Not in the regular fragment map - look it up as a sparse fragment.
	// Note that it's fine to return the pointer from the resulting view, since worst-case-scenario
	// it returns a default-constructed FStructView whose GetMemory() returns nullptr.
	return UE::Mass::IsSparse(FragmentType) ? GetMutableSparseElementDataForEntity(FragmentType, EntityHandle).GetMemory() : nullptr;
}

const void* FMassArchetypeData::GetFragmentDataForEntity(TNotNull<const UScriptStruct*> FragmentType, const FMassEntityHandle EntityHandle) const
{
	// const delegates to mutable variant via const_cast to avoid code duplication. Safe because
	// the mutable accessor's return value is not written through in the const call path.
	return const_cast<FMassArchetypeData*>(this)->GetMutableFragmentDataForEntity(FragmentType, EntityHandle);
}

bool FMassArchetypeData::DoesEntityMatchSparseComposition(const FMassEntityInChunkDataHandle& EntityInChunkHandle
	, const FMassElementBitSet& RequiredAllSparseElements, const FMassElementBitSet& RequiredAnySparseElements, const FMassElementBitSet& RequiredNoneSparseElements) const
{
	checkf(Chunks.IsValidIndex(EntityInChunkHandle.ChunkIndex), TEXT("%hs: Invalid chunk index (%d out of %d)")
		, __FUNCTION__, EntityInChunkHandle.ChunkIndex, Chunks.Num());

	if (Chunks[EntityInChunkHandle.ChunkIndex].HasSparseElements())
	{
		const FMassElementBitSet& ElementsCollection = Chunks[EntityInChunkHandle.ChunkIndex].GetSparseElementsUnsafe().GetElementsForEntity(EntityInChunkHandle);
		return (RequiredAllSparseElements.IsEmpty() || ElementsCollection.HasAll(RequiredAllSparseElements))
			&& (RequiredAnySparseElements.IsEmpty() || ElementsCollection.HasAny(RequiredAnySparseElements))
			&& ElementsCollection.HasNone(RequiredNoneSparseElements);
	}

	// no sparse elements, so the entity "matches" only if the requirements are empty
	return RequiredAllSparseElements.IsEmpty() && RequiredAnySparseElements.IsEmpty();
}

void FMassArchetypeData::SetSharedFragmentsData(const FMassEntityHandle Entity, const FMassEntityInChunkDataHandle EntityInChunkHandle, FMassArchetypeSharedFragmentValues&& SharedFragmentValues)
{
	int32 NewAbsoluteIndex = 0;
	int32 NewIndexWithinChunk = 0;
	FMassArchetypeChunk& NewChunk = GetOrAddChunk(SharedFragmentValues, NewAbsoluteIndex, NewIndexWithinChunk);
	FMassArchetypeChunk& OldChunk = Chunks[EntityInChunkHandle.ChunkIndex];

	if (ensureMsgf(&NewChunk != &OldChunk, TEXT("Found target chunk is the same as the source chunk. Probably "
		"caused by setting shared fragment values resulted in no change, meaning the target values equal the source values")))
	{
		NewChunk.AddInstance();

		// Update the new entity in the table and map
		EntityMap[Entity.Index] = NewAbsoluteIndex;
		NewChunk.GetEntityArrayElementRef(EntityListOffsetWithinChunk, NewIndexWithinChunk) = Entity;
		
		// Move the current entity fragments into the new chunk
		MoveFragmentsToNewLocationInternal({ NewChunk.GetRawMemory(), NewIndexWithinChunk }, { EntityInChunkHandle.ChunkRawMemory, EntityInChunkHandle.IndexWithinChunk }, 1);

		if (OldChunk.HasSparseElements())
		{
			UE::Mass::FChunkSparseElements& OldSparseElements = OldChunk.GetSparseElementsUnsafe();
			UE::Mass::FChunkSparseElements& NewSparseElements = NewChunk.GetSparseElements(NumEntitiesPerChunk);
			// note that we don't have to update bSparseElementsBitSetDirty since we're moving elements
			// between chunks of the same archetype so the archetype's cumulative CachedSparseElementsBitSet 
			// won't be affected
			NewSparseElements.MoveElementsFrom(OldSparseElements, EntityInChunkHandle.IndexWithinChunk, NewIndexWithinChunk, 1);
		}

		// Clean up the old chunk
		RemoveEntityInternal(EntityInChunkHandle.GetAbsoluteIndex(NumEntitiesPerChunk));
	}
}

void FMassArchetypeData::MoveEntityToAnotherArchetype(const FMassEntityHandle Entity, FMassArchetypeData& NewArchetype, const FMassArchetypeSharedFragmentValues* SharedFragmentValuesOverride)
{
	check(&NewArchetype != this);

	const int32 AbsoluteIndex = EntityMap.FindAndRemoveChecked(Entity.Index);
	const int32 ChunkIndex = AbsoluteIndex / NumEntitiesPerChunk;
	const int32 IndexWithinChunk = AbsoluteIndex - (NumEntitiesPerChunk * ChunkIndex);
	FMassArchetypeChunk& Chunk = Chunks[ChunkIndex];

	const int32 NewAbsoluteIndex = NewArchetype.AddEntityInternal(Entity, SharedFragmentValuesOverride ? *SharedFragmentValuesOverride : Chunk.GetSharedFragmentValues());
	const int32 NewChunkIndex = NewAbsoluteIndex / NewArchetype.NumEntitiesPerChunk;
	const int32 NewIndexWithinChunk = NewAbsoluteIndex - (NewArchetype.NumEntitiesPerChunk * NewChunkIndex);
	FMassArchetypeChunk& NewChunk = NewArchetype.Chunks[NewChunkIndex];

	UE_TRACE_MASS_ARCHETYPE_CREATED(NewArchetype)
	UE_TRACE_MASS_ENTITY_MOVED(Entity, NewArchetype)

	MoveFragmentsToAnotherArchetypeInternal(NewArchetype, NewChunk, NewIndexWithinChunk, Chunk, IndexWithinChunk, /*Count=*/1);

	RemoveEntityInternal(AbsoluteIndex);
}

void FMassArchetypeData::ExecuteFunction(FMassExecutionContext& RunContext, const FMassExecuteFunction& Function, const FMassQueryRequirementIndicesMapping& RequirementMapping, FMassArchetypeEntityCollection::FConstEntityRangeArrayView EntityRangeContainer, const FMassChunkConditionFunction& ChunkCondition)
{
	if (GetNumEntities() == 0)
	{
		return;
	}

	// @todo do we really want users to check composition of the archetype being processed at the moment?
	RunContext.SetCurrentArchetype(*this);
#if WITH_MASSENTITY_DEBUG
	RunContext.DebugSetColor(DebugColor);
#endif // WITH_MASSENTITY_DEBUG

	uint32 PrevSharedFragmentValuesHash = TNumericLimits<uint32>::Max();
	for (FMassArchetypeChunkIterator ChunkIterator(EntityRangeContainer); ChunkIterator; ++ChunkIterator)
	{
		FMassArchetypeChunk& Chunk = Chunks[ChunkIterator->ChunkIndex];

		const int32 SubchunkLength = ChunkIterator->Length > 0 ? ChunkIterator->Length : (Chunk.GetNumInstances() - ChunkIterator->SubchunkStart);
		if (SubchunkLength)
		{
			const uint32 SharedFragmentValuesHash = GetTypeHash(Chunk.GetSharedFragmentValues());
			if (PrevSharedFragmentValuesHash != SharedFragmentValuesHash)
			{
				PrevSharedFragmentValuesHash = SharedFragmentValuesHash;
				BindConstSharedFragmentRequirements(RunContext, Chunk.GetSharedFragmentValues(), RequirementMapping.ConstSharedFragments);
				BindSharedFragmentRequirements(RunContext, Chunk.GetMutableSharedFragmentValues(), RequirementMapping.SharedFragments);
			}

			checkf((ChunkIterator->SubchunkStart + SubchunkLength) <= Chunk.GetNumInstances() && SubchunkLength > 0, TEXT("Invalid subchunk, it is going over the number of instances in the chunk or it is empty."));

			BindChunkFragmentRequirements(RunContext, RequirementMapping.ChunkFragments, Chunk);
			RunContext.BindLinkedEntity();

			if (RunContext.SetCurrentChunk(ChunkIterator->ChunkIndex, Chunk) 
				&& (!ChunkCondition || ChunkCondition(RunContext)))
			{
				BindEntityRequirements(RunContext, RequirementMapping.EntityFragments, Chunk, ChunkIterator->SubchunkStart, SubchunkLength);
				Function(RunContext);
			}
		}
	}
}

void FMassArchetypeData::ExecuteFunction(FMassExecutionContext& RunContext, const FMassExecuteFunction& Function, const FMassQueryRequirementIndicesMapping& RequirementMapping, const FMassChunkConditionFunction& ChunkCondition, UE::Mass::FExecutionLimiter* ExecutionLimiter)
{
	if (GetNumEntities() == 0)
	{
		return;
	}

	RunContext.SetCurrentArchetype(*this);
#if WITH_MASSENTITY_DEBUG
	RunContext.DebugSetColor(DebugColor);
#endif // WITH_MASSENTITY_DEBUG

	int32 ChunkIndex = 0;
	int32 EntityCountRemaining = TNumericLimits<int32>::Max();
	int32 MaxChunkIndex = Chunks.Num();

	if (ExecutionLimiter)
	{
		ChunkIndex = FMath::Max(ExecutionLimiter->ChunkIndex, 0);
		EntityCountRemaining = ExecutionLimiter->EntityCountRemaining;
		MaxChunkIndex = FMath::Min(MaxChunkIndex, ExecutionLimiter->MaxChunkIndex);
	}

	uint32 PrevSharedFragmentValuesHash = TNumericLimits<uint32>::Max();
	for (; ChunkIndex < MaxChunkIndex && EntityCountRemaining > 0; ChunkIndex++)
	{
		FMassArchetypeChunk& Chunk = Chunks[ChunkIndex];
		const int32 EntityCount = Chunk.GetNumInstances();

		if (EntityCount)
		{
			const uint32 SharedFragmentValuesHash = GetTypeHash(Chunk.GetSharedFragmentValues());
			if (PrevSharedFragmentValuesHash != SharedFragmentValuesHash)
			{
				PrevSharedFragmentValuesHash = SharedFragmentValuesHash;
				BindConstSharedFragmentRequirements(RunContext, Chunk.GetSharedFragmentValues(), RequirementMapping.ConstSharedFragments);
				BindSharedFragmentRequirements(RunContext, Chunk.GetMutableSharedFragmentValues(), RequirementMapping.SharedFragments);
			}

			BindChunkFragmentRequirements(RunContext, RequirementMapping.ChunkFragments, Chunk);
			RunContext.BindLinkedEntity();

			if (RunContext.SetCurrentChunk(ChunkIndex, Chunk)
				&& (!ChunkCondition || ChunkCondition(RunContext)))
			{
				BindEntityRequirements(RunContext, RequirementMapping.EntityFragments, Chunk, 0, Chunk.GetNumInstances());
				Function(RunContext);

				EntityCountRemaining -= EntityCount;
			}
		}
	}

	if (ExecutionLimiter)
	{
		// set the limiter to continue on the next chunk if this archetype has unprocessed chunks
		ExecutionLimiter->ChunkIndex = ++ChunkIndex;
		ExecutionLimiter->EntityCountRemaining = EntityCountRemaining;
	}
}

void FMassArchetypeData::ExecutionFunctionForChunk(FMassExecutionContext& RunContext, const FMassExecuteFunction& Function, const FMassQueryRequirementIndicesMapping& RequirementMapping, const FMassArchetypeEntityCollection::FArchetypeEntityRange& EntityRange, const FMassChunkConditionFunction& ChunkCondition)
{
	FMassArchetypeChunk& Chunk = Chunks[EntityRange.ChunkIndex];
	const int32 RangeLength = CalculateRangeLength(EntityRange, Chunk);

	if (RangeLength > 0)
	{
		BindConstSharedFragmentRequirements(RunContext, Chunk.GetSharedFragmentValues(), RequirementMapping.ConstSharedFragments);
		BindSharedFragmentRequirements(RunContext, Chunk.GetMutableSharedFragmentValues(), RequirementMapping.SharedFragments);

		RunContext.SetCurrentArchetype(*this);
#if WITH_MASSENTITY_DEBUG
		RunContext.DebugSetColor(DebugColor);
#endif // WITH_MASSENTITY_DEBUG

		BindChunkFragmentRequirements(RunContext, RequirementMapping.ChunkFragments, Chunk);
		RunContext.BindLinkedEntity();

		if (RunContext.SetCurrentChunk(EntityRange.ChunkIndex, Chunk)
			&& (!ChunkCondition || ChunkCondition(RunContext)))
		{
			BindEntityRequirements(RunContext, RequirementMapping.EntityFragments, Chunk, EntityRange.SubchunkStart, RangeLength);
			Function(RunContext);
		}
	}
}

int32 FMassArchetypeData::CompactEntities(const double TimeAllowed)
{
	int32 TotalEntitiesMoved = 0;
	const double TimeAllowedEnd = FPlatformTime::Seconds() + TimeAllowed;

	TMap<uint32, TArray<FMassArchetypeChunk*>> SortedChunksBySharedValues;
	for (FMassArchetypeChunk& Chunk : Chunks)
	{
		// Skip already full chunks
		const int32 NumInstances = Chunk.GetNumInstances();
		if (NumInstances > 0 && NumInstances < NumEntitiesPerChunk)
		{
			const uint32 SharedFragmentHash = GetTypeHash(Chunk.GetSharedFragmentValues());
			TArray<FMassArchetypeChunk*>& SortedChunks = SortedChunksBySharedValues.FindOrAddByHash(SharedFragmentHash, SharedFragmentHash, TArray<FMassArchetypeChunk*>());
			SortedChunks.Add(&Chunk);
		}
	}

	for (TPair<uint32, TArray<FMassArchetypeChunk*>>& Pair : SortedChunksBySharedValues)
	{
		TArray<FMassArchetypeChunk*>& SortedChunks = Pair.Value;

		// Check if there is anything to compact at all
		if (SortedChunks.Num() <= 1)
		{
			continue;
		}

		SortedChunks.Sort([](const FMassArchetypeChunk& LHS, const FMassArchetypeChunk& RHS)
		{
			return LHS.GetNumInstances() < RHS.GetNumInstances();
		});

		int32 ChunkToFillSortedIdx = 0;
		int32 ChunkToEmptySortedIdx = SortedChunks.Num() - 1;
		while (ChunkToFillSortedIdx < ChunkToEmptySortedIdx && FPlatformTime::Seconds() < TimeAllowedEnd)
		{
			while (ChunkToFillSortedIdx < SortedChunks.Num() && SortedChunks[ChunkToFillSortedIdx]->GetNumInstances() == NumEntitiesPerChunk)
			{
				ChunkToFillSortedIdx++;
			}
			while (ChunkToEmptySortedIdx >= 0 && SortedChunks[ChunkToEmptySortedIdx]->GetNumInstances() == 0)
			{
				ChunkToEmptySortedIdx--;
			}
			if (ChunkToFillSortedIdx >= ChunkToEmptySortedIdx)
			{
				break;
			}

			FMassArchetypeChunk* ChunkToFill = SortedChunks[ChunkToFillSortedIdx];
			FMassArchetypeChunk* ChunkToEmpty = SortedChunks[ChunkToEmptySortedIdx];
			const int32 NumberOfEntitiesToMove = FMath::Min(NumEntitiesPerChunk - ChunkToFill->GetNumInstances(), ChunkToEmpty->GetNumInstances());
			const int32 FromIndex = ChunkToEmpty->GetNumInstances() - NumberOfEntitiesToMove;
			const int32 ToIndex = ChunkToFill->GetNumInstances();
			check(NumberOfEntitiesToMove > 0);

			MoveFragmentsToNewLocationInternal({ChunkToFill->GetRawMemory(), ToIndex}, {ChunkToEmpty->GetRawMemory(), FromIndex}
				, NumberOfEntitiesToMove);

			if (ChunkToEmpty->HasSparseElements())
			{
				UE::Mass::FChunkSparseElements& OldSparseElements = ChunkToEmpty->GetSparseElementsUnsafe();
				UE::Mass::FChunkSparseElements& NewSparseElements = ChunkToFill->GetSparseElements(NumEntitiesPerChunk);
				const bool bElementsPresentDirty = NewSparseElements.MoveElementsFrom(OldSparseElements, FromIndex, ToIndex, NumberOfEntitiesToMove);
				bSparseElementsBitSetDirty |= bElementsPresentDirty;
			}

			FMassEntityHandle* FromEntity = &ChunkToEmpty->GetEntityArrayElementRef(EntityListOffsetWithinChunk, FromIndex);
			FMassEntityHandle* ToEntity = &ChunkToFill->GetEntityArrayElementRef(EntityListOffsetWithinChunk, ToIndex);
			FMemory::Memcpy(ToEntity, FromEntity, NumberOfEntitiesToMove * sizeof(FMassEntityHandle));
			ChunkToFill->AddMultipleInstances(NumberOfEntitiesToMove);
			ChunkToEmpty->RemoveMultipleInstances(NumberOfEntitiesToMove);

			const int32 ChunkToFillIdx = UE_PTRDIFF_TO_INT32(ChunkToFill - &Chunks[0]);
			check(ChunkToFillIdx >= 0 && ChunkToFillIdx < Chunks.Num());
			const int32 AbsoluteIndex = ChunkToFillIdx * NumEntitiesPerChunk + ToIndex;

			for (int32 i = 0; i < NumberOfEntitiesToMove; i++, ++ToEntity)
			{
				EntityMap.FindChecked(ToEntity->Index) = AbsoluteIndex + i;
			}

			TotalEntitiesMoved += NumberOfEntitiesToMove;
		}
	}

	if (TotalEntitiesMoved > 0)
	{
		++EntityOrderVersion;
		++ArchetypeVersion;
	}

	return TotalEntitiesMoved;
}

void FMassArchetypeData::GetRequirementsFragmentMapping(TConstArrayView<FMassFragmentRequirementDescription> Requirements, FMassFragmentIndicesMapping& OutFragmentIndices) const
{
	OutFragmentIndices.Reset(Requirements.Num());
	for (const FMassFragmentRequirementDescription& Requirement : Requirements)
	{
		if (Requirement.RequiresBinding())
		{
			const int32* FragmentIndex = FragmentIndexMap.Find(Requirement.StructType);
			check(FragmentIndex != nullptr || Requirement.IsOptional());
			OutFragmentIndices.Add(FragmentIndex ? *FragmentIndex : INDEX_NONE);
		}
	}
}

// @todo make ChunkRequirements a dedicated type, so that we can ensure that the contents are sorted as expected by the for loop below
void FMassArchetypeData::GetRequirementsChunkFragmentMapping(TConstArrayView<FMassFragmentRequirementDescription> ChunkRequirements, FMassFragmentIndicesMapping& OutFragmentIndices) const
{
	int32 LastFoundFragmentIndex = -1;
	OutFragmentIndices.Reset(ChunkRequirements.Num());
	for (const FMassFragmentRequirementDescription& Requirement : ChunkRequirements)
	{
		if (Requirement.RequiresBinding())
		{
			int32 FragmentIndex = INDEX_NONE;
			for (int32 i = LastFoundFragmentIndex + 1; i < ChunkFragmentsTemplate.Num(); ++i)
			{
				if (ChunkFragmentsTemplate[i].GetScriptStruct()->IsChildOf(Requirement.StructType))
				{
					FragmentIndex = i;
					break;
				}
			}

			check(FragmentIndex != INDEX_NONE || Requirement.IsOptional());
			OutFragmentIndices.Add(FragmentIndex);
			LastFoundFragmentIndex = FragmentIndex;
		}
	}
}

void FMassArchetypeData::GetRequirementsConstSharedFragmentMapping(TConstArrayView<FMassFragmentRequirementDescription> Requirements, FMassFragmentIndicesMapping& OutFragmentIndices) const
{
	if (Chunks.Num() == 0)
	{
		return;
	}
	// All shared fragment values for this archetype should have deterministic indices, so anyone will work to calculate them
	const FMassArchetypeSharedFragmentValues& SharedFragmentValues = Chunks[0].GetSharedFragmentValues();

	OutFragmentIndices.Reset(Requirements.Num());
	for (const FMassFragmentRequirementDescription& Requirement : Requirements)
	{
		if (Requirement.RequiresBinding())
		{
			const int32 FragmentIndex = SharedFragmentValues.GetConstSharedFragments().IndexOfByPredicate(FStructTypeEqualOperator(Requirement.StructType));
			check(FragmentIndex != INDEX_NONE || Requirement.IsOptional());
			OutFragmentIndices.Add(FragmentIndex);
		}
	}
}

void FMassArchetypeData::GetRequirementsSharedFragmentMapping(TConstArrayView<FMassFragmentRequirementDescription> Requirements, FMassFragmentIndicesMapping& OutFragmentIndices) const
{
	if (Chunks.Num() == 0)
	{
		return;
	}

	// All shared fragment values for this archetype should have deterministic indices, so anyone will work to calculate them
	const FMassArchetypeSharedFragmentValues& SharedFragmentValues = Chunks[0].GetSharedFragmentValues();

	OutFragmentIndices.Reset(Requirements.Num());
	for (const FMassFragmentRequirementDescription& Requirement : Requirements)
	{
		if (Requirement.RequiresBinding())
		{
			const int32 FragmentIndex = SharedFragmentValues.GetSharedFragments().IndexOfByPredicate(FStructTypeEqualOperator(Requirement.StructType));
			check(FragmentIndex != INDEX_NONE || Requirement.IsOptional());
			OutFragmentIndices.Add(FragmentIndex);
		}
	}
}

void FMassArchetypeData::BindEntityRequirements(FMassExecutionContext& RunContext, const FMassFragmentIndicesMapping& EntityFragmentsMapping, FMassArchetypeChunk& Chunk, const int32 SubchunkStart, const int32 SubchunkLength)
{
	// auto-correcting number of entities to process in case SubchunkStart +  SubchunkLength > Chunk.GetNumInstances()
	const int32 NumEntities = SubchunkLength >= 0 ? FMath::Min(SubchunkLength, Chunk.GetNumInstances() - SubchunkStart) : Chunk.GetNumInstances();
	check(SubchunkStart >= 0 && SubchunkStart < Chunk.GetNumInstances());

	if (EntityFragmentsMapping.Num() > 0)
	{
		check(RunContext.GetMutableRequirements().Num() == EntityFragmentsMapping.Num());

		for (int i = 0; i < EntityFragmentsMapping.Num(); ++i)
		{
			FMassExecutionContext::FFragmentView& Requirement = RunContext.FragmentViews[i];
			const int32 FragmentIndex = EntityFragmentsMapping[i];

			check(FragmentIndex != INDEX_NONE || Requirement.Requirement.IsOptional());
			if (FragmentIndex != INDEX_NONE)
			{
				Requirement.FragmentView = TArrayView<FMassFragment>((FMassFragment*)GetFragmentData(FragmentIndex, Chunk.GetRawMemory(), SubchunkStart), NumEntities);
			}
			else
			{
				// @todo this might not be needed
				Requirement.FragmentView = TArrayView<FMassFragment>();
			}
		}
	}
	else
	{
		// Map in the required data arrays from the current chunk to the array views
		for (FMassExecutionContext::FFragmentView& Requirement : RunContext.GetMutableRequirements())
		{
			const int32* FragmentIndex = FragmentIndexMap.Find(Requirement.Requirement.StructType);
			check(FragmentIndex != nullptr || Requirement.Requirement.IsOptional());
			if (FragmentIndex)
			{
				Requirement.FragmentView = TArrayView<FMassFragment>((FMassFragment*)GetFragmentData(*FragmentIndex, Chunk.GetRawMemory(), SubchunkStart), NumEntities);
			}
			else
			{
				Requirement.FragmentView = TArrayView<FMassFragment>();
			}
		}
	}

	RunContext.EntityListView = TArrayView<FMassEntityHandle>(&Chunk.GetEntityArrayElementRef(EntityListOffsetWithinChunk, SubchunkStart), NumEntities);
}

void FMassArchetypeData::BindChunkFragmentRequirements(FMassExecutionContext& RunContext, const FMassFragmentIndicesMapping& ChunkFragmentsMapping, FMassArchetypeChunk& Chunk)
{
	if (ChunkFragmentsMapping.Num() > 0)
	{
		check(RunContext.GetMutableChunkRequirements().Num() == ChunkFragmentsMapping.Num());

		for (int i = 0; i < ChunkFragmentsMapping.Num(); ++i)
		{
			FMassExecutionContext::FChunkFragmentView& ChunkRequirement = RunContext.ChunkFragmentViews[i];
			const int32 ChunkFragmentIndex = ChunkFragmentsMapping[i];

			check(ChunkFragmentIndex != INDEX_NONE || ChunkRequirement.Requirement.IsOptional());
			ChunkRequirement.FragmentView = ChunkFragmentIndex != INDEX_NONE ? Chunk.GetMutableChunkFragmentViewChecked(ChunkFragmentIndex) : FStructView();
		}
	}
	else
	{
		for (FMassExecutionContext::FChunkFragmentView& ChunkRequirement : RunContext.GetMutableChunkRequirements())
		{
			FInstancedStruct* ChunkFragmentInstance = Chunk.FindMutableChunkFragment(ChunkRequirement.Requirement.StructType);
			check(ChunkFragmentInstance != nullptr || ChunkRequirement.Requirement.IsOptional());
			ChunkRequirement.FragmentView = ChunkFragmentInstance ? FStructView(*ChunkFragmentInstance) : FStructView();
		}
	}
}

void FMassArchetypeData::BindConstSharedFragmentRequirements(FMassExecutionContext& RunContext, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, const FMassFragmentIndicesMapping& FragmentsMapping)
{
	if (FragmentsMapping.Num() > 0)
	{
		check(RunContext.GetMutableConstSharedRequirements().Num() == FragmentsMapping.Num());

		for (int i = 0; i < FragmentsMapping.Num(); ++i)
		{
			FMassExecutionContext::FConstSharedFragmentView& Requirement = RunContext.ConstSharedFragmentViews[i];
			const int32 FragmentIndex = FragmentsMapping[i];

			check(FragmentIndex != INDEX_NONE || Requirement.Requirement.IsOptional());
			Requirement.FragmentView = FragmentIndex != INDEX_NONE ? FConstStructView(SharedFragmentValues.GetConstSharedFragments()[FragmentIndex]) : FConstStructView();
		}
	}
	else
	{
		for (FMassExecutionContext::FConstSharedFragmentView& Requirement : RunContext.GetMutableConstSharedRequirements())
		{
			const FConstSharedStruct* SharedFragment = SharedFragmentValues.GetConstSharedFragments().FindByPredicate(FStructTypeEqualOperator(Requirement.Requirement.StructType) );
			check(SharedFragment != nullptr || Requirement.Requirement.IsOptional());
			Requirement.FragmentView = SharedFragment ? FConstStructView(*SharedFragment) : FConstStructView();
		}
	}
}

void FMassArchetypeData::BindSharedFragmentRequirements(FMassExecutionContext& RunContext, FMassArchetypeSharedFragmentValues& SharedFragmentValues, const FMassFragmentIndicesMapping& FragmentsMapping)
{
	if (FragmentsMapping.Num() > 0)
	{
		check(RunContext.GetMutableSharedRequirements().Num() == FragmentsMapping.Num());

		for (int i = 0; i < FragmentsMapping.Num(); ++i)
		{
			FMassExecutionContext::FSharedFragmentView& Requirement = RunContext.SharedFragmentViews[i];
			const int32 FragmentIndex = FragmentsMapping[i];

			check(FragmentIndex != INDEX_NONE || Requirement.Requirement.IsOptional());
			Requirement.FragmentView = FragmentIndex != INDEX_NONE ? FStructView(SharedFragmentValues.GetMutableSharedFragments()[FragmentIndex]) : FStructView();
		}
	}
	else
	{
		for (FMassExecutionContext::FSharedFragmentView& Requirement : RunContext.GetMutableSharedRequirements())
		{
			FSharedStruct* SharedFragment = SharedFragmentValues.GetMutableSharedFragments().FindByPredicate(FStructTypeEqualOperator(Requirement.Requirement.StructType));
			check(SharedFragment != nullptr || Requirement.Requirement.IsOptional());
			Requirement.FragmentView = SharedFragment ? FStructView(*SharedFragment) : FStructView();
		}
	}
}

int32 FMassArchetypeData::GetNonEmptyChunkCount() const
{
	int32 NumAllocatedChunks = 0;
	for (const FMassArchetypeChunk& Chunk : Chunks)
	{
		if (Chunk.GetRawMemory() != nullptr)
		{
			++NumAllocatedChunks;
		}
	}
	return NumAllocatedChunks;
}

SIZE_T FMassArchetypeData::GetAllocatedSize() const
{
	const int32 NumAllocatedChunkBuffers = GetNonEmptyChunkCount();

	SIZE_T PerChunkOwnedSize = 0;
	for (const FMassArchetypeChunk& Chunk : Chunks)
	{
		PerChunkOwnedSize += Chunk.GetAllocatedSize();
	}

	return sizeof(FMassArchetypeData) +
		ChunkFragmentsTemplate.GetAllocatedSize() +
		FragmentConfigs.GetAllocatedSize() +
		Chunks.GetAllocatedSize() +
		(NumAllocatedChunkBuffers * GetChunkAllocSize()) +
		PerChunkOwnedSize +
		EntityMap.GetAllocatedSize() +
		FragmentIndexMap.GetAllocatedSize();
}

SIZE_T FMassArchetypeData::GetSparseElementsBookkeepingAllocatedSize() const
{
	SIZE_T TotalSize = 0;
	for (const FMassArchetypeChunk& Chunk : Chunks)
	{
		TotalSize += Chunk.GetSparseElementsUnsafe().GetAllocatedSize();
	}
	return TotalSize;
}

void FMassArchetypeData::ExportEntityHandles(const TConstArrayView<FMassArchetypeEntityCollection::FArchetypeEntityRange> Ranges, TArray<FMassEntityHandle>& InOutHandles) const
{
	int32 TotalEntities = 0;
	for (const FMassArchetypeEntityCollection::FArchetypeEntityRange& Range : Ranges)
	{
		check(Chunks.IsValidIndex(Range.ChunkIndex));
		TotalEntities += (Range.Length > 0) ? Range.Length : (Chunks[Range.ChunkIndex].GetNumInstances() - Range.SubchunkStart);
	}

	int32 StartIndex = InOutHandles.Num();
	InOutHandles.AddUninitialized(TotalEntities);

	for (const FMassArchetypeEntityCollection::FArchetypeEntityRange& Range : Ranges)
	{
		const FMassArchetypeChunk& Chunk = Chunks[Range.ChunkIndex];
		const FMassEntityHandle* EntitiesArray = Chunk.GetEntityArray(EntityListOffsetWithinChunk);
		const int32 RangeLength = CalculateRangeLength(Range, Chunk);
		FMemory::Memcpy(&InOutHandles[StartIndex], &EntitiesArray[Range.SubchunkStart], RangeLength * sizeof(FMassEntityHandle));

		StartIndex += RangeLength;
	}
}

void FMassArchetypeData::ExportEntityHandles(TArray<FMassEntityHandle>& InOutHandles) const
{
	for (const FMassArchetypeChunk& Chunk : Chunks)
	{
		InOutHandles.Append(Chunk.GetEntityArray(EntityListOffsetWithinChunk), Chunk.GetNumInstances());
	}
}

void FMassArchetypeData::ListEntitiesForEachChunk(TFunctionRef<void(TConstArrayView<FMassEntityHandle>)> Function) const
{
	for (const FMassArchetypeChunk& Chunk : Chunks)
	{
		if (int32 ChuckInstanceCount = Chunk.GetNumInstances(); ChuckInstanceCount > 0)
		{
			Function(TConstArrayView<FMassEntityHandle>(Chunk.GetEntityArray(EntityListOffsetWithinChunk), ChuckInstanceCount));
		}
	}
}

FString FMassArchetypeData::DebugGetDescription() const
{
#if WITH_MASSENTITY_DEBUG
	FStringOutputDevice OutDescription;

	if (!DebugNames.IsEmpty())
	{
		OutDescription += TEXT("Name: ");
		OutDescription += GetCombinedDebugNamesAsString();
		OutDescription += TEXT("\n");
	}
	OutDescription += TEXT("\nElements: ");
	CompositionBitSet.DebugGetStringDesc(OutDescription);
	
	return static_cast<FString>(OutDescription);
#else
	return {};
#endif
}

#if WITH_MASSENTITY_DEBUG
void FMassArchetypeData::DebugGetEntityMemoryNumbers(SIZE_T& OutActiveChunksMemorySize, SIZE_T& OutActiveEntitiesMemorySize) const
{
	OutActiveChunksMemorySize = GetChunkAllocSize() * GetNonEmptyChunkCount();
	OutActiveEntitiesMemorySize = TotalBytesPerEntity * EntityMap.Num();
}

FString FMassArchetypeData::GetCombinedDebugNamesAsString() const
{
	TStringBuilder<256> StringBuilder;
	for (int i = 0; i < DebugNames.Num(); i++)
	{
		if (i > 0)
		{
			StringBuilder.Append(TEXT(", "));
		}
		StringBuilder.Append(DebugNames[i].ToString());
	}
	return StringBuilder.ToString();
}

void FMassArchetypeData::DebugPrintArchetype(FOutputDevice& Ar)
{
	Ar.Logf(ELogVerbosity::Log, TEXT("Name: %s"), *GetCombinedDebugNamesAsString());

	FStringOutputDevice ElementsDescription;
	CompositionBitSet.DebugGetStringDesc(ElementsDescription);
	Ar.Logf(ELogVerbosity::Log, TEXT("Elements: %s"), *ElementsDescription);
	Ar.Logf(ELogVerbosity::Log, TEXT("\tChunks: %d x %llu KB = %llu KB total"), Chunks.Num(), GetChunkAllocSize() / 1024, (GetChunkAllocSize()*Chunks.Num()) / 1024);
	
	int ChunkWithFragmentsCount = 0;
	for (FMassArchetypeChunk& Chunk : Chunks)
	{
		ChunkWithFragmentsCount += Chunk.DebugGetChunkFragmentCount() > 0 ? 1 : 0;
	}
	if (ChunkWithFragmentsCount)
	{
		Ar.Logf(ELogVerbosity::Log, TEXT("\tChunks with fragments: %d"), ChunkWithFragmentsCount);
	}

	const int32 CurrentEntityCapacity = Chunks.Num() * NumEntitiesPerChunk;
	Ar.Logf(ELogVerbosity::Log, TEXT("\tEntity Count    : %d"), EntityMap.Num());
	Ar.Logf(ELogVerbosity::Log, TEXT("\tEntity Capacity : %d"), CurrentEntityCapacity);
	if (Chunks.Num() > 1)
	{
		const float Scaler = 100.0f / static_cast<float>(CurrentEntityCapacity);
		// count non-last chunks to see how occupied they are
		int EntitiesPerChunkMin = CurrentEntityCapacity;
		int EntitiesPerChunkMax = 0;
		for (int ChunkIndex = 0; ChunkIndex < Chunks.Num() - 1; ++ChunkIndex)
		{
			const int Population = Chunks[ChunkIndex].GetNumInstances();
			EntitiesPerChunkMin = FMath::Min(Population, EntitiesPerChunkMin);
			EntitiesPerChunkMax = FMath::Max(Population, EntitiesPerChunkMax);
		}
		Ar.Logf(ELogVerbosity::Log, TEXT("\tEntity Occupancy: %.1f%% (min: %.1f%%, max: %.1f%%)"),
			Scaler * static_cast<float>(EntityMap.Num()),
			Scaler * static_cast<float>(EntitiesPerChunkMin),
			Scaler * static_cast<float>(EntitiesPerChunkMax));
	}
	else 
	{
		Ar.Logf(ELogVerbosity::Log, TEXT("\tEntity Occupancy: %.1f%%"),
			CurrentEntityCapacity > 0 ? ((static_cast<float>(EntityMap.Num()) * 100.0f) / static_cast<float>(CurrentEntityCapacity)) : 0.f);
	}
	Ar.Logf(ELogVerbosity::Log, TEXT("\tBytes / Entity  : %u"), TotalBytesPerEntity);
	Ar.Logf(ELogVerbosity::Log, TEXT("\tEntities / Chunk: %d"), NumEntitiesPerChunk);

	Ar.Logf(ELogVerbosity::Log, TEXT("\tOffset 0x%04X: Entity[] (%llu bytes each)"), EntityListOffsetWithinChunk, sizeof(FMassEntityHandle));
	int32 NextExpectedOffset = EntityListOffsetWithinChunk + sizeof(FMassEntityHandle) * NumEntitiesPerChunk;
	int32 TotalPaddingBytes = 0;
	int32 TotalBytesOfValidData = sizeof(FMassEntityHandle) * NumEntitiesPerChunk;
	for (const FMassArchetypeFragmentConfig& FragmentConfig : FragmentConfigs)
	{
		const int32 PaddingBeforeFragment = FragmentConfig.ArrayOffsetWithinChunk - NextExpectedOffset;
		if (PaddingBeforeFragment > 0)
		{
			Ar.Logf(ELogVerbosity::Log, TEXT("\tOffset 0x%04X: <Padding> (%d bytes)"), NextExpectedOffset, PaddingBeforeFragment);
			TotalPaddingBytes += PaddingBeforeFragment;
		}
		TotalBytesOfValidData += FragmentConfig.FragmentType->GetStructureSize() * NumEntitiesPerChunk;
		Ar.Logf(ELogVerbosity::Log, TEXT("\tOffset 0x%04X: %s[] (%d bytes each)"), FragmentConfig.ArrayOffsetWithinChunk, *FragmentConfig.FragmentType->GetName(), FragmentConfig.FragmentType->GetStructureSize());
		NextExpectedOffset = FragmentConfig.ArrayOffsetWithinChunk + FragmentConfig.FragmentType->GetStructureSize() * NumEntitiesPerChunk;
	}

	const SIZE_T UnusablePaddingOffset = TotalBytesPerEntity * NumEntitiesPerChunk;
	const SIZE_T UnusablePaddingAmount = GetChunkAllocSize() - UnusablePaddingOffset;
	if (UnusablePaddingAmount > 0)
	{
		Ar.Logf(ELogVerbosity::Log, TEXT("\tOffset 0x%04llX: WastePadding[] (%llu bytes total)"), UnusablePaddingOffset, UnusablePaddingAmount);
	}

	if (TotalPaddingBytes > 0)
	{
		Ar.Logf(ELogVerbosity::Log, TEXT("\tTotal alignment padding between fragments: %d bytes"), TotalPaddingBytes);
	}
}

void FMassArchetypeData::DebugPrintEntity(FMassEntityHandle Entity, FOutputDevice& Ar, const TCHAR* InPrefix) const
{
	for (const FMassArchetypeFragmentConfig& FragmentConfig : FragmentConfigs)
	{
		const void* Data = GetFragmentDataForEntityChecked(FragmentConfig.FragmentType, Entity);
		
		FString FragmentName = FragmentConfig.FragmentType->GetName();
		FragmentName.RemoveFromStart(InPrefix);

		FString ValueStr;
		FragmentConfig.FragmentType->ExportText(ValueStr, Data, /*Default*/nullptr, /*OwnerObject*/nullptr, EPropertyPortFlags::PPF_IncludeTransient, /*ExportRootScope*/nullptr);

		Ar.Logf(TEXT("%s: %s"), *FragmentName, *ValueStr);
	}
}

const FMassArchetypeChunk& FMassArchetypeData::DebugGetChunk(const int32 Index) const
{
	static FMassArchetypeChunk DummyChunk(0, {}, {});
	return Chunks.IsValidIndex(Index) ? Chunks[Index] : DummyChunk;
}
#endif // WITH_MASSENTITY_DEBUG

void FMassArchetypeData::SetDebugColor(const FColor InDebugColor)
{
#if WITH_MASSENTITY_DEBUG
	if (InDebugColor == FColor{0})
	{
		// pick a color based on the composition
		const uint32 CompositionHash = GetTypeHash(CompositionBitSet);
		const uint8* Bytes = reinterpret_cast<const uint8*>(&CompositionHash);
		
		const FLinearColor AdjustedColor = FLinearColor::MakeFromHSV8(
			static_cast<uint8>((Bytes[0] >> 1) + (Bytes[1] >> 1)),
			static_cast<uint8>((Bytes[2] >> 1) + 128),
			static_cast<uint8>((Bytes[3] >> 1) + 128)
		);
		DebugColor = AdjustedColor.ToFColorSRGB();
	}
	else
	{
		DebugColor = InDebugColor;
	}
#endif // WITH_MASSENTITY_DEBUG
}

void FMassArchetypeData::REMOVEME_GetArrayViewForFragmentInChunk(int32 ChunkIndex, const UScriptStruct* FragmentType, void*& OutChunkBase, int32& OutNumEntities)
{
	const FMassArchetypeChunk& Chunk = Chunks[ChunkIndex];
	const int32 FragmentIndex = FragmentIndexMap.FindChecked(FragmentType);

	OutChunkBase = FragmentConfigs[FragmentIndex].GetFragmentData(Chunk.GetRawMemory(), 0);
	OutNumEntities = Chunk.GetNumInstances();
}

//-----------------------------------------------------------------------------
// FMassArchetypeData batched api
//-----------------------------------------------------------------------------
void FMassArchetypeData::BatchAddEntities(TConstArrayView<FMassEntityHandle> Entities, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, TArray<FMassArchetypeEntityCollection::FArchetypeEntityRange>& OutNewRanges)
{
	SCOPE_CYCLE_COUNTER(STAT_Mass_ArchetypeBatchAdd);

	checkfSlow(SharedFragmentValues.DoesMatchComposition(CompositionBitSet)
		, TEXT("%hs parameter SharedFragmentValues doesn't match archetype's composition"), __FUNCTION__);
	
	++ArchetypeVersion;

	FMassArchetypeEntityCollection::FArchetypeEntityRange ResultSubchunk;
	ResultSubchunk.ChunkIndex = 0;
	int32 NumberMoved = 0;
	do 
	{
		ResultSubchunk = PrepareNextEntitiesSpanInternal(MakeArrayView(Entities.GetData() + NumberMoved, Entities.Num() - NumberMoved), SharedFragmentValues, ResultSubchunk.ChunkIndex);
		check(Chunks.IsValidIndex(ResultSubchunk.ChunkIndex) && Chunks[ResultSubchunk.ChunkIndex].IsValidSubChunk(ResultSubchunk.SubchunkStart, ResultSubchunk.Length));
		
		for (const FMassArchetypeFragmentConfig& FragmentConfig : FragmentConfigs)
		{
			void* FragmentPtr = FragmentConfig.GetFragmentData(Chunks[ResultSubchunk.ChunkIndex].GetRawMemory(), ResultSubchunk.SubchunkStart);
			FragmentConfig.FragmentType->InitializeStruct(FragmentPtr, ResultSubchunk.Length);
		}

		NumberMoved += ResultSubchunk.Length;

		OutNewRanges.Add(ResultSubchunk);

	} while (NumberMoved < Entities.Num());
}

void FMassArchetypeData::BatchMoveEntitiesToAnotherArchetype(const FMassArchetypeEntityCollection& EntityCollection
	, FMassArchetypeData& NewArchetype, TArray<FMassEntityHandle>& OutEntitiesBeingMoved
	, TArray<FMassArchetypeEntityCollection::FArchetypeEntityRange>* OutNewRanges, const FMassArchetypeSharedFragmentValues* SharedFragmentValuesToAdd
	, const FMassSharedFragmentBitSet* SharedFragmentToRemoveBitSet, const FMassConstSharedFragmentBitSet* ConstSharedFragmentToRemoveBitSet)
{
	check(&NewArchetype != this);

	// verify the new archetype's shared fragment composition matches current archetype's composition modified as requested
	const bool bChangeSharedFragments = SharedFragmentValuesToAdd || SharedFragmentToRemoveBitSet || ConstSharedFragmentToRemoveBitSet;
	FMassElementBitSet NewSharedElementComposition;
	if (bChangeSharedFragments)
	{
		NewSharedElementComposition = CompositionBitSet.GetSharedFragments();
		if (SharedFragmentToRemoveBitSet)
		{
			NewSharedElementComposition -= *SharedFragmentToRemoveBitSet;
		}
		if (ConstSharedFragmentToRemoveBitSet)
		{
			NewSharedElementComposition -= *ConstSharedFragmentToRemoveBitSet;
		}
		if (SharedFragmentValuesToAdd)
		{
			NewSharedElementComposition += SharedFragmentValuesToAdd->GetBitSet();
		}

		const bool bIsValidArchetype = DoContainEquivalentSharedFragments(NewArchetype.GetCompositionBitSet(), NewSharedElementComposition);
		testableCheckfReturn(bIsValidArchetype, return, TEXT("%hs combined new shared fragment composition doesn't match the  target archetype's composition"), __FUNCTION__);
	}

	TArray<FMassArchetypeEntityCollection::FArchetypeEntityRange> Subchunks(EntityCollection.GetRanges());

	const int32 InitialOutEntitiesCount = OutEntitiesBeingMoved.Num();

	for (const FMassArchetypeEntityCollection::FArchetypeEntityRange EntityRange : Subchunks)
	{
		if (!ensureMsgf(EntityRange.IsSet() && EntityRange.Length >= 0, TEXT("We only expect to get valid EntityRanges at this point.")))
		{
			continue;
		}

		FMassArchetypeChunk& Chunk = Chunks[EntityRange.ChunkIndex];
		const int32 RangeLength = CalculateRangeLength(EntityRange, Chunk);

		// 0 - consider compacting new archetype to ensure larger empty spaces
		// 1. find next free spot in the destination archetype
		// 2. min(amount of elements) to move

		// gather entities we're about to remove
		FMassEntityHandle* DyingEntityPtr = &Chunk.GetEntityArrayElementRef(EntityListOffsetWithinChunk, EntityRange.SubchunkStart);
		OutEntitiesBeingMoved.Append(DyingEntityPtr, RangeLength);

		FMassArchetypeEntityCollection::FArchetypeEntityRange ResultSubChunk;
		ResultSubChunk.ChunkIndex = 0;
		ResultSubChunk.Length = 0;
		int32 NumberMoved = 0;

		do
		{
			const int32 IndexWithinChunk = EntityRange.SubchunkStart + NumberMoved;

			if (bChangeSharedFragments == false)
			{
				ResultSubChunk = NewArchetype.PrepareNextEntitiesSpanInternal(MakeArrayView(DyingEntityPtr + NumberMoved, RangeLength - NumberMoved)
					, Chunk.GetSharedFragmentValues(), ResultSubChunk.ChunkIndex);
			}
			else
			{
				// create new shared values
				FMassArchetypeSharedFragmentValues NewSharedValues = FMassArchetypeSharedFragmentValues::CreateCombined(Chunk.GetSharedFragmentValues(), NewSharedElementComposition, SharedFragmentValuesToAdd);

				ResultSubChunk = NewArchetype.PrepareNextEntitiesSpanInternal(MakeArrayView(DyingEntityPtr + NumberMoved, RangeLength - NumberMoved)
					, NewSharedValues, ResultSubChunk.ChunkIndex);
			}

			FMassArchetypeChunk& NewChunk = NewArchetype.Chunks[ResultSubChunk.ChunkIndex];
			MoveFragmentsToAnotherArchetypeInternal(NewArchetype, NewChunk, ResultSubChunk.SubchunkStart, Chunk, IndexWithinChunk, ResultSubChunk.Length);

			NumberMoved += ResultSubChunk.Length;

			if (OutNewRanges)
			{
				// if the new ResultSubChunk is right next to the last stored one then merge them both
				if (OutNewRanges->Num() && OutNewRanges->Last().IsAdjacentAfter(ResultSubChunk))
				{
					OutNewRanges->Last().Length += ResultSubChunk.Length;
				}
				else // just add
				{
					OutNewRanges->Add(ResultSubChunk);
				}
			}

		} while (NumberMoved < RangeLength);

	}

	UE_TRACE_MASS_ARCHETYPE_CREATED(NewArchetype)
	UE_TRACE_MASS_ENTITIES_MOVED(TConstArrayView<FMassEntityHandle>(OutEntitiesBeingMoved).Mid(InitialOutEntitiesCount), NewArchetype)

	// Sorting the subchunks info so that subchunks of a given chunk are processed "from the back". Otherwise removing
	// a subchunk from the front of the chunk would inevitably invalidate following subchunks' information.
	// Note that we do this after already having added the entities to the new archetype to preserve the order of entities
	// as given by the input data.
	Subchunks.Sort([](const FMassArchetypeEntityCollection::FArchetypeEntityRange& A, const FMassArchetypeEntityCollection::FArchetypeEntityRange& B)
	{
		return A.ChunkIndex < B.ChunkIndex || (A.ChunkIndex == B.ChunkIndex && A.SubchunkStart > B.SubchunkStart);
	});

	for (const FMassArchetypeEntityCollection::FArchetypeEntityRange& Subchunk : Subchunks)
	{
		BatchRemoveEntitiesInternal(Subchunk.ChunkIndex, Subchunk.SubchunkStart, Subchunk.Length);
	}

	for (int i = InitialOutEntitiesCount; i < OutEntitiesBeingMoved.Num(); ++i)
	{
		EntityMap.FindAndRemoveChecked(OutEntitiesBeingMoved[i].Index);
	}
}

FMassArchetypeEntityCollection::FArchetypeEntityRange FMassArchetypeData::PrepareNextEntitiesSpanInternal(TConstArrayView<FMassEntityHandle> Entities
	, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, int32 StartingChunk)
{
	checkf(SharedFragmentValues.IsSorted(), TEXT("Expecting shared fragment values to be previously sorted"));
	checkfSlow(SharedFragmentValues.DoesMatchComposition(CompositionBitSet)
		, TEXT("Expecting values for every specified shared fragment in the archetype and only those"));

	int32 StartIndexWithinChunk = INDEX_NONE;
	int32 AbsoluteStartIndex = 0;

	FMassArchetypeChunk* DestinationChunk = nullptr;
	
	int32 ChunkIndex = StartingChunk;
	// find a chunk with any room left
	for (; ChunkIndex < Chunks.Num(); ++ChunkIndex)
	{
		FMassArchetypeChunk& Chunk = Chunks[ChunkIndex];
		if (Chunk.GetNumInstances() < NumEntitiesPerChunk && Chunk.GetSharedFragmentValues().IsEquivalent(SharedFragmentValues))
		{
			StartIndexWithinChunk = Chunk.GetNumInstances();
			AbsoluteStartIndex = ChunkIndex * NumEntitiesPerChunk + StartIndexWithinChunk;

			DestinationChunk = &Chunk;

			if (StartIndexWithinChunk == 0)
			{
				Chunk.Recycle(ChunkFragmentsTemplate, SharedFragmentValues);
			}
			break;
		}
	}

	// if no chunk found create one
	if (DestinationChunk == nullptr)
	{
		ChunkIndex = Chunks.Num();
		AbsoluteStartIndex = Chunks.Num() * NumEntitiesPerChunk;
		StartIndexWithinChunk = 0;

		DestinationChunk = &Chunks.Emplace_GetRef(GetChunkAllocSize(), ChunkFragmentsTemplate, SharedFragmentValues);
	}

	check(DestinationChunk);

	// we might be able to fit in less entities than requested
	const int32 NumToAdd = FMath::Min(NumEntitiesPerChunk - StartIndexWithinChunk, Entities.Num());
	check(NumToAdd);
	DestinationChunk->AddMultipleInstances(NumToAdd);

	// Add to the table and map
	int32 AbsoluteIndex = AbsoluteStartIndex;
	for (int32 i = 0; i < NumToAdd; ++i)
	{
		EntityMap.Add(Entities[i].Index, AbsoluteIndex++);
	}

	FMassEntityHandle* FirstAddedEntity = &DestinationChunk->GetEntityArrayElementRef(EntityListOffsetWithinChunk, StartIndexWithinChunk);
	FMemory::Memcpy(FirstAddedEntity, Entities.GetData(), sizeof(FMassEntityHandle) * NumToAdd);

	return FMassArchetypeEntityCollection::FArchetypeEntityRange(ChunkIndex, StartIndexWithinChunk, NumToAdd);
}

void FMassArchetypeData::BatchRemoveEntitiesInternal(const int32 ChunkIndex, const int32 StartIndexWithinChunk, const int32 NumberToRemove)
{
	if (UNLIKELY(NumberToRemove <= 0))
	{
		return;
	}

	++EntityOrderVersion;
	++ArchetypeVersion;

	FMassArchetypeChunk& Chunk = Chunks[ChunkIndex];
	
	const int32 NumberToMove = FMath::Min(Chunk.GetNumInstances() - (StartIndexWithinChunk + NumberToRemove), NumberToRemove);
	checkf(NumberToMove >= 0, TEXT("Trying to move a negative number of elements indicates a problem with sub-chunk indicators, it's possibly out of date."));

	if (NumberToMove > 0)
	{
		FMassEntityHandle* DyingEntityPtr = &Chunk.GetEntityArrayElementRef(EntityListOffsetWithinChunk, StartIndexWithinChunk);

		const int32 SwapStartIndex = Chunk.GetNumInstances() - NumberToMove;
		checkf((StartIndexWithinChunk + NumberToMove - 1) < SwapStartIndex, TEXT("Remove and Move ranges overlap"));

		MoveFragmentsToNewLocationInternal({ Chunk.GetRawMemory(), StartIndexWithinChunk }, { Chunk.GetRawMemory(), SwapStartIndex }, NumberToMove);

		if (Chunk.HasSparseElements())
		{
			UE::Mass::FChunkSparseElements& SparseElements = Chunk.GetSparseElementsUnsafe();
			const bool bElementsPresentDirty = SparseElements.MoveElementsFrom(SparseElements, SwapStartIndex, StartIndexWithinChunk, NumberToMove);
			bSparseElementsBitSetDirty |= bElementsPresentDirty;
		}

		// Update the entity table and map
		const FMassEntityHandle* MovingEntityPtr = &Chunk.GetEntityArrayElementRef(EntityListOffsetWithinChunk, SwapStartIndex);
		int32 AbsoluteIndex = ChunkIndex * NumEntitiesPerChunk + StartIndexWithinChunk;

		for (int i = 0; i < NumberToMove; ++i)
		{
			DyingEntityPtr[i] = MovingEntityPtr[i];
			EntityMap.FindChecked(MovingEntityPtr[i].Index) = AbsoluteIndex++;
		}
	}

	Chunk.RemoveMultipleInstances(NumberToRemove);

	// If the chunk itself is empty now, see if we can remove it entirely
	// Note: This is only possible for trailing chunks, to avoid messing up the absolute indices in the entities map
	while ((Chunks.Num() > 0) && (Chunks.Last().GetNumInstances() == 0))
	{
		Chunks.RemoveAt(Chunks.Num() - 1, EAllowShrinking::No);
	}
}

void FMassArchetypeData::MoveFragmentsToAnotherArchetypeInternal(FMassArchetypeData& TargetArchetype, FMassArchetypeChunk& TargetChunk, const int32 NewIndexWithinChunk
	, FMassArchetypeChunk& SourceChunk, const int32 OriginalIndexWithinChunk, const int32 ElementsNum)
{
	FTransientChunkLocation Target(TargetChunk.GetRawMemory(), NewIndexWithinChunk);
	FTransientChunkLocation Source(SourceChunk.GetRawMemory(), OriginalIndexWithinChunk);
	
	++TargetArchetype.ArchetypeVersion;

	// for every TargetArchetype's fragment see if it was in the old archetype as well and if so copy it's value. 
	// If not then initialize the fragment.
	for (const FMassArchetypeFragmentConfig& TargetFragmentConfig : TargetArchetype.FragmentConfigs)
	{
		const int32* OldFragmentIndex = FragmentIndexMap.Find(TargetFragmentConfig.FragmentType);
		void* Dst = TargetFragmentConfig.GetFragmentData(Target.RawChunkMemory, Target.IndexWithinChunk);

		// Only copy if the fragment type exists in both archetypes
		if (OldFragmentIndex)
		{
			const void* Src = FragmentConfigs[*OldFragmentIndex].GetFragmentData(Source.RawChunkMemory, Source.IndexWithinChunk);
			FMemory::Memcpy(Dst, Src, TargetFragmentConfig.FragmentType->GetStructureSize() * ElementsNum);
		}
		else
		{
			// the fragment's unique to the TargetArchetype need to be initialized
			// @todo we're doing it for tags here as well. A tiny bit of perf lost. Probably not worth adding a check
			// but something to keep in mind. Will go away once tags are more of an archetype fragment than entity's
			TargetFragmentConfig.FragmentType->InitializeStruct(Dst, ElementsNum);
		}
	}

	// Delete fragments that were left behind
	for (const FMassArchetypeFragmentConfig& FragmentConfig : FragmentConfigs)
	{
		// If the fragment is not in the new archetype, destroy it.
		const int32* NewFragmentIndex = TargetArchetype.FragmentIndexMap.Find(FragmentConfig.FragmentType);
		if (NewFragmentIndex == nullptr)
		{
			void* DyingFragmentPtr = FragmentConfig.GetFragmentData(Source.RawChunkMemory, Source.IndexWithinChunk);
			FragmentConfig.FragmentType->DestroyStruct(DyingFragmentPtr, ElementsNum);
		}
	}

	// move sparse elements
	if (SourceChunk.HasSparseElements())
	{
		UE::Mass::FChunkSparseElements& SourceSparseElements = SourceChunk.GetSparseElements(NumEntitiesPerChunk);
		// calling GetSparseElements to make sure FChunkSparseElements gets created with the right size (FChunkSparseElements)
		UE::Mass::FChunkSparseElements& TargetElements = TargetChunk.GetSparseElements(TargetArchetype.NumEntitiesPerChunk);
		const bool bElementsPresentDirty = TargetElements.MoveElementsFrom(SourceSparseElements, OriginalIndexWithinChunk, NewIndexWithinChunk, ElementsNum);
		bSparseElementsBitSetDirty |= bElementsPresentDirty;
		TargetArchetype.bSparseElementsBitSetDirty |= bElementsPresentDirty;
	}
}

FORCEINLINE void FMassArchetypeData::MoveFragmentsToNewLocationInternal(FMassArchetypeData::FTransientChunkLocation Target, const FMassArchetypeData::FTransientChunkLocation Source, const int32 NumberToMove)
{
	for (const FMassArchetypeFragmentConfig& FragmentConfig : FragmentConfigs)
	{
		void* TargetPtr = FragmentConfig.GetFragmentData(Target.RawChunkMemory, Target.IndexWithinChunk);
		const void* SourcePtr = FragmentConfig.GetFragmentData(Source.RawChunkMemory, Source.IndexWithinChunk); 

		FMemory::Memcpy(TargetPtr, SourcePtr, FragmentConfig.FragmentType->GetStructureSize() * NumberToMove);
	}
}

void FMassArchetypeData::BatchSetFragmentValues(TConstArrayView<FMassArchetypeEntityCollection::FArchetypeEntityRange> EntityCollection, const FMassGenericPayloadViewSlice& Payload)
{
	int32 EntitiesHandled = 0;

	for (const FMassArchetypeEntityCollection::FArchetypeEntityRange EntityRange : EntityCollection)
	{
		FMassArchetypeChunk& Chunk = Chunks[EntityRange.ChunkIndex];
		const int32 RangeLength = CalculateRangeLength(EntityRange, Chunk);

		for (int i = 0; i < Payload.Num(); ++i)
		{
			FStructArrayView FragmentPayload = Payload[i];
			check(FragmentPayload.Num() - EntitiesHandled >= RangeLength);

			const UScriptStruct* FragmentType = FragmentPayload.GetScriptStruct();
			check(FragmentType);

			if (UE::Mass::IsSparse(FragmentType) == false)
			{
				const int32 FragmentIndex = FragmentIndexMap.FindChecked(FragmentType);
				void* Dst = FragmentConfigs[FragmentIndex].GetFragmentData(Chunk.GetRawMemory(), EntityRange.SubchunkStart);
				void* Src = FragmentPayload.GetDataAt(EntitiesHandled);

				// MoveAssignScriptStruct uses C++ move assignment for types that opt in via WithMoveAssign,
				// leaving the source in a valid moved-from state. For types without the trait this falls
				// back to CopyScriptStruct (zero regression).
				FragmentType->MoveAssignScriptStruct(Dst, Src, RangeLength);
			}
			else
			{
				UE::Mass::FChunkSparseElements& SparseElements = Chunk.GetSparseElements(NumEntitiesPerChunk);
				bSparseElementsBitSetDirty |= SparseElements.BatchAdd(FragmentType, EntityRange.SubchunkStart, RangeLength);

				const FMassEntityHandle* EntitiesArray = Chunk.GetEntityArray(EntityListOffsetWithinChunk);
				TArrayView<const FMassEntityHandle> EntityHandlesView(&EntitiesArray[EntityRange.SubchunkStart], RangeLength);
				FStructArrayView FragmentPayloadSlice(*FragmentType, FragmentPayload.GetDataAt(EntitiesHandled), EntityHandlesView.Num());
				SparseElementsStorage->BatchAddElementInstancesToEntities(EntityHandlesView, FragmentPayloadSlice);
			}
		}

		EntitiesHandled += RangeLength;
	}
}

void FMassArchetypeData::BatchAddSparseElementToEntities(const FMassArchetypeEntityCollection& EntityCollection, TNotNull<const UScriptStruct*> ElementType)
{
	checkf(UE::Mass::IsSparse(ElementType), TEXT("%s is not a sparse element type"), *ElementType->GetName());

	const bool bIsTag = UE::Mass::IsA<FMassTag>(ElementType);

	TArray<FMassArchetypeEntityCollection::FArchetypeEntityRange> Subchunks(EntityCollection.GetRanges());
	for (const FMassArchetypeEntityCollection::FArchetypeEntityRange EntityRange : Subchunks)
	{
		FMassArchetypeChunk& Chunk = Chunks[EntityRange.ChunkIndex];
		const int32 RangeLength = CalculateRangeLength(EntityRange, Chunk);

		UE::Mass::FChunkSparseElements& SparseElements = Chunk.GetSparseElements(NumEntitiesPerChunk);
		bSparseElementsBitSetDirty |= SparseElements.BatchAdd(ElementType, EntityRange.SubchunkStart, RangeLength);

		if (!bIsTag)
		{
			const FMassEntityHandle* EntitiesArray = Chunk.GetEntityArray(EntityListOffsetWithinChunk);
			TArrayView<const FMassEntityHandle> EntityHandlesView(&EntitiesArray[EntityRange.SubchunkStart], RangeLength);
			SparseElementsStorage->BatchAddElementToEntities(EntityHandlesView, ElementType);
		}
	}
}

void FMassArchetypeData::BatchRemoveSparseElementFromEntities(const FMassArchetypeEntityCollection& EntityCollection, TNotNull<const UScriptStruct*> ElementType)
{
	checkf(UE::Mass::IsSparse(ElementType), TEXT("%s is not a sparse element type"), *ElementType->GetName());

	const bool bIsTag = UE::Mass::IsA<FMassTag>(ElementType);

	TArray<FMassArchetypeEntityCollection::FArchetypeEntityRange> Subchunks(EntityCollection.GetRanges());
	for (const FMassArchetypeEntityCollection::FArchetypeEntityRange EntityRange : Subchunks)
	{
		FMassArchetypeChunk& Chunk = Chunks[EntityRange.ChunkIndex];
		const int32 RangeLength = CalculateRangeLength(EntityRange, Chunk);

		UE::Mass::FChunkSparseElements& SparseElements = Chunk.GetSparseElementsUnsafe();
		if (SparseElements.IsEmpty() == false)
		{
			const bool bElementsPresentDirty = SparseElements.BatchRemove(ElementType, EntityRange.SubchunkStart, RangeLength);
			bSparseElementsBitSetDirty |= bElementsPresentDirty;

			if (!bIsTag)
			{
				const FMassEntityHandle* EntitiesArray = Chunk.GetEntityArray(EntityListOffsetWithinChunk);
				TArrayView<const FMassEntityHandle> EntityHandlesView(&EntitiesArray[EntityRange.SubchunkStart], RangeLength);
				SparseElementsStorage->BatchRemoveElementFromEntities(EntityHandlesView, ElementType);
			}
		}
	}
}

bool FMassArchetypeData::IsEquivalent(const FMassElementBitSet& OtherCompositionBitSet, const UE::Mass::FArchetypeGroups& OtherGroups) const
{
	return CompositionBitSet.IsEquivalent(OtherCompositionBitSet) && Groups == OtherGroups;
}

//-----------------------------------------------------------------------------
// FMassArchetypeHelper
//-----------------------------------------------------------------------------
bool FMassArchetypeHelper::DoesArchetypeMatchRequirements(const FMassArchetypeData& Archetype, const FMassFragmentRequirements& Requirements)
{
	return DoesArchetypeMatchRequirements(Archetype.GetCompositionBitSet(), Requirements);
}
bool FMassArchetypeHelper::DoesArchetypeMatchRequirements(const FMassArchetypeCompositionDescriptor& ArchetypeComposition, const FMassFragmentRequirements& Requirements)
{
	return Requirements.DoesArchetypeMatchRequirements(ArchetypeComposition);
}

bool FMassArchetypeHelper::DoesArchetypeMatchRequirements(const FMassElementBitSet& ArchetypeCompositionBitSet, const FMassFragmentRequirements& Requirements)
{
	return DoesArchetypeMatchRequirements(FMassArchetypeCompositionDescriptor(ArchetypeCompositionBitSet), Requirements);
}

#if WITH_MASSENTITY_DEBUG
bool FMassArchetypeHelper::DoesArchetypeMatchRequirements(const FMassArchetypeData& Archetype, const FMassFragmentRequirements& Requirements
	, const bool bBailOutOnFirstFail, FOutputDevice* OutputDevice)
{
	if (DoesArchetypeMatchRequirements(Archetype.GetCompositionBitSet(), Requirements))
	{
		// nothing to log
		return true;
	}
	
	if (OutputDevice)
	{
		// do logging
		OutputDevice->Logf(TEXT("%s")
			, *FMassDebugger::GetArchetypeRequirementCompatibilityDescription(Requirements, FMassArchetypeCompositionDescriptor(Archetype.GetCompositionBitSet())));
	}

	return false;
}
#endif // WITH_MASSENTITY_DEBUG