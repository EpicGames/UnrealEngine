// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassArchetypeTypes.h"
#include "MassEntityManager.h"
#include "MassArchetypeData.h"
#include "MassEntityUtils.h"
#include "Containers/StridedView.h"
#include "Mass/TestableEnsures.h"

//-----------------------------------------------------------------------------
// FMassArchetypeHandle
//-----------------------------------------------------------------------------
uint32 GetTypeHash(const FMassArchetypeHandle& Instance)
{
	return GetTypeHash(Instance.DataPtr.Get());
}

const FMassElementBitSet& FMassArchetypeHandle::GetCompositionBitSetChecked() const
{
	check(DataPtr);
	return DataPtr.Get()->GetCompositionBitSet();
}

//-----------------------------------------------------------------------------
// FMassArchetypeVersionedHandle
//-----------------------------------------------------------------------------
FMassArchetypeVersionedHandle::FMassArchetypeVersionedHandle(const FMassArchetypeHandle& InHandle)
	: ArchetypeHandle(InHandle)
	, HandleVersion(ArchetypeHandle.IsValid() ? ArchetypeHandle.DataPtr->GetEntityOrderVersion() : 0)
{
}

FMassArchetypeVersionedHandle::FMassArchetypeVersionedHandle(FMassArchetypeHandle&& InHandle)
	: ArchetypeHandle(Forward<FMassArchetypeHandle>(InHandle))
	, HandleVersion(ArchetypeHandle.IsValid() ? ArchetypeHandle.DataPtr->GetEntityOrderVersion() : 0)
{
}

bool FMassArchetypeVersionedHandle::IsUpToDate() const
{
	return ArchetypeHandle.IsValid() && (ArchetypeHandle.DataPtr->GetEntityOrderVersion() == HandleVersion);
}

//-----------------------------------------------------------------------------
// FMassArchetypeEntityCollection 
//-----------------------------------------------------------------------------
FORCEINLINE FMassArchetypeEntityCollection::FArchetypeEntityRange FMassArchetypeEntityCollection::CreateRangeForEntity(const FMassArchetypeHandle& InArchetype, const FMassEntityHandle EntityHandle)
{
	if (const FMassArchetypeData* ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandle(InArchetype))
	{
		const int32* TrueIndexPtr = ArchetypeData->GetInternalIndexForEntity(EntityHandle.Index);
		check(TrueIndexPtr);
		const int32 TrueIndex = *TrueIndexPtr;
		checkf(TrueIndex >= 0, TEXT("Negative values are unexpected here"));

		const int32 NumEntitiesPerChunk = ArchetypeData->GetNumEntitiesPerChunk();
		const int32 ChunkIndex = TrueIndex / NumEntitiesPerChunk;
		const int32 SubchunkStart = TrueIndex - (NumEntitiesPerChunk * ChunkIndex);
		return FArchetypeEntityRange(ChunkIndex, SubchunkStart, 1);
	}

	return FArchetypeEntityRange(0, EntityHandle.Index, 1);
}

FMassArchetypeEntityCollection::FMassArchetypeEntityCollection(const FMassArchetypeHandle& InArchetype, TConstArrayView<FMassEntityHandle> InEntities, EDuplicatesHandling DuplicatesHandling)
	: Archetype(InArchetype)
{
	if (InEntities.Num() <= 0)
	{
		return;
	}
	if (InEntities.Num() == 1)
	{
		Ranges.Add(CreateRangeForEntity(InArchetype, InEntities[0]));
		return;
	}

	const FMassArchetypeData* ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandle(InArchetype);

	// InEntities has a real chance of not being sorted by AbsoluteIndex. We gotta fix that to optimize how we process the data 
	TArray<int32> TrueIndices;
	TrueIndices.AddUninitialized(InEntities.Num());
	int32 NumValidEntities = 0;
	if (ArchetypeData)
	{
		for (const FMassEntityHandle& Entity : InEntities)
		{
			if (Entity.IsValid())
			{
				if (const int32* TrueIndex = ArchetypeData->GetInternalIndexForEntity(Entity.Index))
				{
					TrueIndices[NumValidEntities++] = *TrueIndex;
				}
			}
		}
	}
	else
	{
		// special case, where we have a bunch of entities that have been built but not assigned an archetype yet.
		// we use their base index for the sake of sorting here. Will still get some perf benefits and we can keep using
		// FMassArchetypeEntityCollection as the generic batched API wrapper for entities
		for (const FMassEntityHandle& Entity : InEntities)
		{
			if (Entity.IsValid())
			{
				TrueIndices[NumValidEntities++] = Entity.Index;
			}
		}
	}

	TrueIndices.SetNum(NumValidEntities, EAllowShrinking::No);
	TrueIndices.Sort();

#if DO_GUARD_SLOW
	if (DuplicatesHandling == NoDuplicates)
	{
		// ensure there are no duplicates. 
		int32 PrevIndex = TrueIndices[0];
		for (int j = 1; j < TrueIndices.Num(); ++j)
		{
			checkf(TrueIndices[j] != PrevIndex, TEXT("InEntities contains duplicate while DuplicatesHandling is set to NoDuplicates"));
			if (TrueIndices[j] == PrevIndex)
			{
				// fix it, for development's sake
				DuplicatesHandling = FoldDuplicates;
				break;
			}
			PrevIndex = TrueIndices[j];
		}
	}
#endif // DO_GUARD_SLOW

	if (DuplicatesHandling == FoldDuplicates)
	{
		int32 PrevIndex = TrueIndices[0];
		for (int j = 1; j < TrueIndices.Num(); ++j)
		{
			if (TrueIndices[j] == PrevIndex)
			{
				const int32 Num = TrueIndices.Num();
				int Skip = 0;
				while ((j + ++Skip) < Num && TrueIndices[j + Skip] == PrevIndex);
				
				TrueIndices.RemoveAt(j, Skip, EAllowShrinking::No);
				--j;
				continue;
			}
			PrevIndex = TrueIndices[j];
		}
	}

	BuildEntityRanges(MakeStridedView<const int32>(TrueIndices));
}

FMassArchetypeEntityCollection::FMassArchetypeEntityCollection(const FMassArchetypeHandle& InArchetype, const FMassEntityHandle EntityHandle)
	: Ranges({CreateRangeForEntity(InArchetype, EntityHandle)})
	, Archetype(InArchetype)
{	
}

FMassArchetypeEntityCollection::FMassArchetypeEntityCollection(FMassArchetypeHandle&& InArchetype, const FMassEntityHandle EntityHandle)
	: Ranges({CreateRangeForEntity(InArchetype, EntityHandle)})
	, Archetype(MoveTemp(InArchetype))
{	
}

void FMassArchetypeEntityCollection::BuildEntityRanges(TStridedView<const int32> TrueIndices)
{
	checkf(Ranges.Num() == 0, TEXT("Calling %s is valid only for initial configuration"), ANSI_TO_TCHAR(__FUNCTION__));

	const FMassArchetypeData* ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandle(Archetype);
	const int32 NumEntitiesPerChunk = ArchetypeData ? ArchetypeData->GetNumEntitiesPerChunk() : MAX_int32;

	// the following block of code is splitting up sorted AbsoluteIndices into 
	// continuous chunks
	int32 ChunkEnd = INDEX_NONE;
	FArchetypeEntityRange DummyChunk;
	FArchetypeEntityRange* SubChunkPtr = &DummyChunk;
	int32 SubchunkLen = 0;
	int32 PrevAbsoluteIndex = INDEX_NONE;
	for (const int32 Index : TrueIndices)
	{
		// if run across a chunk border or run into an index discontinuity 
		if (Index >= ChunkEnd || Index != (PrevAbsoluteIndex + 1))
		{
			SubChunkPtr->Length = SubchunkLen;
			// note that both ChunkIndex and ChunkEnd will change only if AbsoluteIndex >= ChunkEnd
			const int32 ChunkIndex = Index / NumEntitiesPerChunk;
			ChunkEnd = (ChunkIndex + 1) * NumEntitiesPerChunk;
			SubchunkLen = 0;
			// new subchunk
			const int32 SubchunkStart = Index - (NumEntitiesPerChunk * ChunkIndex);
			SubChunkPtr = &Ranges.Add_GetRef(FArchetypeEntityRange(ChunkIndex, SubchunkStart));
		}
		++SubchunkLen;
		PrevAbsoluteIndex = Index;
	}

	SubChunkPtr->Length = SubchunkLen;
}

FMassArchetypeEntityCollection::FMassArchetypeEntityCollection(const FMassArchetypeHandle& InArchetypeHandle, const EInitializationType Initialization)
	: Archetype(InArchetypeHandle)
{
	if (Initialization == EInitializationType::GatherAll)
	{
		check(InArchetypeHandle.IsValid());
		GatherChunksFromArchetype();
	}
}

FMassArchetypeEntityCollection::FMassArchetypeEntityCollection(TSharedPtr<FMassArchetypeData>& InArchetype, const EInitializationType Initialization)
	: Archetype(FMassArchetypeHelper::ArchetypeHandleFromData(InArchetype))
{	
	if (Initialization == EInitializationType::GatherAll)
	{
		check(InArchetype.IsValid());
		GatherChunksFromArchetype();
	}
}

void FMassArchetypeEntityCollection::GatherChunksFromArchetype()
{
	if (const FMassArchetypeData* ArchetypePtr = FMassArchetypeHelper::ArchetypeDataFromHandle(Archetype))
	{
		const int32 ChunkCount = ArchetypePtr->GetChunkCount();
		Ranges.Reset(ChunkCount);
		for (int32 i = 0; i < ChunkCount; ++i)
		{
			Ranges.Add(FArchetypeEntityRange(i));
		}
	}
}

bool FMassArchetypeEntityCollection::IsSame(const FMassArchetypeEntityCollection& Other) const
{
	if (Archetype != Other.Archetype || Ranges.Num() != Other.Ranges.Num())
	{
		return false;
	}

	for (int i = 0; i < Ranges.Num(); ++i)
	{
		if (Ranges[i] != Other.Ranges[i])
		{
			return false;
		}
	}
	return true;
}

bool FMassArchetypeEntityCollection::ExportEntityHandles(TArray<FMassEntityHandle>& InOutHandles) const
{
	if (Ranges.IsEmpty())
	{
		return false;
	}

	const int32 InOutHandlesStartingNum = InOutHandles.Num();
	const FMassArchetypeData* ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandle(Archetype);
	if (testableEnsureMsgf(ArchetypeData, TEXT("This operation is supported only for collections containing created entities (i.e. not 'reserved'")))
	{
		testableCheckfReturn(IsUpToDate(), return false, TEXT("The entity collection is out of date. Unable to reliably generate handle list."));

		ArchetypeData->ExportEntityHandles(Ranges, InOutHandles);
		ensureMsgf(InOutHandlesStartingNum != InOutHandles.Num(), TEXT("Exporting non-empty ranges resulted in no entity handles"));
	}

	return InOutHandlesStartingNum != InOutHandles.Num();
}

bool FMassArchetypeEntityCollection::DoesContainOverlappingRanges(FConstEntityRangeArrayView Ranges)
{
	for (int RangeIndex = 1; RangeIndex < Ranges.Num(); ++RangeIndex) 
	{
		if (Ranges[RangeIndex-1].IsOverlapping(Ranges[RangeIndex]))
		{
			return true;
		}
	}
	return false;
}

#if WITH_MASSENTITY_DEBUG
int32 FMassArchetypeEntityCollection::DebugCountEntities() const
{
	int32 EntitiesCount = 0;

	if (const FMassArchetypeData* ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandle(Archetype))
	{
		for (const FArchetypeEntityRange& Range : Ranges)
		{
			EntitiesCount += ArchetypeData->CalculateRangeLength(Range);
		}
	}
	else
	{
		for (const FArchetypeEntityRange& Range : Ranges)
		{
			ensureAlwaysMsgf(Range.Length > 0, TEXT("Empty range length in archetype-less collection is unexpected. There's no way to determine what it means"));
			EntitiesCount += Range.Length;
		}
	}

	return EntitiesCount;
}
#endif // WITH_MASSENTITY_DEBUG

//-----------------------------------------------------------------------------
// FMassArchetypeEntityCollectionWithPayload
//-----------------------------------------------------------------------------
void FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(const FMassEntityManager& EntityManager, const TConstArrayView<FMassEntityHandle> Entities
	, const FMassArchetypeEntityCollection::EDuplicatesHandling DuplicatesHandling, FMassGenericPayloadView Payload
	, TArray<FMassArchetypeEntityCollectionWithPayload>& OutEntityCollections)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("Mass_CreateEntityRangesWithPayload");

	check(Payload.Num() > 0);
	for (const FStructArrayView& Element : Payload.Content)
	{
		check(Entities.Num() == Element.Num());
	}

	struct FEntityInArchetype
	{
		int32 ArchetypeIndex = INDEX_NONE;
		int32 TrueIndex = INDEX_NONE;

		bool operator==(const FEntityInArchetype& Other) const
		{
			return ArchetypeIndex == Other.ArchetypeIndex && TrueIndex == Other.TrueIndex;
		}

		/** @return whether A should come before B in an ordered collection */
		static bool Compare(const FEntityInArchetype& A, const FEntityInArchetype& B)
		{
			// using "greater" to ensure INDEX_NONE archetypes end up at the end of the collection
			return A.ArchetypeIndex > B.ArchetypeIndex || (A.ArchetypeIndex == B.ArchetypeIndex && A.TrueIndex < B.TrueIndex);
		}
	};

	TArray<FEntityInArchetype> EntityData;
	EntityData.AddUninitialized(Entities.Num());

	struct FArchetypeInfo
	{
		// @todo using a handle here is temporary. Once ArchetypeHandle switches to using an index we'll use that instead
		FMassArchetypeHandle Archetype;
		int32 Count = 0;

		bool operator==(const FMassArchetypeHandle& InArchetype) const
		{
			return Archetype == InArchetype;
		}
		bool operator==(const FArchetypeInfo& Other) const
		{
			return Archetype == Other.Archetype;
		}
	};
	TArray<FArchetypeInfo> Archetypes;

	for (int32 i = 0; i < Entities.Num(); ++i)
	{
		const FMassEntityHandle& Entity = Entities[i];
		if (EntityManager.IsEntityValid(Entity))
		{
			// using Unsafe since we just checked that the entity is valid
			const FMassArchetypeHandle ArchetypeHandle = EntityManager.GetArchetypeForEntityUnsafe(Entity);
			const FMassArchetypeData* ArchetypePtr = FMassArchetypeHelper::ArchetypeDataFromHandle(ArchetypeHandle);
		
			// @todo if FMassArchetypeHandle used indices the look up would be a lot faster
			int32 ArchetypeIndex = INDEX_NONE;
		
			if (Archetypes.Find(FArchetypeInfo{ ArchetypeHandle, 0 }, ArchetypeIndex) == false)
			{
				ArchetypeIndex = Archetypes.Add({ ArchetypeHandle, 0 });
			}
			++Archetypes[ArchetypeIndex].Count;
			EntityData[i] = { ArchetypeIndex, ArchetypePtr ? ArchetypePtr->GetInternalIndexForEntityChecked(Entity.Index) : Entity.Index };
		}
		else
		{
			// for invalid entities we create an entry that will result in all of them batched together 
			// (due to having the same archetype index, INDEX_NONE). Since the main logic loop below relies on entries 
			// in Archetypes and the "invalid" EntityData doesn't correspond to any, all the invalid entities 
			// will be silently filtered out.
			EntityData[i] = FEntityInArchetype();
			UE_LOGF(LogMass, Warning, "%s: Invalid entity handle passed in. Ignoring it, but check your code to make sure you don't mix synchronous entity-mutating Mass API function calls with Mass commands"
				, __FUNCTION__);
		}
	}

	// A paranoid programmer might point out that there are no guarantees that a sorting algorithm will compare all elements.
	// While that's true we make an assumption here, that the elements next to each other will in fact all get compared
	// and since all we care about with `bDuplicatesFound` is whether same elements exist (that will be right next to each other
	// in the final lineup) we feel safe in the assumption.
	bool bDuplicatesFound = false;
	UE::Mass::Utils::AbstractSort(Entities.Num(), [&EntityData, &bDuplicatesFound](const int32 LHS, const int32 RHS)
		{
			bDuplicatesFound = bDuplicatesFound || (EntityData[LHS] == EntityData[RHS]);
			return FEntityInArchetype::Compare(EntityData[LHS], EntityData[RHS]);
		}
		, [&EntityData, &Payload](const int32 A, const int32 B)
		{
			::Swap(EntityData[A], EntityData[B]);
			Payload.Swap(A, B);
		});
	ensureMsgf(bDuplicatesFound == false || (DuplicatesHandling != FMassArchetypeEntityCollection::NoDuplicates)
		, TEXT("Caller declared lack of duplicates in the input data, but duplicates have been found"));

#if !UE_BUILD_SHIPPING
	// in non shipping builds we still want to verify that the assumption expressed in bDuplicatesFound comment above
	// is correct
	if (!bDuplicatesFound && (DuplicatesHandling == FMassArchetypeEntityCollection::FoldDuplicates))
	{
		for (int32 EntryIndex = 0; EntryIndex < EntityData.Num() - 1; ++EntryIndex)
		{
			checkf(EntityData[EntryIndex] != EntityData[EntryIndex + 1], TEXT("Assumption regarding comparison between identical elements while sorting is wrong!"));
		}
	}
#endif // !UE_BUILD_SHIPPING

	if (bDuplicatesFound && (DuplicatesHandling == FMassArchetypeEntityCollection::FoldDuplicates))
	{
		// we cannot remove elements from Payload, since it's a view to existing data, we need to sort the data in 
		// such a way that all the duplicates end up at the end of the view. We can then ignore the appropriate
		// number of elements.
		
		// processing Num - 1 elements since there's no point in checking the last one - there's nothing to compare it against
		for (int32 EntryIndex = 0; EntryIndex < EntityData.Num() - 1; ++EntryIndex)
		{	
			FEntityInArchetype& Entry = EntityData[EntryIndex];
			if (Entry.ArchetypeIndex == INDEX_NONE)
			{
				// we're reached INDEX_NONE archetypes, which are at the end of EntityData
				// breaking since there's nothing more to process. 
				break;
			}
			else if (Entry != EntityData[EntryIndex + 1])
			{
				continue;
			}

			int32 DuplicateIndex = EntryIndex + 1;
			while (DuplicateIndex + 1 < EntityData.Num() && Entry == EntityData[DuplicateIndex + 1])
			{
				++DuplicateIndex;
			};

			const int32 NumDuplicates = DuplicateIndex - EntryIndex;

			EntityData.RemoveAt(EntryIndex + 1, NumDuplicates, EAllowShrinking::No);
			Payload.SwapElementsToEnd(EntryIndex + 1, NumDuplicates);
			// even though we don't remove the elements from payload we later limit the number of elements used with
			// ArchetypeInfo.Count, so we need to update that
			Archetypes[Entry.ArchetypeIndex].Count -= NumDuplicates;
		}
	}

	int32 ProcessedEntitiesCount = 0;
	// processing from the back since that's how EntityData is sorted - higher-index archetypes come first
	for (int32 ArchetypeIndex = Archetypes.Num() - 1; ArchetypeIndex >= 0; --ArchetypeIndex)
	{
		FArchetypeInfo& ArchetypeInfo = Archetypes[ArchetypeIndex];
		TArrayView<FEntityInArchetype> EntityDataSubset = MakeArrayView(&EntityData[ProcessedEntitiesCount], ArchetypeInfo.Count);
		ensure(EntityDataSubset[0].ArchetypeIndex == ArchetypeIndex);
		ensure(EntityDataSubset.Last().ArchetypeIndex == ArchetypeIndex);
		TStridedView<int32> TrueIndices = MakeStridedView(EntityDataSubset, &FEntityInArchetype::TrueIndex);

		FMassGenericPayloadViewSlice PayloadSubView(Payload, ProcessedEntitiesCount, ArchetypeInfo.Count);

		OutEntityCollections.Add(FMassArchetypeEntityCollectionWithPayload(ArchetypeInfo.Archetype, TrueIndices, MoveTemp(PayloadSubView)));

		ProcessedEntitiesCount += ArchetypeInfo.Count;
	}
}

FMassArchetypeEntityCollectionWithPayload::FMassArchetypeEntityCollectionWithPayload(const FMassArchetypeHandle& InArchetype, TStridedView<const int32> TrueIndices, FMassGenericPayloadViewSlice&& InPayloadSlice)
	: Entities(InArchetype, FMassArchetypeEntityCollection::DoNothing)
	, PayloadSlice(InPayloadSlice)
{
	Entities.BuildEntityRanges(TrueIndices);
}

//-----------------------------------------------------------------------------
// FMassEntityInChunkDataHandle
//-----------------------------------------------------------------------------
bool FMassEntityInChunkDataHandle::IsValid(const FMassArchetypeData* ArchetypeData) const
{
	return ArchetypeData != nullptr
		&& ArchetypeData->IsValidHandle(*this);
}

bool FMassEntityInChunkDataHandle::IsValid(const FMassArchetypeHandle& ArchetypeHandle) const
{
	return IsValid(FMassArchetypeHelper::ArchetypeDataFromHandle(ArchetypeHandle));
}
