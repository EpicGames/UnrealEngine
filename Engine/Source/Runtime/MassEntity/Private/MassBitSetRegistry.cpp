// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassBitSetRegistry.h"

namespace UE::Mass
{
	template<>
	FFragmentBitRegistry::TBitTypeRegistry()
		: StructTracker(FMassFragment::StaticStruct()
			, [](const UStruct* Struct)
				{
					return UE::Mass::IsA<FMassFragment>(Struct);
				})
	{	
	}

	template<>
	FTagBitRegistry::TBitTypeRegistry()
		: StructTracker(FMassTag::StaticStruct()
			, [](const UStruct* Struct)
				{
					return UE::Mass::IsA<FMassTag>(Struct);
				})
	{	
	}
} // namespace UE::Mass
