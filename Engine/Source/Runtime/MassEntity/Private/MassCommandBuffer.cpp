// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassCommandBuffer.h"
#include "Containers/AnsiString.h"
#include "MassEntityManager.h"
#include "MassObserverManager.h"
#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "VisualLogger/VisualLogger.h"


CSV_DEFINE_CATEGORY(MassEntities, true);
CSV_DEFINE_CATEGORY(MassEntitiesCounters, true);
DECLARE_CYCLE_STAT(TEXT("Mass Flush Commands"), STAT_Mass_FlushCommands, STATGROUP_Mass);

namespace UE::Mass::Command
{
	/**
	 * Note that we default to `false` because the correctness of the feature's behavior depends on use cases.
	 * If there are no observers watching fragment removal, everything will be great. If not, enabling the feature
	 * will result in the data removed no longer being available when the removal-observers get triggered upon lock's release
	 */
	bool bLockObserversDuringFlushing = false;
	FAutoConsoleVariableRef CVarLockObserversDuringFlushing(TEXT("mass.commands.LockObserversDuringFlushing"), bLockObserversDuringFlushing
		, TEXT("Controls whether observers will get locked during commands flushing."), ECVF_Default);

#if CSV_PROFILER_STATS
	bool bEnableDetailedStats = false;

	FAutoConsoleVariableRef CVarEnableDetailedCommandStats(TEXT("massentities.EnableCommandDetailedStats"), bEnableDetailedStats,
		TEXT("Set to true create a dedicated stat per type of command."), ECVF_Default);

	/** CSV stat names */
	static FString DefaultBatchedName = TEXT("BatchedCommand");
	static TMap<FName, TPair<FString, FAnsiString>> CommandBatchedFNames;

	/** CSV custom stat names (ANSI) */
	static FAnsiString DefaultANSIBatchedName = "BatchedCommand";

	/**
	 * Provides valid names for CSV profiling.
	 * @param Command is the command instance
	 * @param OutName is the name to use for csv custom stats
	 * @param OutANSIName is the name to use for csv stats
	 */
	void GetCommandStatNames(FMassBatchedCommand& Command, FString*& OutName, FAnsiString*& OutANSIName)
	{
		OutANSIName = &DefaultANSIBatchedName;
		OutName     = &DefaultBatchedName;
		if (!bEnableDetailedStats)
		{
			return;
		}

		const FName CommandFName = Command.GetFName();

		TPair<FString, FAnsiString>& Names = CommandBatchedFNames.FindOrAdd(CommandFName);
		OutName     = &Names.Get<FString>();
		OutANSIName = &Names.Get<FAnsiString>();
		if (OutName->IsEmpty())
		{
			*OutName     = CommandFName.ToString();
			*OutANSIName = **OutName;
		}
	}
#endif
} // namespace UE::Mass::Command

//-----------------------------------------------------------------------------
// FMassBatchedCommand
//-----------------------------------------------------------------------------
std::atomic<uint32> FMassBatchedCommand::CommandsCounter;

//-----------------------------------------------------------------------------
// FMassCommandBuffer
//-----------------------------------------------------------------------------
FMassCommandBuffer::FMassCommandBuffer()
	: OwnerThreadId(FPlatformTLS::GetCurrentThreadId())
{	
}

FMassCommandBuffer::~FMassCommandBuffer()
{
	ensureMsgf(HasPendingCommands() == false, TEXT("Destroying FMassCommandBuffer while there are still unprocessed commands. These operations will never be performed now."));

	CleanUp();
}

void FMassCommandBuffer::ForceUpdateCurrentThreadID()
{
	OwnerThreadId = FPlatformTLS::GetCurrentThreadId();
}

bool FMassCommandBuffer::Flush(FMassEntityManager& EntityManager)
{
	check(!bIsFlushing);
	TGuardValue FlushingGuard(bIsFlushing, true);

	// short-circuit exit
	if (HasPendingCommands() == false)
	{
		return false;
	}

	UE_MT_SCOPED_WRITE_ACCESS(PendingBatchCommandsDetector);
	LLM_SCOPE_BYNAME(TEXT("Mass/FlushCommands"));
	SCOPE_CYCLE_COUNTER(STAT_Mass_FlushCommands);

	// array used to group commands depending on their operations. Based on EMassCommandOperationType
	constexpr int32 CommandTypeOrder[static_cast<uint8>(EMassCommandOperationType::MAX)] =
	{
		MAX_int32 - 1, // None
		0, // Create
		2, // Add
		6, // Remove
		3, // ChangeComposition
		4, // Set
		6, // Destroy
	};

	/**
	 * The following three types of commands are the ones where we cannot guarantee the new behavior
	 * will be consistent with the pre-change behavior.
	 * Before the change, every removal-observer gets notified before the data is actually removed,
	 * which means the observer can access the data-about-to-be-removed. Now, if removal happens while
	 * an observer lock is active, then the removal-observers will get notified after the fact.
	 * For now we're going to support the old behavior. 
	 */
	constexpr int32 CommandTypeGroupToReleaseObserverLock = FMath::Min3(
		CommandTypeOrder[static_cast<uint8>(EMassCommandOperationType::Remove)]
		, CommandTypeOrder[static_cast<uint8>(EMassCommandOperationType::ChangeComposition)]
		, CommandTypeOrder[static_cast<uint8>(EMassCommandOperationType::Destroy)]
	);

	struct FBatchedCommandsSortedIndex
	{
		FBatchedCommandsSortedIndex(const int32 InIndex, const int32 InGroupOrder)
			: Index(InIndex), GroupOrder(InGroupOrder)
		{
		}

		const int32 Index = -1;
		const int32 GroupOrder = MAX_int32;
		bool IsValid() const
		{
			return GroupOrder < MAX_int32;
		}
		bool operator<(const FBatchedCommandsSortedIndex& Other) const
		{
			return GroupOrder < Other.GroupOrder;
		}
	};
		
	TArray<FBatchedCommandsSortedIndex> CommandsOrder;
	const int32 OwnedCommandsCount = CommandInstances.Num();

	CommandsOrder.Reserve(OwnedCommandsCount);
	for (int32 i = 0; i < OwnedCommandsCount; ++i)
	{
		const TUniquePtr<FMassBatchedCommand>& Command = CommandInstances[i];
		CommandsOrder.Add(FBatchedCommandsSortedIndex(i, (Command && Command->HasWork())? CommandTypeOrder[static_cast<int32>(Command->GetOperationType())] : MAX_int32));
	}
	for (int32 i = 0; i < AppendedCommandInstances.Num(); ++i)
	{
		const TUniquePtr<FMassBatchedCommand>& Command = AppendedCommandInstances[i];
		CommandsOrder.Add(FBatchedCommandsSortedIndex(i + OwnedCommandsCount, (Command && Command->HasWork()) ? CommandTypeOrder[static_cast<int32>(Command->GetOperationType())] : MAX_int32));
	}
	CommandsOrder.StableSort();

	TSharedPtr<FMassObserverManager::FObserverLock> ObserverLock;
	TSharedPtr<FMassObserverManager::FCreationContext> CreationLock;
	if (UE::Mass::Command::bLockObserversDuringFlushing 
		&& CommandsOrder[0].GroupOrder < CommandTypeGroupToReleaseObserverLock)
	{
		ObserverLock = EntityManager.GetOrMakeObserversLock();
		//  we only want to create CreationLock if the very first command is of `Create` type.
		if (CommandsOrder[0].GroupOrder == CommandTypeOrder[static_cast<uint8>(EMassCommandOperationType::Create)])
		{
			CreationLock = EntityManager.GetOrMakeCreationContext();
		}
	}
	bool bObserversLock = ObserverLock.IsValid();
	bool bCreationLock = CreationLock.IsValid();

	for (int32 k = 0; k < CommandsOrder.Num() && CommandsOrder[k].IsValid(); ++k)
	{
		if (bCreationLock && CommandsOrder[k].GroupOrder > 0)
		{
			bCreationLock = false;
			CreationLock.Reset();
		}
		if (bObserversLock && CommandsOrder[k].GroupOrder >= CommandTypeGroupToReleaseObserverLock)
		{
			bObserversLock = false;
			ObserverLock.Reset();
		}

		const int32 CommandIndex = CommandsOrder[k].Index;
		TUniquePtr<FMassBatchedCommand>& Command = CommandIndex < OwnedCommandsCount
			? CommandInstances[CommandIndex]
			: AppendedCommandInstances[CommandIndex - OwnedCommandsCount];
		check(Command)

#if CSV_PROFILER_STATS
		using namespace UE::Mass::Command;

		// Extract name (default or detailed)
		FAnsiString* ANSIName = nullptr;
		FString*     Name     = nullptr;
		GetCommandStatNames(*Command, Name, ANSIName);

		// Push stats
		FScopedCsvStat ScopedCsvStat(**ANSIName, CSV_CATEGORY_INDEX(MassEntities));
		FCsvProfiler::RecordCustomStat(**Name, CSV_CATEGORY_INDEX(MassEntitiesCounters), Command->GetNumOperationsStat(), ECsvCustomStatOp::Accumulate);
#endif // CSV_PROFILER_STATS

		Command->Run(EntityManager);
		Command->Reset();
	}

	AppendedCommandInstances.Reset();

	ActiveCommandsCounter = 0;

	return true;
}
 
void FMassCommandBuffer::CleanUp()
{
	CommandInstances.Reset();
	AppendedCommandInstances.Reset();

	ActiveCommandsCounter = 0;
}

void FMassCommandBuffer::MoveAppend(FMassCommandBuffer& Other)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandBuffer_MoveAppend);

	// @todo optimize, there's surely a way to do this faster than this.
	UE_MT_SCOPED_READ_ACCESS(Other.PendingBatchCommandsDetector);
	if (Other.HasPendingCommands())
	{
		UE::TScopeLock Lock(AppendingCommandsCS);
		UE_MT_SCOPED_WRITE_ACCESS(PendingBatchCommandsDetector);
		AppendedCommandInstances.Append(MoveTemp(Other.CommandInstances));
		AppendedCommandInstances.Append(MoveTemp(Other.AppendedCommandInstances));
		ActiveCommandsCounter += Other.ActiveCommandsCounter;
		Other.ActiveCommandsCounter = 0;
	}
}

SIZE_T FMassCommandBuffer::GetAllocatedSize() const
{
	SIZE_T TotalSize = 0;
	for (const TUniquePtr<FMassBatchedCommand>& Command : CommandInstances)
	{
		TotalSize += Command ? Command->GetAllocatedSize() : 0;
	}
	for (const TUniquePtr<FMassBatchedCommand>& Command : AppendedCommandInstances)
	{
		TotalSize += Command ? Command->GetAllocatedSize() : 0;
	}

	TotalSize += CommandInstances.GetAllocatedSize();
	
	return TotalSize;
}

