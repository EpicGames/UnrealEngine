// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDebugger.h"
#include "MassExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassDebugger)
#if WITH_MASSENTITY_DEBUG
#include "MassProcessor.h"
#include "MassEntityManager.h"
#include "MassEntityManagerStorage.h"
#include "MassEntitySubsystem.h"
#include "MassArchetypeTypes.h"
#include "MassArchetypeData.h"
#include "MassRequirements.h"
#include "MassEntityQuery.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "MassEntityUtils.h"
#include "MassCommandBuffer.h"
#include "MassEntityTrace.h"
#include "Misc/StringOutputDevice.h"
#include "Misc/MessageDialog.h"
#include "HAL/PlatformMisc.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "CoreGlobals.h"

#define LOCTEXT_NAMESPACE "MassDebugger"

namespace UE::Mass::Debug
{
	enum class EServerPIEDebugDrawMode : uint8
	{
		Default = 0,			// Attempt to draw server in server world, overal debug drawing unaffected
		ReplaceClient = 1,		// Draw server in client world, replace client debug drawing
		AppendToClient = 2,		// Draw server in client world, add to client debug drawing
	};

	bool bAllowProceduralDebuggedEntitySelection = false;
	bool bAllowBreakOnDebuggedEntity = false;
	bool bTestSelectedEntityAgainstProcessorQueries = true;
	int32 ServerPIEDebugDrawMode = static_cast<int32>(EServerPIEDebugDrawMode::Default);

	FAutoConsoleVariableRef CVars[] =
	{
		{ TEXT("mass.debug.AllowProceduralDebuggedEntitySelection"), bAllowProceduralDebuggedEntitySelection
			, TEXT("Guards whether MASS_SET_ENTITY_DEBUGGED calls take effect."), ECVF_Cheat}
		, {TEXT("mass.debug.AllowBreakOnDebuggedEntity"), bAllowBreakOnDebuggedEntity
			, TEXT("Guards whether MASS_BREAK_IF_ENTITY_DEBUGGED calls take effect."), ECVF_Cheat}
		, {	TEXT("mass.debug.TestSelectedEntityAgainstProcessorQueries"), bTestSelectedEntityAgainstProcessorQueries
			, TEXT("Enabling will result in testing all processors' queries against SelectedEntity (as indicated by")
			TEXT("mass.debug.DebugEntity or the gameplay debugger) and storing potential failure results to be viewed in MassDebugger")
			, ECVF_Cheat }
		, {	TEXT("mass.debug.ServerPIEDebugDrawMode"), ServerPIEDebugDrawMode
			, TEXT("If enabled will (where supported) draw server's debug drawing in the PIE's client view\n"
			" 0 - default, debug drawing unaffected\n"
			" 1 - will draw only server information\n"
			" 2 - will draw both client and server\n"
			"Non-networked modes are not affected")
			, ECVF_Cheat }
	};

	struct FNoDebuggerNotification
	{
		static void ConditionallyNotifyUser()
		{
			const bool bLocalDebuggerPresent = FPlatformMisc::IsDebuggerPresent();
			if (bDebuggerPresent != bLocalDebuggerPresent)
			{
				bDebuggerPresent = bLocalDebuggerPresent;
				if (bLocalDebuggerPresent == false)
				{
					if (bUserNotified == false)
					{
						FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoDebuggerAttached", "Breakpoint set but no debugger is attached."));
						bUserNotified = true;
					}
				}
				else
				{
					bUserNotified = false;
				}
			}
		}
		static bool bDebuggerPresent;
		static bool bUserNotified;
	};
	bool FNoDebuggerNotification::bDebuggerPresent = false;
	bool FNoDebuggerNotification::bUserNotified = false;
	

	FString DebugGetFragmentAccessString(EMassFragmentAccess Access)
	{
		switch (Access)
		{
		case EMassFragmentAccess::None:	return TEXT("--");
		case EMassFragmentAccess::ReadOnly:	return TEXT("RO");
		case EMassFragmentAccess::ReadWrite:	return TEXT("RW");
		default:
			ensureMsgf(false, TEXT("Missing string conversion for EMassFragmentAccess=%d"), EnumToUnderlyingType(Access));
			break;
		}
		return TEXT("Missing string conversion");
	}

	void DebugOutputDescription(TConstArrayView<UMassProcessor*> Processors, FOutputDevice& Ar)
	{
		const bool bAutoLineEnd = Ar.GetAutoEmitLineTerminator();
		Ar.SetAutoEmitLineTerminator(false);
		for (const UMassProcessor* Proc : Processors)
		{
			if (Proc)
			{
				Proc->DebugOutputDescription(Ar);
				Ar.Logf(TEXT("\n"));
			}
			else
			{
				Ar.Logf(TEXT("NULL\n"));
			}
		}
		Ar.SetAutoEmitLineTerminator(bAutoLineEnd);
	}

	// First Id of a range of lightweight entity for which we want to activate debug information
	int32 DebugEntityBegin = INDEX_NONE;

	// Last Id of a range of lightweight entity for which we want to activate debug information
	int32 DebugEntityEnd = INDEX_NONE;

	void SetDebugEntityRange(const int32 InDebugEntityBegin, const int32 InDebugEntityEnd)
	{
		DebugEntityBegin = InDebugEntityBegin;
		DebugEntityEnd = InDebugEntityEnd;
	}

	static FAutoConsoleCommand SetDebugEntityRangeCommand(
		TEXT("mass.debug.SetDebugEntityRange"),
		TEXT("Range of lightweight entity IDs that we want to debug.")
		TEXT("Usage: \"mass.debug.SetDebugEntityRange <FirstEntity> <LastEntity>\""),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				if (Args.Num() != 2)
				{
					UE_LOGF(LogConsoleResponse, Display, "Error: Expecting 2 parameters");
					return;
				}

				int32 FirstID = INDEX_NONE;
				int32 LastID = INDEX_NONE;
				if (!LexTryParseString<int32>(FirstID, *Args[0]))
				{
					UE_LOGF(LogConsoleResponse, Display, "Error: first parameter must be an integer");
					return;
				}
			
				if (!LexTryParseString<int32>(LastID, *Args[1]))
				{
					UE_LOGF(LogConsoleResponse, Display, "Error: second parameter must be an integer");
					return;
				}

				SetDebugEntityRange(FirstID, LastID);
			}));

	static FAutoConsoleCommand ResetDebugEntity(
		TEXT("mass.debug.ResetDebugEntity"),
		TEXT("Disables lightweight entities debugging.")
		TEXT("Usage: \"mass.debug.ResetDebugEntity\""),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				SetDebugEntityRange(INDEX_NONE, INDEX_NONE);
			}));

	bool HasDebugEntities()
	{
		return DebugEntityBegin != INDEX_NONE && DebugEntityEnd != INDEX_NONE;
	}

	bool IsDebuggingSingleEntity()
	{
		return DebugEntityBegin != INDEX_NONE && DebugEntityBegin == DebugEntityEnd;
	}

	bool GetDebugEntitiesRange(int32& OutBegin, int32& OutEnd)
	{
		OutBegin = DebugEntityBegin;
		OutEnd = DebugEntityEnd;
		return DebugEntityBegin != INDEX_NONE && DebugEntityEnd != INDEX_NONE && DebugEntityBegin <= DebugEntityEnd;
	}
	
	bool IsDebuggingEntity(FMassEntityHandle Entity, FColor* OutEntityColor)
	{
		const int32 EntityIdx = Entity.Index;
		const bool bIsDebuggingEntity = (DebugEntityBegin != INDEX_NONE && DebugEntityEnd != INDEX_NONE && DebugEntityBegin <= EntityIdx && EntityIdx <= DebugEntityEnd);
	
		if (bIsDebuggingEntity && OutEntityColor != nullptr)
		{
			*OutEntityColor = GetEntityDebugColor(Entity);
		}

		return bIsDebuggingEntity;
	}

	FColor GetEntityDebugColor(FMassEntityHandle Entity)
	{
		const int32 EntityIdx = Entity.Index;
		return EntityIdx != INDEX_NONE ? GColorList.GetFColorByIndex(EntityIdx % GColorList.GetColorsNum()) : FColor::Black;
	}

	UWorld* GetDebugDrawWorld(const FMassExecutionContext& Context)
	{
		UWorld* World = Context.GetWorld();
#if WITH_EDITOR
		if (GIsEditor && World)
		{
			//                    | Default  | ReplaceClient | AppendToClient |
			// NM_Standalone      | Allow    | Allow            | Allow           |
			// NM_DedicatedServer | Suppress | Redirect         | Redirect        |
			// NM_ListenServer    | Allow    | Redirect         | Redirect        |
			// NM_Client          | Allow    | Suppress         | Allow           |

			enum class EDrawStatus
			{
				Allow,
				Suppress,
				Redirect,
			};

			const ENetMode NetMode = World->GetNetMode();
			EServerPIEDebugDrawMode DrawMode = static_cast<EServerPIEDebugDrawMode>(FMath::Clamp(ServerPIEDebugDrawMode, 0, 2));

			EDrawStatus DrawStatus;
			switch (NetMode)
			{
				default: // falling though on purpose
				case NM_Standalone:
					DrawStatus = EDrawStatus::Allow;
					break;

				case NM_DedicatedServer:
					DrawStatus = (DrawMode == EServerPIEDebugDrawMode::Default)
						? EDrawStatus::Suppress
						: EDrawStatus::Redirect;
					break;

				case NM_ListenServer:
					DrawStatus = (DrawMode == EServerPIEDebugDrawMode::Default)
						? EDrawStatus::Allow
						: EDrawStatus::Redirect;
					break;

				case NM_Client:
					DrawStatus = (DrawMode == EServerPIEDebugDrawMode::ReplaceClient)
						? EDrawStatus::Suppress
						: EDrawStatus::Allow;
					break;
			}

			switch (DrawStatus)
			{
				case EDrawStatus::Allow:
					break;

				case EDrawStatus::Suppress:
					World = nullptr;
					break;

				case EDrawStatus::Redirect:
					for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
					{
						UWorld* LocalWorld = WorldContext.World();
						if (LocalWorld && LocalWorld->GetNetMode() == NM_Client)
						{
							World = LocalWorld;
							break;
						}
					}
					break;
			}
		}
#endif // WITH_EDITOR
		return World;
	}

	FAutoConsoleCommandWithWorldArgsAndOutputDevice PrintEntityFragmentsCmd(
		TEXT("mass.PrintEntityFragments"),
		TEXT("Prints all fragment types and values (uproperties) for the specified Entity index"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda(
			[](const TArray<FString>& Params, UWorld* World, FOutputDevice& Ar)
			{
				check(World);
				if (UMassEntitySubsystem* EntityManager = World->GetSubsystem<UMassEntitySubsystem>())
				{
					int32 Index = INDEX_NONE;
					if (LexTryParseString<int32>(Index, *Params[0]))
					{
						FMassDebugger::OutputEntityDescription(Ar, EntityManager->GetEntityManager(), Index);
					}
					else
					{
						Ar.Logf(ELogVerbosity::Error, TEXT("Entity index parameter must be an integer"));
					}
				}
				else
				{
					Ar.Logf(ELogVerbosity::Error, TEXT("Failed to find MassEntitySubsystem for world %s"), *GetPathNameSafe(World));
				}
			})
	);

	FAutoConsoleCommandWithWorldArgsAndOutputDevice LogArchetypesCmd(
		TEXT("mass.LogArchetypes"),
		TEXT("Dumps description of archetypes to log. Optional parameter controls whether to include or exclude non-occupied archetypes. Defaults to 'include'."),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Params, UWorld*, FOutputDevice& Ar)
			{
				const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
				for (const FWorldContext& Context : WorldContexts)
				{
					UWorld* World = Context.World();
					if (World == nullptr || World->IsPreviewWorld())
					{
						continue;
					}

					Ar.Logf(ELogVerbosity::Log, TEXT("Dumping description of archetypes for world: %s (%s - %s)"),
						*GetPathNameSafe(World),
						LexToString(World->WorldType),
						*ToString(World->GetNetMode()));

					if (UMassEntitySubsystem* EntityManager = World->GetSubsystem<UMassEntitySubsystem>())
					{
						bool bIncludeEmpty = true;
						if (Params.Num())
						{
							LexTryParseString(bIncludeEmpty, *Params[0]);
						}
						Ar.Logf(ELogVerbosity::Log, TEXT("Include empty archetypes: %s"), bIncludeEmpty ? TEXT("TRUE") : TEXT("FALSE"));
						EntityManager->GetEntityManager().DebugGetArchetypesStringDetails(Ar, bIncludeEmpty);
					}
					else
					{
						Ar.Logf(ELogVerbosity::Error, TEXT("Failed to find MassEntitySubsystem for world: %s (%s - %s)"),
							*GetPathNameSafe(World),
							LexToString(World->WorldType),
							*ToString(World->GetNetMode()));
					}
				}
			})
	);

	// @todo these console commands will be reparented to "massentities" domain once we rename and shuffle the modules around 
	FAutoConsoleCommandWithWorld RecacheQueries(
		TEXT("mass.RecacheQueries"),
		TEXT("Forces EntityQueries to recache their valid archetypes"),
		FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld)
			{
				check(InWorld);
				if (UMassEntitySubsystem* System = InWorld->GetSubsystem<UMassEntitySubsystem>())
				{
					System->GetMutableEntityManager().DebugForceArchetypeDataVersionBump();
				}
			}
	));

	FAutoConsoleCommandWithWorldArgsAndOutputDevice LogFragmentSizes(
		TEXT("mass.LogFragmentSizes"),
		TEXT("Logs all the fragment types being used along with their sizes."),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Params, UWorld* World, FOutputDevice& Ar)
			{
				for (const TWeakObjectPtr<const UScriptStruct>& WeakStruct : FMassFragmentBitSet::DebugGetAllStructTypes())
				{
					if (const UScriptStruct* StructType = WeakStruct.Get())
					{
						Ar.Logf(ELogVerbosity::Log, TEXT("%s, size: %d"), *StructType->GetName(), StructType->GetStructureSize());
					}
				}
			})
	);

	FAutoConsoleCommandWithWorldArgsAndOutputDevice LogMemoryUsage(
		TEXT("mass.LogMemoryUsage"),
		TEXT("Logs how much memory the mass entity system uses"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Params, UWorld* World, FOutputDevice& Ar)
			{
				check(World);
				if (UMassEntitySubsystem* System = World->GetSubsystem<UMassEntitySubsystem>())
				{
					FResourceSizeEx CumulativeResourceSize;
					System->GetResourceSizeEx(CumulativeResourceSize);
					Ar.Logf(ELogVerbosity::Log, TEXT("MassEntity system uses: %d bytes"), CumulativeResourceSize.GetDedicatedSystemMemoryBytes());
				}
			}));

	FAutoConsoleCommandWithOutputDevice LogFragments(
		TEXT("mass.LogKnownFragments"),
		TEXT("Logs all the known tags and fragments along with their \"index\" as stored via bitsets."),
		FConsoleCommandWithOutputDeviceDelegate::CreateStatic([](FOutputDevice& OutputDevice)
			{
				auto PrintKnownTypes = [&OutputDevice](TConstArrayView<TWeakObjectPtr<const UScriptStruct>> AllStructs) {
					int i = 0;
					for (TWeakObjectPtr<const UScriptStruct> Struct : AllStructs)
					{
						if (Struct.IsValid())
						{
							OutputDevice.Logf(TEXT("\t%d. %s"), i++, *Struct->GetName());
						}
					}
				};

				OutputDevice.Logf(TEXT("Known tags:"));
				PrintKnownTypes(FMassTagBitSet::DebugGetAllStructTypes());

				OutputDevice.Logf(TEXT("Known Fragments:"));
				PrintKnownTypes(FMassFragmentBitSet::DebugGetAllStructTypes());

				OutputDevice.Logf(TEXT("Known Shared Fragments:"));
				PrintKnownTypes(FMassSharedFragmentBitSet::DebugGetAllStructTypes());

				OutputDevice.Logf(TEXT("Known Chunk Fragments:"));
				PrintKnownTypes(FMassChunkFragmentBitSet::DebugGetAllStructTypes());
			}));

	static FAutoConsoleCommandWithWorldAndArgs DestroyEntity(
		TEXT("mass.debug.DestroyEntity"),
		TEXT("ID of a Mass entity that we want to destroy.")
		TEXT("Usage: \"mass.debug.DestoryEntity <Entity>\""),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		if (Args.Num() != 1)
		{
			UE_LOGF(LogConsoleResponse, Display, "Error: Expecting 1 parameter");
			return;
		}

		int32 ID = INDEX_NONE;
		if (!LexTryParseString<int32>(ID, *Args[0]))
		{
			UE_LOGF(LogConsoleResponse, Display, "Error: parameter must be an integer");
			return;
		}

		if (!World)
		{
			UE_LOGF(LogConsoleResponse, Display, "Error: invalid world");
			return;
		}

		FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(*World);
		FMassEntityHandle EntityToDestroy = EntityManager.CreateEntityIndexHandle(ID);
		if (!EntityToDestroy.IsSet())
		{
			UE_LOGF(LogConsoleResponse, Display, "Error: cannot find entity for this index");
			return;
		}

		EntityManager.Defer().DestroyEntity(EntityToDestroy);
	}));

	static FAutoConsoleCommandWithWorldAndArgs SetDebugEntity(
		TEXT("mass.debug.DebugEntity"),
		TEXT("ID of a Mass entity that we want to debug.")
		TEXT("Note that this call results in the same behavior as if the entity was picked via the Mass GameplayDebugger's category.")
		TEXT("Usage: \"mass.debug.DebugEntity <Entity>\""),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
		{
			if (!World)
			{
				UE_LOGF(LogConsoleResponse, Display, "Error: invalid world");
				return;
			}

			int32 ID = INDEX_NONE;
			if (Args.Num() > 0)
			{
				LexTryParseString<int32>(ID, *Args[0]);
			}

			SetDebugEntityRange(ID, ID);

			FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(*World);
			FMassEntityHandle EntityToDebug = EntityManager.CreateEntityIndexHandle(ID);
			if (!EntityToDebug.IsSet() && ID != INDEX_NONE)
			{
				UE_LOGF(LogConsoleResponse, Display, "Cannot find entity for this index, clearing current selection");
				return;
			}

			FMassDebugger::SelectEntity(EntityManager, EntityToDebug);
		}
	));

	const UScriptStruct* FindElementTypeByName(const FString& PartialFragmentName)
	{
		const UScriptStruct* Result = nullptr;
#if WITH_STRUCTUTILS_DEBUG
		Result = FMassFragmentBitSet::DebugFindTypeByPartialName(PartialFragmentName);
		if (Result == nullptr)
		{
			Result = FMassSharedFragmentBitSet::DebugFindTypeByPartialName(PartialFragmentName);
		}
		if (Result == nullptr)
		{
			Result = FMassConstSharedFragmentBitSet::DebugFindTypeByPartialName(PartialFragmentName);
		}
#endif // WITH_STRUCTUTILS_DEBUG
		return Result;
	}

	static FAutoConsoleCommandWithWorldAndArgs SetFragmentBreakpoint(
		TEXT("mass.debug.SetFragmentBreakpoint"),
		TEXT("Enables fragment write break-point on an arbitrary number of fragment types, on the selected entity (see `mass.debug.DebugEntity`).")
		TEXT("Usage: `mass.debug.SetFragmentBreakpoint <FragmentTypeName> <FragmentTypeName2> <FragmentTypeName3> <...>`"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
		{
			if (!World)
			{
				UE_LOGF(LogConsoleResponse, Display, "Error: invalid world");
				return;
			}

			if (Args.Num() == 0)
			{
				UE_LOGF(LogConsoleResponse, Display, "No fragment types indicated");
			}
			else
			{
				FMassEntityManager& EntityManager = Utils::GetEntityManagerChecked(*World);
				FMassEntityHandle SelectedEntity = FMassDebugger::GetSelectedEntity(EntityManager);
				if (SelectedEntity.IsValid())
				{
					for (const FString& PartialFragmentName : Args)
					{
						if (const UScriptStruct* FragmentType = FindElementTypeByName(PartialFragmentName))
						{
							FMassDebugger::SetFragmentWriteBreakpoint(EntityManager, FragmentType, SelectedEntity);
						}
						else
						{
							UE_LOGF(LogConsoleResponse, Display, "Warning: Unable to find element type %ls", *PartialFragmentName);
						}
					}
				}
				else
				{
					UE_LOGF(LogConsoleResponse, Display, "Warning: No entity selected, no break points set");
				}
			}
		}
	));

	static FAutoConsoleCommandWithWorldAndArgs ClearFragmentBreakpoint(
		TEXT("mass.debug.ClearFragmentBreakpoint"),
		TEXT("Clears fragment write break-point on an arbitrary number of fragment types, on the selected entity (see `mass.debug.DebugEntity`).")
		TEXT("If no entity is currently selected then the call will clear the type breakpoints on all entities.")
		TEXT("Usage: `mass.debug.ClearFragmentBreakpoint <FragmentTypeName> <FragmentTypeName2> <FragmentTypeName3> <...>`"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
		{
			if (!World)
			{
				UE_LOGF(LogConsoleResponse, Display, "Error: invalid world");
				return;
			}

			if (Args.Num() == 0)
			{
				UE_LOGF(LogConsoleResponse, Display, "No fragment types indicated");
			}
			else
			{
				FMassEntityManager& EntityManager = Utils::GetEntityManagerChecked(*World);
				FMassEntityHandle SelectedEntity = FMassDebugger::GetSelectedEntity(EntityManager);
				const bool bEntityValid = SelectedEntity.IsValid();

				for (const FString& PartialFragmentName : Args)
				{
					if (const UScriptStruct* FragmentType = FindElementTypeByName(PartialFragmentName))
					{
						bEntityValid
							? FMassDebugger::ClearFragmentWriteBreak(EntityManager, FragmentType, SelectedEntity)
							: FMassDebugger::ClearFragmentWriteBreak(EntityManager, FragmentType, FMassEntityHandle());
					}
					else
					{
						UE_LOGF(LogConsoleResponse, Display, "Warning: Unable to find element type %ls", *PartialFragmentName);
					}
				}
			}
		}
	));

} // namespace UE::Mass::Debug

//----------------------------------------------------------------------//
// FMassDebugger
//----------------------------------------------------------------------//
FMassDebugger::FOnBreakpointsChanged FMassDebugger::OnBreakpointsChangedDelegate;
FMassDebugger::FOnEntitySelected FMassDebugger::OnEntitySelectedDelegate;

FMassDebugger::FOnMassEntityManagerEvent FMassDebugger::OnEntityManagerInitialized;
FMassDebugger::FOnMassEntityManagerEvent FMassDebugger::OnEntityManagerDeinitialized;
FMassDebugger::FOnEnvironmentEvent FMassDebugger::OnProcessorProviderRegistered;
FMassDebugger::FOnDebugEvent FMassDebugger::OnDebugEvent;
TArray<FMassDebugger::FEnvironment> FMassDebugger::ActiveEnvironments;
UE::FTransactionallySafeMutex FMassDebugger::EntityManagerRegistrationLock;
TMap<FName, const UScriptStruct*> FMassDebugger::FragmentsByName;

TConstArrayView<FMassEntityQuery*> FMassDebugger::GetProcessorQueries(const UMassProcessor& Processor)
{
	return Processor.OwnedQueries;
}

TConstArrayView<FMassEntityQuery*> FMassDebugger::GetUpToDateProcessorQueries(const FMassEntityManager& EntityManager, UMassProcessor& Processor)
{
	for (FMassEntityQuery* Query : Processor.OwnedQueries)
	{
		if (Query)
		{
			Query->CacheArchetypes();
		}
	}

	return Processor.OwnedQueries;
}

UE::Mass::Debug::FQueryRequirementsView FMassDebugger::GetQueryRequirements(const FMassEntityQuery& Query)
{
	UE::Mass::Debug::FQueryRequirementsView View = { Query.FragmentRequirements, Query.ChunkFragmentRequirements, Query.ConstSharedFragmentRequirements, Query.SharedFragmentRequirements
		, Query.RequiredAllTags, Query.RequiredAnyTags, Query.RequiredNoneTags, Query.RequiredOptionalTags
		, Query.RequiredConstSubsystems, Query.RequiredMutableSubsystems };

	return View;
}

void FMassDebugger::GetQueryExecutionRequirements(const FMassEntityQuery& Query, FMassExecutionRequirements& OutExecutionRequirements)
{
	Query.ExportRequirements(OutExecutionRequirements);
}

TArray<FMassEntityHandle> FMassDebugger::GetEntitiesMatchingQuery(const FMassEntityManager& EntityManager, const FMassEntityQuery& Query)
{
	TArray<FMassEntityHandle> Entities;
	TArray<FMassArchetypeHandle> Archetypes;
	EntityManager.GetMatchingArchetypes(Query, Archetypes, 0);
	for (FMassArchetypeHandle& ArchHandle : Archetypes)
	{
		Entities.Append(GetEntitiesOfArchetype(ArchHandle));
	}
	return Entities;
}

void FMassDebugger::ForEachArchetype(const FMassEntityManager& EntityManager, const UE::Mass::Debug::FArchetypeFunction& Function)
{
	for (auto& KVP : EntityManager.FragmentHashToArchetypeMap)
	{
		for (const TSharedPtr<FMassArchetypeData>& Archetype : KVP.Value)
		{
			Function(FMassArchetypeHelper::ArchetypeHandleFromData(Archetype));
		}
	}
}

TArray<FMassArchetypeHandle> FMassDebugger::GetAllArchetypes(const FMassEntityManager& EntityManager)
{
	TArray<FMassArchetypeHandle> Archetypes;

	for (auto& KVP : EntityManager.FragmentHashToArchetypeMap)
	{
		for (const TSharedPtr<FMassArchetypeData>& Archetype : KVP.Value)
		{
			Archetypes.Add(FMassArchetypeHelper::ArchetypeHandleFromData(Archetype));
		}
	}

	return Archetypes;
}

FMassArchetypeCompositionDescriptor FMassDebugger::GetArchetypeComposition(const FMassArchetypeHandle& ArchetypeHandle)
{
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	return FMassArchetypeCompositionDescriptor(ArchetypeData.GetCompositionBitSet());
}

FMassElementBitSet FMassDebugger::GetArchetypeSparseElementsBitset(const FMassArchetypeHandle& ArchetypeHandle)
{
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	return ArchetypeData.GetSparseElementsBitSet();
}

uint64 FMassDebugger::GetArchetypeTraceID(const FMassArchetypeData& ArchetypeData)
{
	return reinterpret_cast<uint64>(&ArchetypeData);
}

uint64 FMassDebugger::GetArchetypeTraceID(const FMassArchetypeHandle& ArchetypeHandle)
{ 
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	return GetArchetypeTraceID(ArchetypeData);
}

TConstArrayView<FMassEntityHandle> FMassDebugger::GetEntitiesViewOfArchetype(const FMassArchetypeData& ArchetypeData, const FMassArchetypeChunk& Chunk)
{
	FMassArchetypeChunk& MutableChunk = const_cast<FMassArchetypeChunk&>(Chunk);
	TConstArrayView<FMassEntityHandle> View(&MutableChunk.GetEntityArrayElementRef(ArchetypeData.EntityListOffsetWithinChunk, 0), Chunk.GetNumInstances());
	return View;
}

const FMassArchetypeData* FMassDebugger::GetArchetypeData(const FMassArchetypeHandle& ArchetypeHandle)
{
	FMassArchetypeData* ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandle(ArchetypeHandle);
	return ArchetypeData;
}

void FMassDebugger::EnumerateChunks(const FMassArchetypeData& Archetype, TFunctionRef<void(const FMassArchetypeChunk&)> Fn)
{
	for (const FMassArchetypeChunk& Chunk : Archetype.Chunks)
	{
		Fn(Chunk);
	}
}

void FMassDebugger::GetArchetypeEntityStats(const FMassArchetypeHandle& ArchetypeHandle, UE::Mass::Debug::FArchetypeStats& OutStats)
{
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	OutStats.EntitiesCount = ArchetypeData.GetNumEntities();
	OutStats.EntitiesCountPerChunk = ArchetypeData.GetNumEntitiesPerChunk();
	OutStats.ChunksCount = ArchetypeData.GetChunkCount();
	OutStats.AllocatedSize = ArchetypeData.GetAllocatedSize();
	OutStats.BytesPerEntity = ArchetypeData.GetBytesPerEntity();
	OutStats.SparseElementsBookkeepingSize = ArchetypeData.GetSparseElementsBookkeepingAllocatedSize();

	SIZE_T ActiveChunksMemorySize = 0;
	SIZE_T ActiveEntitiesMemorySize = 0;
	ArchetypeData.DebugGetEntityMemoryNumbers(ActiveChunksMemorySize, ActiveEntitiesMemorySize);
	OutStats.WastedEntityMemory = ActiveChunksMemorySize - ActiveEntitiesMemorySize;
}

const TConstArrayView<FName> FMassDebugger::GetArchetypeDebugNames(const FMassArchetypeHandle& ArchetypeHandle)
{
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	return ArchetypeData.GetDebugNames();
}

TArray<FMassEntityHandle> FMassDebugger::GetEntitiesOfArchetype(const FMassArchetypeHandle& ArchetypeHandle)
{
	TArray<FMassEntityHandle> EntitiesOfArchetype;
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	EntitiesOfArchetype.Reserve(ArchetypeData.GetNumEntities());
	for (FMassArchetypeChunk& Chunk : ArchetypeData.Chunks)
	{
		TArrayView<FMassEntityHandle> EntityListView = TArrayView<FMassEntityHandle>(&Chunk.GetEntityArrayElementRef(ArchetypeData.EntityListOffsetWithinChunk, 0), Chunk.GetNumInstances());
		EntitiesOfArchetype.Append(EntityListView);
	}
	return EntitiesOfArchetype;
}

TConstArrayView<UMassCompositeProcessor::FDependencyNode> FMassDebugger::GetProcessingGraph(const UMassCompositeProcessor& GraphOwner)
{
	return GraphOwner.FlatProcessingGraph;
}

TConstArrayView<TObjectPtr<UMassProcessor>> FMassDebugger::GetHostedProcessors(const UMassCompositeProcessor& GraphOwner)
{
	return GraphOwner.ChildPipeline.GetProcessors();
}

FString FMassDebugger::GetRequirementsDescription(const FMassFragmentRequirements& Requirements)
{
	TStringBuilder<256> StringBuilder;
	StringBuilder.Append(TEXT("<"));

	bool bNeedsComma = false;
	for (const FMassFragmentRequirementDescription& Requirement : Requirements.FragmentRequirements)
	{
		if (bNeedsComma)
		{
			StringBuilder.Append(TEXT(","));
		}
		StringBuilder.Append(*FMassDebugger::GetSingleRequirementDescription(Requirement));
		bNeedsComma = true;
	}

	StringBuilder.Append(TEXT(">"));
	return StringBuilder.ToString();
}

FString FMassDebugger::GetArchetypeRequirementCompatibilityDescription(const FMassFragmentRequirements& Requirements, const FMassArchetypeHandle& ArchetypeHandle)
{
	if (ArchetypeHandle.IsValid() == false)
	{
		return TEXT("Invalid");
	}

	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	return FMassDebugger::GetArchetypeRequirementCompatibilityDescription(Requirements, ArchetypeData.GetCompositionBitSet());
}
	
FString FMassDebugger::GetArchetypeRequirementCompatibilityDescription(const FMassFragmentRequirements& Requirements, const FMassElementBitSet& ArchetypeCompositionBitSet)
{
	FStringOutputDevice OutDescription;

	FMassFragmentBitSet FragmentBitSet = ArchetypeCompositionBitSet.Get<FMassFragmentBitSet>();
	FMassTagBitSet TagBitSet = ArchetypeCompositionBitSet.Get<FMassTagBitSet>();
	FMassChunkFragmentBitSet ChunkFragmentBitSet = ArchetypeCompositionBitSet.Get<FMassChunkFragmentBitSet>();
	FMassSharedFragmentBitSet SharedFragmentBitSet = ArchetypeCompositionBitSet.Get<FMassSharedFragmentBitSet>();
	FMassConstSharedFragmentBitSet ConstSharedFragmentBitSet = ArchetypeCompositionBitSet.Get<FMassConstSharedFragmentBitSet>();

	if (Requirements.HasNegativeRequirements())
	{
		if (FragmentBitSet.HasNone(Requirements.RequiredNoneFragments) == false)
		{
			// has some of the fragments required absent
			OutDescription += TEXT("\nHas fragments required absent: ");
			(Requirements.RequiredNoneFragments & FragmentBitSet).DebugGetStringDesc(OutDescription);
		}

		if (TagBitSet.HasNone(Requirements.RequiredNoneTags) == false)
		{
			// has some of the tags required absent
			OutDescription += TEXT("\nHas tags required absent: ");
			(Requirements.RequiredNoneTags & TagBitSet).DebugGetStringDesc(OutDescription);
		}

		if (ChunkFragmentBitSet.HasNone(Requirements.RequiredNoneChunkFragments) == false)
		{
			// has some of the chunk fragments required absent
			OutDescription += TEXT("\nHas chunk fragments required absent: ");
			(Requirements.RequiredNoneChunkFragments & ChunkFragmentBitSet).DebugGetStringDesc(OutDescription);
		}

		if (SharedFragmentBitSet.HasNone(Requirements.RequiredNoneSharedFragments) == false)
		{
			// has some of the chunk fragments required absent
			OutDescription += TEXT("\nHas shared fragments required absent: ");
			(Requirements.RequiredNoneSharedFragments & SharedFragmentBitSet).DebugGetStringDesc(OutDescription);
		}

		if (ConstSharedFragmentBitSet.HasNone(Requirements.RequiredNoneConstSharedFragments) == false)
		{
			// has some of the chunk fragments required absent
			OutDescription += TEXT("\nHas const shared fragments required absent: ");
			(Requirements.RequiredNoneConstSharedFragments & ConstSharedFragmentBitSet).DebugGetStringDesc(OutDescription);
		}
	}

	// if we have regular (i.e. non-optional) positive requirements then these are the determining factor, we don't check optionals
	if (Requirements.HasPositiveRequirements())
	{
		if (FragmentBitSet.HasAll(Requirements.RequiredAllFragments) == false)
		{
			// missing one of the strictly required fragments
			OutDescription += TEXT("\nMissing required fragments: ");
			(Requirements.RequiredAllFragments - FragmentBitSet).DebugGetStringDesc(OutDescription);
		}

		if (Requirements.RequiredAnyFragments.IsEmpty() == false && FragmentBitSet.HasAny(Requirements.RequiredAnyFragments) == false)
		{
			// missing all of the "any" fragments
			OutDescription += TEXT("\nMissing all \'any\' fragments: ");
			Requirements.RequiredAnyFragments.DebugGetStringDesc(OutDescription);
		}

		if (TagBitSet.HasAll(Requirements.RequiredAllTags) == false)
		{
			// missing one of the strictly required tags
			OutDescription += TEXT("\nMissing required tags: ");
			(Requirements.RequiredAllTags - TagBitSet).DebugGetStringDesc(OutDescription);
		}

		if (Requirements.RequiredAnyTags.IsEmpty() == false && TagBitSet.HasAny(Requirements.RequiredAnyTags) == false)
		{
			// missing all of the "any" tags
			OutDescription += TEXT("\nMissing all \'any\' tags: ");
			Requirements.RequiredAnyTags.DebugGetStringDesc(OutDescription);
		}

		if (ChunkFragmentBitSet.HasAll(Requirements.RequiredAllChunkFragments) == false)
		{
			// missing one of the strictly required chunk fragments
			OutDescription += TEXT("\nMissing required chunk fragments: ");
			(Requirements.RequiredAllChunkFragments - ChunkFragmentBitSet).DebugGetStringDesc(OutDescription);
		}

		if (SharedFragmentBitSet.HasAll(Requirements.RequiredAllSharedFragments) == false)
		{
			// missing one of the strictly required Shared fragments
			OutDescription += TEXT("\nMissing required Shared fragments: ");
			(Requirements.RequiredAllSharedFragments - SharedFragmentBitSet).DebugGetStringDesc(OutDescription);
		}

		if (ConstSharedFragmentBitSet.HasAll(Requirements.RequiredAllConstSharedFragments) == false)
		{
			// missing one of the strictly required Shared fragments
			OutDescription += TEXT("\nMissing required Const Shared fragments: ");
			(Requirements.RequiredAllConstSharedFragments - ConstSharedFragmentBitSet).DebugGetStringDesc(OutDescription);
		}
	}
	// else we check if there are any optionals and if so test them
	else if (Requirements.HasOptionalRequirements() && (Requirements.DoesMatchAnyOptionals(ArchetypeCompositionBitSet) == false))
	{
		// we report that none of the optionals has been met
		OutDescription += TEXT("\nNone of the optionals were safisfied while not having other positive hard requirements: ");

		Requirements.RequiredOptionalTags.DebugGetStringDesc(OutDescription);
		Requirements.RequiredOptionalFragments.DebugGetStringDesc(OutDescription);
		Requirements.RequiredOptionalChunkFragments.DebugGetStringDesc(OutDescription);
		Requirements.RequiredOptionalSharedFragments.DebugGetStringDesc(OutDescription);
		Requirements.RequiredOptionalConstSharedFragments.DebugGetStringDesc(OutDescription);
	}

	return OutDescription.Len() > 0 ? static_cast<FString>(OutDescription) : TEXT("Match");
}

FString FMassDebugger::GetSingleRequirementDescription(const FMassFragmentRequirementDescription& Requirement)
{
	return FString::Printf(TEXT("%s%s[%s]"), Requirement.IsOptional() ? TEXT("?") : (Requirement.Presence == EMassFragmentPresence::None ? TEXT("-") : TEXT("+"))
		, *GetNameSafe(Requirement.StructType), *UE::Mass::Debug::DebugGetFragmentAccessString(Requirement.AccessMode));
}

void FMassDebugger::OutputArchetypeDescription(FOutputDevice& Ar, const FMassArchetypeHandle& ArchetypeHandle)
{
	Ar.Logf(TEXT("%s"), ArchetypeHandle.IsValid() ? *FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle).DebugGetDescription() : TEXT("INVALID"));
}

void FMassDebugger::OutputEntityDescription(FOutputDevice& Ar, const FMassEntityManager& EntityManager, const int32 EntityIndex, const TCHAR* InPrefix)
{
	if (EntityIndex >= EntityManager.DebugGetEntityStorageInterface().Num())
	{
		Ar.Logf(ELogVerbosity::Log, TEXT("Unable to list fragments values for out of range index in EntityManager owned by %s"), *GetPathNameSafe(EntityManager.GetOwner()));
		return;
	}
	
	if (!EntityManager.DebugGetEntityStorageInterface().IsValid(EntityIndex))
	{
		Ar.Logf(ELogVerbosity::Log, TEXT("Unable to list fragments values for invalid entity in EntityManager owned by %s"), *GetPathNameSafe(EntityManager.GetOwner()));
	}
	
	FMassEntityHandle Entity;
	Entity.Index = EntityIndex;
	Entity.SerialNumber = EntityManager.DebugGetEntityStorageInterface().GetSerialNumber(EntityIndex);
	OutputEntityDescription(Ar, EntityManager, Entity, InPrefix);
}

void FMassDebugger::OutputEntityDescription(FOutputDevice& Ar, const FMassEntityManager& EntityManager, const FMassEntityHandle Entity, const TCHAR* InPrefix)
{
	if (!EntityManager.IsEntityActive(Entity))
	{
		Ar.Logf(ELogVerbosity::Log, TEXT("Unable to list fragments values for invalid entity in EntityManager owned by %s"), *GetPathNameSafe(EntityManager.GetOwner()));
	}

	Ar.Logf(ELogVerbosity::Log, TEXT("Listing fragments values for Entity[%s] in EntityManager owned by %s"), *Entity.DebugGetDescription(), *GetPathNameSafe(EntityManager.GetOwner()));

	FMassArchetypeData* Archetype = EntityManager.DebugGetEntityStorageInterface().GetArchetypeAsShared(Entity.Index).Get();
	if (Archetype == nullptr)
	{
		Ar.Logf(ELogVerbosity::Log, TEXT("Unable to list fragments values for invalid entity in EntityManager owned by %s"), *GetPathNameSafe(EntityManager.GetOwner()));
	}
	else
	{
		Archetype->DebugPrintEntity(Entity, Ar, InPrefix);
	}
}

void FMassDebugger::SelectEntity(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle)
{
	if (EntityManager.IsEntityValid(EntityHandle))
	{
		UE::Mass::Debug::SetDebugEntityRange(EntityHandle.Index, EntityHandle.Index);

		GetActiveEnvironmentChecked(EntityManager).SelectedEntity = EntityHandle;

		OnEntitySelectedDelegate.Broadcast(EntityManager, EntityHandle);
	}
}

FMassEntityHandle FMassDebugger::GetSelectedEntity(const FMassEntityManager& EntityManager)
{
	return GetActiveEnvironmentChecked(EntityManager).SelectedEntity;
}

void FMassDebugger::HighlightEntity(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle)
{
	if (FEnvironment* ActiveEnvironment = GetActiveEnvironment(EntityManager))
	{
		ActiveEnvironment->HighlightedEntity = EntityHandle;
	}
}

FMassEntityHandle FMassDebugger::GetHighlightedEntity(const FMassEntityManager& EntityManager)
{
	if (FEnvironment* ActiveEnvironment = GetActiveEnvironment(EntityManager))
	{
		return ActiveEnvironment->HighlightedEntity;
	}
	return FMassEntityHandle();
}

bool FMassDebugger::IsEntityManagerInitialized(const FMassEntityManager& EntityManager)
{
	return EntityManager.InitializationState == FMassEntityManager::EInitializationState::Initialized;
}

int32 FMassDebugger::RegisterEntityManager(FMassEntityManager& EntityManager)
{
	int32 NewEnvironmentIndex = INDEX_NONE;
	{
		UE::TScopeLock ScopeLock(EntityManagerRegistrationLock);
		NewEnvironmentIndex = ActiveEnvironments.Emplace(EntityManager);
	}
	OnEntityManagerInitialized.Broadcast(EntityManager);
	return NewEnvironmentIndex;
}

void FMassDebugger::UnregisterEntityManager(FMassEntityManager& EntityManager)
{
	if (EntityManager.DoesSharedInstanceExist())
	{
		UE::TScopeLock ScopeLock(EntityManagerRegistrationLock);
		const int32 Index = ActiveEnvironments.IndexOfByPredicate([WeakManager = EntityManager.AsWeak()](const FEnvironment& Element) 
		{
			return Element.EntityManager == WeakManager;
		});
		if (Index != INDEX_NONE)
		{
			ActiveEnvironments.RemoveAt(Index, EAllowShrinking::No);
		}
	}
	else
	{
		UE::TScopeLock ScopeLock(EntityManagerRegistrationLock);
		ActiveEnvironments.RemoveAll([](const FEnvironment& Item)
			{
				return Item.IsValid() == false;
			});
	}
	OnEntityManagerDeinitialized.Broadcast(EntityManager);
}

void FMassDebugger::RegisterProcessorDataProvider(FName ProviderName, const TSharedRef<FMassEntityManager>& EntityManager, const UE::Mass::Debug::FProcessorProviderFunction& ProviderFunction)
{
	int32 Index;
	{
		UE::TScopeLock ScopeLock(EntityManagerRegistrationLock);
		Index = ActiveEnvironments.IndexOfByPredicate([WeakEntityManager = EntityManager->AsWeak()](const FEnvironment& Element) 
		{
			return Element.EntityManager == WeakEntityManager;
		});

		if (Index == INDEX_NONE)
		{
			Index = RegisterEntityManager(*EntityManager);
		}
	
		ActiveEnvironments[Index].ProcessorProviders.FindOrAdd(ProviderName, ProviderFunction);
	}
	OnProcessorProviderRegistered.Broadcast(ActiveEnvironments[Index]);
}

FMassDebugger::FEnvironment* FMassDebugger::FindEnvironmentForEntityManager(const FMassEntityManager& EntityManager)
{
	for (FMassDebugger::FEnvironment& Environment : ActiveEnvironments)
	{
		if (Environment.EntityManager.HasSameObject(&EntityManager))
		{
			return &Environment;
		}
	}
	return nullptr;
}

bool FMassDebugger::DoesArchetypeMatchRequirements(const FMassArchetypeHandle& ArchetypeHandle, const FMassFragmentRequirements& Requirements, FOutputDevice& OutputDevice)
{
	if (const FMassArchetypeData* Archetype = FMassArchetypeHelper::ArchetypeDataFromHandle(ArchetypeHandle))
	{
		return FMassArchetypeHelper::DoesArchetypeMatchRequirements(*Archetype, Requirements, /*bBailOutOnFirstFail=*/false, &OutputDevice);
	}
	return false;
}

UE::Mass::Debug::FBreakpointHandle FMassDebugger::ShouldProcessorBreak(const FMassEntityManager& EntityManager, const UMassProcessor* Processor, FMassEntityHandle Entity)
{
	if (LIKELY(!UE::Mass::Debug::FBreakpoint::HasBreakpoint()))
	{
		return UE::Mass::Debug::FBreakpointHandle::Invalid();
	}

	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);

	if (LIKELY(!ActiveEnvironment.bHasBreakpoint))
	{
		return UE::Mass::Debug::FBreakpointHandle::Invalid();
	}

	const bool ProcessorHasBreakpoint = ActiveEnvironment.ProcessorsWithBreakpoints.Contains(Processor);
	
	if (ProcessorHasBreakpoint)
	{
		UE::Mass::Debug::FBreakpointHandle FoundHandle = 0;
		for (UE::Mass::Debug::FBreakpoint& BreakPoint : ActiveEnvironment.Breakpoints)
		{
			if (BreakPoint.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::ProcessorExecute)
			{
				TObjectKey<const UMassProcessor> Proc = BreakPoint.Trigger.Get<TObjectKey<const UMassProcessor>>();
				if (Proc == Processor && BreakPoint.ApplyEntityFilter(EntityManager, Entity))
				{
					++BreakPoint.HitCount;
					if (BreakPoint.bEnabled)
					{
						UE::Mass::Debug::FBreakpoint::LastBreakpointHandle = BreakPoint.Handle;
						FoundHandle = BreakPoint.Handle;
					}
				}
			}
		}
		return FoundHandle;
	}
	return UE::Mass::Debug::FBreakpointHandle::Invalid();
}

bool FMassDebugger::HasAnyProcessorBreakpoints(const FMassEntityManager& EntityManager, const UMassProcessor* Processor)
{
	if (LIKELY(!UE::Mass::Debug::FBreakpoint::HasBreakpoint()))
	{
		return false;
	}

	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);

	if (LIKELY(!ActiveEnvironment.bHasBreakpoint))
	{
		return false;
	}

	return ActiveEnvironment.ProcessorsWithBreakpoints.Contains(Processor);
}

UE::Mass::Debug::FBreakpointHandle FMassDebugger::ShouldBreakOnFragmentWrite(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity)
{
	if (LIKELY(!UE::Mass::Debug::FBreakpoint::HasBreakpoint()))
	{
		return UE::Mass::Debug::FBreakpointHandle::Invalid();
	}

	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);

	if (LIKELY(!ActiveEnvironment.bHasBreakpoint))
	{
		return UE::Mass::Debug::FBreakpointHandle::Invalid();
	}

	const bool bFragmentHasBreakpoint = ActiveEnvironment.FragmentsWithBreakpoints.Contains(FragmentType);

	if (bFragmentHasBreakpoint)
	{
		UE::Mass::Debug::FBreakpointHandle FoundHandle;
		for (UE::Mass::Debug::FBreakpoint& BreakPoint : ActiveEnvironment.Breakpoints)
		{
			if (BreakPoint.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentWrite)
			{
				TObjectKey<const UScriptStruct> BreakFragment = BreakPoint.Trigger.Get<TObjectKey<const UScriptStruct>>();
				if (BreakFragment == FragmentType && BreakPoint.ApplyEntityFilter(EntityManager, Entity))
				{
					++BreakPoint.HitCount;
					UE::Mass::Debug::FBreakpoint::LastBreakpointHandle = BreakPoint.Handle;
					if (BreakPoint.bEnabled)
					{
						FoundHandle = BreakPoint.Handle;
					}
				}
			}
		}
		return FoundHandle;
	}
	return UE::Mass::Debug::FBreakpointHandle::Invalid();
}

UE::Mass::Debug::FBreakpoint* FMassDebugger::FindBreakpoint(const FMassEntityManager& EntityManager, UE::Mass::Debug::FBreakpointHandle Handle)
{
	FEnvironment* ActiveEnvironment = GetActiveEnvironment(EntityManager);
	if (ActiveEnvironment)
	{
		return ActiveEnvironment->FindBreakpointByHandle(Handle);
	}
	return nullptr;
}

TArray<UE::Mass::Debug::FBreakpoint>& FMassDebugger::GetBreakpoints(const FMassEntityManager& EntityManager)
{
	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);
	return ActiveEnvironment.Breakpoints;
}

bool FMassDebugger::HasAnyFragmentWriteBreakpoints(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType)
{
	if (LIKELY(!UE::Mass::Debug::FBreakpoint::HasBreakpoint()))
	{
		return false;
	}

	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);

	if (LIKELY(!ActiveEnvironment.bHasBreakpoint))
	{
		return false;
	}

	if (FragmentType == nullptr)
	{
		return ActiveEnvironment.FragmentsWithBreakpoints.Num() > 0;
	}

	return ActiveEnvironment.FragmentsWithBreakpoints.Contains(FragmentType);
}

UE::Mass::Debug::FBreakpoint& FMassDebugger::CreateBreakpoint(const FMassEntityManager& EntityManager)
{
	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);

	return ActiveEnvironment.Breakpoints.Emplace_GetRef();
}

void FMassDebugger::SetProcessorBreakpoint(const FMassEntityManager& EntityManager, TNotNull<const UMassProcessor*> Processor, FMassEntityHandle Entity)
{
	UE::Mass::Debug::FNoDebuggerNotification::ConditionallyNotifyUser();

	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);

	ActiveEnvironment.bHasBreakpoint = true;
	UE::Mass::Debug::FBreakpoint::bHasBreakpoint = true;

	UE::Mass::Debug::FBreakpoint& NewBreakpoint = ActiveEnvironment.Breakpoints.Emplace_GetRef();
	NewBreakpoint.bEnabled = true;
	NewBreakpoint.TriggerType = UE::Mass::Debug::FBreakpoint::ETriggerType::ProcessorExecute;
	NewBreakpoint.Trigger.Set<TObjectKey<const UMassProcessor>>(Processor);
	if (Entity.IsSet())
	{
		NewBreakpoint.FilterType = UE::Mass::Debug::FBreakpoint::EFilterType::SpecificEntity;
		NewBreakpoint.Filter.Set<FMassEntityHandle>(Entity);
	}

	ActiveEnvironment.ProcessorsWithBreakpoints.Add(Processor);
	OnBreakpointsChangedDelegate.Broadcast();
}

void FMassDebugger::SetFragmentWriteBreakpoint(const FMassEntityManager& EntityManager, TNotNull<const UScriptStruct*> FragmentType, FMassEntityHandle Entity)
{
	UE::Mass::Debug::FNoDebuggerNotification::ConditionallyNotifyUser();

	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);

	ActiveEnvironment.bHasBreakpoint = true;
	UE::Mass::Debug::FBreakpoint::bHasBreakpoint = true;

	UE::Mass::Debug::FBreakpoint& NewBreakpoint = ActiveEnvironment.Breakpoints.Emplace_GetRef();
	NewBreakpoint.bEnabled = true;
	NewBreakpoint.TriggerType = UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentWrite;
	NewBreakpoint.Trigger.Set<TObjectKey<const UScriptStruct>>(FragmentType);
	if (Entity.IsSet())
	{
		NewBreakpoint.FilterType = UE::Mass::Debug::FBreakpoint::EFilterType::SpecificEntity;
		NewBreakpoint.Filter.Set<FMassEntityHandle>(Entity);
	}

	ActiveEnvironment.FragmentsWithBreakpoints.Add(FragmentType);
	OnBreakpointsChangedDelegate.Broadcast();
}

void FMassDebugger::SetFragmentWriteBreakForSelectedEntity(const FMassEntityManager& EntityManager, TNotNull<const UScriptStruct*> FragmentType)
{
	UE::Mass::Debug::FNoDebuggerNotification::ConditionallyNotifyUser();

	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);

	ActiveEnvironment.bHasBreakpoint = true;
	UE::Mass::Debug::FBreakpoint::bHasBreakpoint = true;

	UE::Mass::Debug::FBreakpoint NewBreakpoint = ActiveEnvironment.Breakpoints.Emplace_GetRef();
	NewBreakpoint.bEnabled = true;
	NewBreakpoint.TriggerType = UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentWrite;
	NewBreakpoint.Trigger.Set<TObjectKey<const UScriptStruct>>(FragmentType);
	NewBreakpoint.FilterType = UE::Mass::Debug::FBreakpoint::EFilterType::SelectedEntity;

	ActiveEnvironment.FragmentsWithBreakpoints.Add(FragmentType);
	OnBreakpointsChangedDelegate.Broadcast();
}

void FMassDebugger::SetBreakpointEnabled(UE::Mass::Debug::FBreakpointHandle Handle, bool bEnable)
{
	for (FEnvironment& Environment : ActiveEnvironments)
	{
		for (UE::Mass::Debug::FBreakpoint& Breakpoint : Environment.Breakpoints)
		{
			if (Breakpoint.Handle == Handle)
			{
				Breakpoint.bEnabled = bEnable;
				OnBreakpointsChangedDelegate.Broadcast();
				return;
			}
		}
	}
}

void FMassDebugger::ClearProcessorBreakpoint(const FMassEntityManager& EntityManager, const UMassProcessor* Processor, FMassEntityHandle Entity)
{
	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);

	if (ActiveEnvironment.ProcessorsWithBreakpoints.Contains(Processor))
	{
		for (int EnvironmentIndex = 0; EnvironmentIndex < ActiveEnvironment.Breakpoints.Num(); EnvironmentIndex++)
		{
			UE::Mass::Debug::FBreakpoint& Breakpoint = ActiveEnvironment.Breakpoints[EnvironmentIndex];

			if (Breakpoint.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::ProcessorExecute
				&& Breakpoint.Trigger.Get<TObjectKey<const UMassProcessor>>() == Processor
				&& Breakpoint.ApplyEntityFilter(EntityManager, Entity))
			{
				ActiveEnvironment.Breakpoints.RemoveAt(EnvironmentIndex);
			}
		}
	}
	OnBreakpointsChangedDelegate.Broadcast();
}

void FMassDebugger::ClearAllProcessorBreakpoints(const FMassEntityManager& EntityManager, const UMassProcessor* Processor)
{
	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);
	for (int EnvironmentIndex = 0; EnvironmentIndex < ActiveEnvironment.Breakpoints.Num(); EnvironmentIndex++)
	{
		UE::Mass::Debug::FBreakpoint& Breakpoint = ActiveEnvironment.Breakpoints[EnvironmentIndex];

		if (Breakpoint.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::ProcessorExecute
			&& Breakpoint.Trigger.Get<TObjectKey<const UMassProcessor>>() == Processor)
		{
			ActiveEnvironment.Breakpoints.RemoveAt(EnvironmentIndex);
		}
	}

	ActiveEnvironment.ProcessorsWithBreakpoints.Remove(Processor);
	OnBreakpointsChangedDelegate.Broadcast();
}

void FMassDebugger::ClearFragmentWriteBreak(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity)
{
	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);

	if (ActiveEnvironment.FragmentsWithBreakpoints.Contains(FragmentType))
	{
		for (int EnvironmentIndex = 0; EnvironmentIndex < ActiveEnvironment.Breakpoints.Num(); EnvironmentIndex++)
		{
			UE::Mass::Debug::FBreakpoint& Breakpoint = ActiveEnvironment.Breakpoints[EnvironmentIndex];

			if (Breakpoint.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::FragmentWrite
				&& Breakpoint.Trigger.Get<TObjectKey<const UScriptStruct>>() == FragmentType
				&& Breakpoint.ApplyEntityFilter(EntityManager, Entity))
			{
				ActiveEnvironment.Breakpoints.RemoveAt(EnvironmentIndex);
			}
		}
	}
	OnBreakpointsChangedDelegate.Broadcast();
}

void FMassDebugger::ClearAllFragmentWriteBreak(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType)
{
	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);
	for (int EnvironmentIndex = 0; EnvironmentIndex < ActiveEnvironment.Breakpoints.Num(); EnvironmentIndex++)
	{
		UE::Mass::Debug::FBreakpoint& Breakpoint = ActiveEnvironment.Breakpoints[EnvironmentIndex];

		if (Breakpoint.TriggerType == UE::Mass::Debug::FBreakpoint::ETriggerType::ProcessorExecute
			&& Breakpoint.Trigger.Get<TObjectKey<const UScriptStruct>>() == FragmentType)
		{
			ActiveEnvironment.Breakpoints.RemoveAt(EnvironmentIndex);
		}
	}

	ActiveEnvironment.FragmentsWithBreakpoints.Remove(FragmentType);
	OnBreakpointsChangedDelegate.Broadcast();
}

void FMassDebugger::ClearAllEntityBreakpoints(const FMassEntityManager& EntityManager, FMassEntityHandle Entity)
{
	FEnvironment& ActiveEnvironment = GetActiveEnvironmentChecked(EntityManager);

	for (int EnvironmentIndex = 0; EnvironmentIndex < ActiveEnvironment.Breakpoints.Num(); EnvironmentIndex++)
	{
		UE::Mass::Debug::FBreakpoint& Breakpoint = ActiveEnvironment.Breakpoints[EnvironmentIndex];

		FMassEntityHandle* BreakHandle = Breakpoint.Filter.TryGet<FMassEntityHandle>();
		if (BreakHandle
			&& *BreakHandle == Entity)
		{
			ActiveEnvironment.Breakpoints.RemoveAt(EnvironmentIndex);
		}
	}

	OnBreakpointsChangedDelegate.Broadcast();
}
 
void FMassDebugger::BreakOnFragmentWriteForSelectedEntity(FName FragmentName)
{
	for (FEnvironment& Environment : ActiveEnvironments)
	{
		SetFragmentWriteBreakForSelectedEntity(*Environment.EntityManager.Pin(), GetFragmentTypeFromName(FragmentName));
	}
	OnBreakpointsChangedDelegate.Broadcast();
}

void FMassDebugger::ClearAllBreakpoints()
{
	for (FEnvironment& Environment : ActiveEnvironments)
	{
		Environment.ClearBreakpoints();
	}
	UE::Mass::Debug::FBreakpoint::bHasBreakpoint = false;
	OnBreakpointsChangedDelegate.Broadcast();
}

void FMassDebugger::RemoveBreakpoint(UE::Mass::Debug::FBreakpointHandle Handle)
{
	for (FEnvironment& Environment : ActiveEnvironments)
	{
		for (int EnvironmentIndex = 0; EnvironmentIndex < Environment.Breakpoints.Num(); EnvironmentIndex++)
		{
			if (Environment.Breakpoints[EnvironmentIndex].Handle == Handle)
			{
				Environment.Breakpoints.RemoveAt(EnvironmentIndex);
				RefreshBreakpoints();
				return;
			}
		}
	}
}

void FMassDebugger::ShutdownDebugger()
{
	// Don't call ClearAllBreakpoints here because we don't want to send the BreakpointsChanged delegate
	for (FEnvironment& Environment : ActiveEnvironments)
	{
		Environment.ClearBreakpoints();
	}
	UE::Mass::Debug::FBreakpoint::bHasBreakpoint = false;
	ActiveEnvironments.Empty();
	FragmentsByName.Empty();
}

const UScriptStruct* FMassDebugger::GetFragmentTypeFromName(FName FragmentName)
{
	const UScriptStruct** FoundType = FragmentsByName.Find(FragmentName);
	if (FoundType)
	{
		return *FoundType;
	}

	for (FEnvironment& Environment : ActiveEnvironments)
	{
		TArray<FMassArchetypeHandle> ArchetypeHandles = GetAllArchetypes(*Environment.EntityManager.Pin());
		for (FMassArchetypeHandle& ArchetypeHandle : ArchetypeHandles)
		{
			const FMassArchetypeCompositionDescriptor& Composition = GetArchetypeComposition(ArchetypeHandle);

			FMassFragmentBitSet::FIndexIterator It = Composition.DebugGetFragments().GetIndexIterator();
			while (It)
			{
				FName StructName = Composition.DebugGetFragments().DebugGetStructTypeName(*It);
				const UScriptStruct* StructType = Composition.DebugGetFragments().GetTypeAtIndex(*It);
				
				FragmentsByName.Add(StructName, StructType);
				++It;
			}

			FMassChunkFragmentBitSet::FIndexIterator ChunkIt = Composition.DebugGetChunkFragments().GetIndexIterator();
			while (ChunkIt)
			{
				FName StructName = Composition.DebugGetChunkFragments().DebugGetStructTypeName(*ChunkIt);
				const UScriptStruct* StructType = Composition.DebugGetChunkFragments().GetTypeAtIndex(*ChunkIt);

				FragmentsByName.Add(StructName, StructType);
				++ChunkIt;
			}

			FMassSharedFragmentBitSet::FIndexIterator SharedFragIt = Composition.DebugGetSharedFragments().GetIndexIterator();
			while (SharedFragIt)
			{
				FName StructName = Composition.DebugGetSharedFragments().DebugGetStructTypeName(*SharedFragIt);
				const UScriptStruct* StructType = Composition.DebugGetSharedFragments().GetTypeAtIndex(*SharedFragIt);

				FragmentsByName.Add(StructName, StructType);
				++SharedFragIt;
			}

			FMassConstSharedFragmentBitSet::FIndexIterator ConstSharedFragIt = Composition.DebugGetConstSharedFragments().GetIndexIterator();
			while (ConstSharedFragIt)
			{
				FName StructName = Composition.DebugGetConstSharedFragments().DebugGetStructTypeName(*ConstSharedFragIt);
				const UScriptStruct* StructType = Composition.DebugGetConstSharedFragments().GetTypeAtIndex(*ConstSharedFragIt);

				FragmentsByName.Add(StructName, StructType);
				++ConstSharedFragIt;
			}
		}
	}

	FoundType = FragmentsByName.Find(FragmentName);
	if (FoundType)
	{
		return *FoundType;
	}

	return nullptr;
}

TSharedPtr<FStructOnScope> FMassDebugger::GetFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity)
{
	TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FragmentType);
	if (GetFragmentData(EntityManager, FragmentType, Entity, StructOnScope))
	{
		return StructOnScope;
	}
	return nullptr;
}

bool FMassDebugger::GetFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity, TSharedPtr<FStructOnScope>& OutStructData)
{
	if (!EntityManager.IsEntityValid(Entity))
	{
		return false;
	}
	TSharedPtr<FMassArchetypeData> Archetype = EntityManager.DebugGetEntityStorageInterface().GetArchetypeAsShared(Entity.Index);
	if (Archetype.IsValid())
	{
		const void* FragmentData = Archetype->GetFragmentDataForEntity(FragmentType, Entity);
		if (FragmentData)
		{
			const UStruct* AsUStruct = FragmentType;
			if (OutStructData->GetStruct() != AsUStruct)
			{
				OutStructData->Initialize(FragmentType);
			}
			
			CastChecked<UScriptStruct>(OutStructData->GetStruct())->CopyScriptStruct(OutStructData->GetStructMemory(), FragmentData);
			return true;
		}
	}
	return false;
}

const FMassArchetypeSharedFragmentValues& FMassDebugger::GetSharedFragmentValues(const FMassEntityManager& EntityManager, FMassEntityHandle Entity)
{
	if (EntityManager.IsEntityValid(Entity))
	{
		TSharedPtr<FMassArchetypeData> Archetype = EntityManager.DebugGetEntityStorageInterface().GetArchetypeAsShared(Entity.Index);
		if (Archetype.IsValid())
		{
			return Archetype->GetSharedFragmentValues(Entity);
		}
	}
	
	static FMassArchetypeSharedFragmentValues Dummy;
	return Dummy;
}

TSharedPtr<FStructOnScope> FMassDebugger::GetSharedFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity)
{
	TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FragmentType);
	if (GetSharedFragmentData(EntityManager, FragmentType, Entity, StructOnScope))
	{
		return StructOnScope;
	}
	return nullptr;
}

bool FMassDebugger::GetSharedFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity, TSharedPtr<FStructOnScope>& OutStructData)
{
	if (!EntityManager.IsEntityValid(Entity))
	{
		return false;
	}
	TSharedPtr<FMassArchetypeData> Archetype = EntityManager.DebugGetEntityStorageInterface().GetArchetypeAsShared(Entity.Index);
	if (Archetype.IsValid())
	{
		const FSharedStruct* SharedFragment = Archetype->GetSharedFragmentValues(Entity).GetSharedFragments().FindByPredicate(FStructTypeEqualOperator(FragmentType));
		void* FragmentData = (SharedFragment != nullptr) ? SharedFragment->GetMemory() : nullptr;

		if (FragmentData)
		{
			const UStruct* AsUStruct = FragmentType;
			if (OutStructData->GetStruct() != AsUStruct)
			{
				OutStructData->Initialize(FragmentType);
			}

			CastChecked<UScriptStruct>(OutStructData->GetStruct())->CopyScriptStruct(OutStructData->GetStructMemory(), FragmentData);
			return true;
		}
	}
	return false;
}

TSharedPtr<FStructOnScope> FMassDebugger::GetConstSharedFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity)
{
	TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FragmentType);
	if (GetConstSharedFragmentData(EntityManager, FragmentType, Entity, StructOnScope))
	{
		return StructOnScope;
	}
	return nullptr;
}

bool FMassDebugger::GetConstSharedFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity, TSharedPtr<FStructOnScope>& OutStructData)
{
	if (!EntityManager.IsEntityValid(Entity))
	{
		return false;
	}
	TSharedPtr<FMassArchetypeData> Archetype = EntityManager.DebugGetEntityStorageInterface().GetArchetypeAsShared(Entity.Index);
	if (Archetype.IsValid())
	{
		const FConstSharedStruct* SharedFragment = Archetype->GetSharedFragmentValues(Entity).GetConstSharedFragments().FindByPredicate(FStructTypeEqualOperator(FragmentType));
		const void* FragmentData = (SharedFragment != nullptr) ? SharedFragment->GetMemory() : nullptr;

		if (FragmentData)
		{
			const UStruct* AsUStruct = FragmentType;
			if (OutStructData->GetStruct() != AsUStruct)
			{
				OutStructData->Initialize(FragmentType);
			}

			CastChecked<UScriptStruct>(OutStructData->GetStruct())->CopyScriptStruct(OutStructData->GetStructMemory(), FragmentData);
			return true;
		}
	}
	return false;
}

void FMassDebugger::RefreshBreakpoints()
{
	UE::Mass::Debug::FBreakpoint::bHasBreakpoint = false;

	for (FEnvironment& Environment : ActiveEnvironments)
	{
		Environment.ProcessorsWithBreakpoints.Reset();
		Environment.FragmentsWithBreakpoints.Reset();

		for (UE::Mass::Debug::FBreakpoint& BreakPoint : Environment.Breakpoints)
		{
			if(TObjectKey<const UMassProcessor>* BreakProcessor = BreakPoint.Trigger.TryGet<TObjectKey<const UMassProcessor>>())
			{
				Environment.ProcessorsWithBreakpoints.Add(*BreakProcessor);
			}
			else if (TObjectKey<const UScriptStruct>* BreakFragment = BreakPoint.Trigger.TryGet<TObjectKey<const UScriptStruct>>())
			{
				Environment.FragmentsWithBreakpoints.Add(*BreakFragment);
			}
		}

		Environment.bHasBreakpoint = Environment.ProcessorsWithBreakpoints.Num() != 0
			|| Environment.FragmentsWithBreakpoints.Num() != 0;
		UE::Mass::Debug::FBreakpoint::bHasBreakpoint |= Environment.bHasBreakpoint;
	}

	OnBreakpointsChangedDelegate.Broadcast();
}

FMassDebugger::FEnvironment& FMassDebugger::GetActiveEnvironmentChecked(const FMassEntityManager& EntityManager)
{
	const int32 Index = ActiveEnvironments.IndexOfByPredicate([WeakManager = EntityManager.AsWeak()](const FEnvironment& Element)
	{
		return Element.EntityManager == WeakManager;
	});

	checkf(Index != INDEX_NONE, TEXT("Mass Debug Environment not found for specified EntitManager"));

	return ActiveEnvironments[Index];
}

FMassDebugger::FEnvironment* FMassDebugger::GetActiveEnvironment(const FMassEntityManager& EntityManager)
{
	const int32 Index = ActiveEnvironments.IndexOfByPredicate([WeakManager = EntityManager.AsWeak()](const FEnvironment& Element)
	{
		return Element.EntityManager == WeakManager;
	});

	if (Index == INDEX_NONE)
	{
		return nullptr;
	}

	return &ActiveEnvironments[Index];
}

FMassDebugger::FEnvironment::FEnvironment(FMassEntityManager& InEntityManager)
	: EntityManager(InEntityManager.AsWeak())
	, MutableEntityManager(InEntityManager.AsWeak())
{
#if UE_MASS_TRACE_ENABLED
	TraceStartedDelegateHandle = FTraceAuxiliary::OnTraceStarted.AddLambda([WeakEntityManager = EntityManager](FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination)
	{
		if (TSharedPtr<const FMassEntityManager> Manager = WeakEntityManager.Pin())
		{
			ForEachArchetype(*Manager, [](FMassArchetypeHandle ArchetypeHandle)
			{
				UE_TRACE_MASS_ARCHETYPE_CREATED(ArchetypeHandle)
			});
		}
	});
#endif
}

FMassDebugger::FEnvironment::~FEnvironment()
{
#if UE_MASS_TRACE_ENABLED
	FTraceAuxiliary::OnTraceStarted.Remove(TraceStartedDelegateHandle);
#endif
}

void FMassDebugger::FEnvironment::ClearBreakpoints()
{
	ProcessorsWithBreakpoints.Reset();
	FragmentsWithBreakpoints.Reset();
	Breakpoints.Reset();
	UE::Mass::Debug::FBreakpoint::bHasBreakpoint = false;
}

UE::Mass::Debug::FBreakpoint* FMassDebugger::FEnvironment::FindBreakpointByHandle(UE::Mass::Debug::FBreakpointHandle Handle)
{
	for (UE::Mass::Debug::FBreakpoint& Breakpoint : Breakpoints)
	{
		if (Breakpoint.Handle == Handle)
		{
			return &Breakpoint;
		}
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE

#else

namespace UE::Mass::Debug
{
	UWorld* GetDebugDrawWorld(const FMassExecutionContext& Context)
	{
		return Context.GetWorld();
	}
} // namespace UE::Mass::Debug

#endif // WITH_MASSENTITY_DEBUG
