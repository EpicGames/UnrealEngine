// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDebuggerBreakpoints.h"
#if WITH_MASSENTITY_DEBUG
#include "MassDebugger.h"
#include "UObject/Class.h"
#include "UObject/ObjectKey.h"
#include "MassRequirements.h"
#include "MassProcessor.h"

#define LOCTEXT_NAMESPACE "MassDebugger"

namespace UE::Mass::Debug
{
	UE_DISABLE_OPTIMIZATION_SHIP
	void FBreakpoint::DebugBreak()
	{
		bool bDisableThisBreakpoint = false;

		//====================================================================
		//= A breakpoint set in the MassDebugger has triggered
		//= Step out of this function to debug the actual code being run
		// 
		//= To disable this specific breakpoint use the Watch window to set
		//= bDisableThisBreakpoint to `true` or 1
		//====================================================================
		UE_DEBUG_BREAK();

		if (bDisableThisBreakpoint)
		{
			DisableActiveBreakpoint();
		}
	}
	UE_ENABLE_OPTIMIZATION_SHIP

	bool FBreakpoint::bHasBreakpoint = false;
	FBreakpointHandle FBreakpoint::LastBreakpointHandle = FBreakpointHandle::Invalid();

	FBreakpoint::FBreakpoint()
	{
		Handle = FBreakpointHandle::CreateHandle();
	}

	bool FBreakpoint::ApplyEntityFilterByFragments(const TArray<const UScriptStruct*>& Fragments) const
	{
		switch (FilterType)
		{
		case EFilterType::None:
			return true;

		case EFilterType::SpecificEntity:
			return false;

		case EFilterType::SelectedEntity:
			return false;

		case EFilterType::Query:
			if (const FMassFragmentRequirements* Requirements = Filter.TryGet<FMassFragmentRequirements>())
			{
				TConstArrayView<FMassFragmentRequirementDescription> FragmentRequirements = Requirements->GetFragmentRequirements();
				for (const FMassFragmentRequirementDescription& FragmentRequirement : FragmentRequirements)
				{
					if (FragmentRequirement.Presence == EMassFragmentPresence::All
						&& !Fragments.Contains(FragmentRequirement.StructType))
					{
						return false;
					}
					else if (FragmentRequirement.Presence == EMassFragmentPresence::None
						&& Fragments.Contains(FragmentRequirement.StructType))
					{
						return false;
					}
				}

				TConstArrayView<FMassFragmentRequirementDescription> ChunkRequirements = Requirements->GetChunkFragmentRequirements();
				for (const FMassFragmentRequirementDescription& FragmentRequirement : ChunkRequirements)
				{
					if (FragmentRequirement.Presence == EMassFragmentPresence::All
						&& !Fragments.Contains(FragmentRequirement.StructType))
					{
						return false;
					}
					else if (FragmentRequirement.Presence == EMassFragmentPresence::None
						&& Fragments.Contains(FragmentRequirement.StructType))
					{
						return false;
					}
				}

				const FMassTagBitSet& RequiredAllTagsBitSet = Requirements->GetRequiredAllTags();
				TArray<const UScriptStruct*> RequiredAllTags;
				RequiredAllTagsBitSet.ExportTypes(RequiredAllTags);
				for (const UScriptStruct* Tag : RequiredAllTags)
				{
					if (!Fragments.Contains(Tag))
					{
						return false;
					}
				}

				const FMassTagBitSet& BlockedTagsBitSet = Requirements->GetRequiredNoneTags();
				TArray<const UScriptStruct*> BlockedTags;
				BlockedTagsBitSet.ExportTypes(BlockedTags);
				for (const UScriptStruct* Tag : BlockedTags)
				{
					if (Fragments.Contains(Tag))
					{
						return false;
					}
				}
			}
			return false;
		default:
			break;
		}

		return true;
	}

	bool FBreakpoint::ApplyEntityFilterByArchetype(const FMassArchetypeHandle& ArchetypeHandle) const
	{
		switch (FilterType)
		{
		case EFilterType::None:
			return true;

		case EFilterType::Query:
		{
			if (const FMassFragmentRequirements* Requirements = Filter.TryGet<FMassFragmentRequirements>())
			{
				return Requirements->DoesArchetypeMatchRequirements(ArchetypeHandle);
			}
			return false;
		}
		default:
			break;
		}

		return false;
	}

	bool FBreakpoint::ApplyEntityFilter(const FMassEntityManager& EntityManager, const FMassEntityHandle& Entity) const
	{
		switch (FilterType)
		{
		case EFilterType::None:
			return true;

		case EFilterType::SpecificEntity:
		{
			if (const FMassEntityHandle* SpecificEntity = Filter.TryGet<FMassEntityHandle>())
			{
				return SpecificEntity && (*SpecificEntity == Entity);
			}
			return false;
		}

		case EFilterType::SelectedEntity:
		{
			FMassEntityHandle SelectedEntity = FMassDebugger::GetSelectedEntity(EntityManager);
			return SelectedEntity == Entity;
		}

		case EFilterType::Query:
		{
			if (const FMassFragmentRequirements* Requirements = Filter.TryGet<FMassFragmentRequirements>())
			{
				FMassArchetypeHandle Archetype = EntityManager.GetArchetypeForEntity(Entity);
				if (Archetype.IsValid())
				{
					return Requirements->DoesArchetypeMatchRequirements(Archetype);
				}
			}
			return false;
		}
		default:
			break;
		}

		return false;
	}

	FString FBreakpoint::TriggerTypeToString(FBreakpoint::ETriggerType InType)
	{
		switch (InType)
		{
		case FBreakpoint::ETriggerType::None:
			return TEXT("None");

		case FBreakpoint::ETriggerType::ProcessorExecute:
			return TEXT("ProcessorExecute");

		case FBreakpoint::ETriggerType::FragmentWrite:
			return TEXT("FragmentWrite");

		case FBreakpoint::ETriggerType::EntityCreate:
			return TEXT("EntityCreate");

		case FBreakpoint::ETriggerType::EntityDestroy:
			return TEXT("EntityDestroy");

		case FBreakpoint::ETriggerType::FragmentAdd:
			return TEXT("FragmentAdd");

		case FBreakpoint::ETriggerType::FragmentRemove:
			return TEXT("FragmentRemove");

		case FBreakpoint::ETriggerType::TagAdd:
			return TEXT("TagAdd");

		case FBreakpoint::ETriggerType::TagRemove:
			return TEXT("TagRemove");
		}
		return TEXT("UnknownTrigger");
	}

	bool FBreakpoint::StringToTriggerType(const FString& InString, FBreakpoint::ETriggerType& OutType)
	{
		if (InString == TEXT("None"))
		{
			OutType = FBreakpoint::ETriggerType::None;
			return true;
		}
		if (InString == TEXT("ProcessorExecute"))
		{
			OutType = FBreakpoint::ETriggerType::ProcessorExecute;
			return true;
		}
		if (InString == TEXT("FragmentWrite"))
		{
			OutType = FBreakpoint::ETriggerType::FragmentWrite;
			return true;
		}
		if (InString == TEXT("EntityCreate"))
		{
			OutType = FBreakpoint::ETriggerType::EntityCreate;
			return true;
		}
		if (InString == TEXT("EntityDestroy"))
		{
			OutType = FBreakpoint::ETriggerType::EntityDestroy;
			return true;
		}
		if (InString == TEXT("FragmentAdd"))
		{
			OutType = FBreakpoint::ETriggerType::FragmentAdd;
			return true;
		}
		if (InString == TEXT("FragmentRemove"))
		{
			OutType = FBreakpoint::ETriggerType::FragmentRemove;
			return true;
		}
		if (InString == TEXT("TagAdd"))
		{
			OutType = FBreakpoint::ETriggerType::TagAdd;
			return true;
		}
		if (InString == TEXT("TagRemove"))
		{
			OutType = FBreakpoint::ETriggerType::TagRemove;
			return true;
		}

		return false;
	}


	FString FBreakpoint::FilterTypeToString(FBreakpoint::EFilterType InType)
	{
		switch (InType)
		{
		case FBreakpoint::EFilterType::None:
			return TEXT("None");

		case FBreakpoint::EFilterType::SpecificEntity:
			return TEXT("SpecificEntity");

		case FBreakpoint::EFilterType::SelectedEntity:
			return TEXT("SelectedEntity");

		case FBreakpoint::EFilterType::Query:
			return TEXT("Query");
		}

		return TEXT("UnknownFilter");
	}

	bool FBreakpoint::StringToFilterType(const FString& InString, FBreakpoint::EFilterType& OutType)
	{
		if (InString == TEXT("None"))
		{
			OutType = FBreakpoint::EFilterType::None;
			return true;
		}
		if (InString == TEXT("SpecificEntity"))
		{
			OutType = FBreakpoint::EFilterType::SpecificEntity;
			return true;
		}
		if (InString == TEXT("SelectedEntity"))
		{
			OutType = FBreakpoint::EFilterType::SelectedEntity;
			return true;
		}
		if (InString == TEXT("Query"))
		{
			OutType = FBreakpoint::EFilterType::Query;
			return true;
		}

		return false;
	}

	bool FBreakpoint::CheckDestroyEntityBreakpoints(const FMassEntityHandle& Entity)
	{
		if (LIKELY(!bHasBreakpoint))
		{
			return false;
		}

		for (const FMassDebugger::FEnvironment& Env : FMassDebugger::GetEnvironments())
		{
			if (!Env.bHasBreakpoint)
			{
				continue;
			}

			if (TSharedPtr<const FMassEntityManager> Manager = Env.EntityManager.Pin())
			{
				bool bShouldBreak = false;
				for (const FBreakpoint& Breakpoint : Env.Breakpoints)
				{
					if (Breakpoint.TriggerType == FBreakpoint::ETriggerType::EntityDestroy
						&& Breakpoint.ApplyEntityFilter(*Manager, Entity))
					{
						++Breakpoint.HitCount;
						if (Breakpoint.bEnabled)
						{
							LastBreakpointHandle = Breakpoint.Handle;
							// Set flag to break but check the rest of the breakpoints to keep accurate HitCounts
							bShouldBreak = true;
						}
					}
				}
				if (bShouldBreak)
				{
					return true;
				}
			}
		}
		return false;
	}

	bool FBreakpoint::CheckDestroyEntityBreakpoints(TConstArrayView<FMassEntityHandle> Entities)
	{
		if (LIKELY(!bHasBreakpoint))
		{
			return false;
		}

		bool ShouldBreak = false;
		for (const FMassEntityHandle& Handle : Entities)
		{
			// need to check them all before returning to ensure hitcount is accurate
			ShouldBreak = CheckDestroyEntityBreakpoints(Handle) | ShouldBreak;
		}
		return ShouldBreak;
	}

	bool FBreakpoint::CheckFragmentAddBreakpoints(const FMassEntityHandle& Entity, const UScriptStruct* FragmentType)
	{
		if (LIKELY(!bHasBreakpoint))
		{
			return false;
		}

		for (const FMassDebugger::FEnvironment& Env : FMassDebugger::GetEnvironments())
		{
			if (!Env.bHasBreakpoint)
			{
				continue;
			}
			
			if (TSharedPtr<const FMassEntityManager> ManagerPtr = Env.EntityManager.Pin())
			{
				if (Env.FragmentsWithBreakpoints.Contains(TObjectKey<const UScriptStruct>(FragmentType)))
				{
					bool bShouldBreak = false;

					for (const FBreakpoint& Breakpoint : Env.Breakpoints)
					{
						if (Breakpoint.TriggerType != FBreakpoint::ETriggerType::FragmentAdd
							|| !Breakpoint.Trigger.IsType<TObjectKey<const UScriptStruct>>())
						{
							continue;
						}

						const UScriptStruct* BreakpointFragmentType = Breakpoint.Trigger.Get<TObjectKey<const UScriptStruct>>().ResolveObjectPtr();

						if (BreakpointFragmentType == FragmentType && Breakpoint.ApplyEntityFilter(*ManagerPtr, Entity))
						{
							++Breakpoint.HitCount;
							if (Breakpoint.bEnabled)
							{
								LastBreakpointHandle = Breakpoint.Handle;
								bShouldBreak = true;
							}
						}
					}
					if (bShouldBreak)
					{
						return true;
					}
				}
			}
		}
		return false;
	}

	bool FBreakpoint::CheckFragmentAddBreakpoints(TConstArrayView<FMassEntityHandle> Entities, const UScriptStruct* FragmentType)
	{
		if (LIKELY(!bHasBreakpoint))
		{
			return false;
		}

		for (const FMassEntityHandle& Handle : Entities)
		{
			if (FBreakpoint::CheckFragmentAddBreakpoints(Handle, FragmentType))
			{
				return true;
			}
		}
		return false;
	}

	bool FBreakpoint::CheckFragmentRemoveBreakpoints(const FMassEntityHandle& Handle, const UScriptStruct* FragmentType)
	{
		if (LIKELY(!bHasBreakpoint))
		{
			return false;
		}

		for (const FMassDebugger::FEnvironment& Env : FMassDebugger::GetEnvironments())
		{
			if (!Env.bHasBreakpoint)
			{
				continue;
			}

			if (TSharedPtr<const FMassEntityManager> ManagerPtr = Env.EntityManager.Pin())
			{
				if (Env.FragmentsWithBreakpoints.Contains(TObjectKey<const UScriptStruct>(FragmentType)))
				{
					bool bShouldBreak = false;

					for (const FBreakpoint& Breakpoint : Env.Breakpoints)
					{
						if (Breakpoint.TriggerType != FBreakpoint::ETriggerType::FragmentRemove
							|| !Breakpoint.Trigger.IsType<TObjectKey<const UScriptStruct>>())
						{
							continue;
						}

						const UScriptStruct* BreakpointFragmentType = Breakpoint.Trigger.Get<TObjectKey<const UScriptStruct>>().ResolveObjectPtr();

						if (BreakpointFragmentType == FragmentType && Breakpoint.ApplyEntityFilter(*ManagerPtr, Handle))
						{
							++Breakpoint.HitCount;
							if (Breakpoint.bEnabled)
							{
								LastBreakpointHandle = Breakpoint.Handle;
								bShouldBreak = true;
							}
						}
					}
					if (bShouldBreak)
					{
						return true;
					}
				}
			}
		}
		return false;
	}

	bool FBreakpoint::CheckFragmentRemoveBreakpoints(TConstArrayView<FMassEntityHandle> Entities, const UScriptStruct* FragmentType)
	{
		if (LIKELY(!bHasBreakpoint))
		{
			return false;
		}

		for (const FMassEntityHandle& Handle : Entities)
		{
			if (CheckFragmentRemoveBreakpoints(Handle, FragmentType))
			{
				return true;
			}
		}
		return false;
	}

	bool FBreakpoint::CheckCreateEntityBreakpoints(TArray<const UScriptStruct*> Fragments)
	{
		if (LIKELY(!bHasBreakpoint))
		{
			return false;
		}

		for (const FMassDebugger::FEnvironment& Env : FMassDebugger::GetEnvironments())
		{
			if (!Env.bHasBreakpoint)
			{
				continue;
			}

			bool bShouldBreak = false;

			for (const FBreakpoint& Breakpoint : Env.Breakpoints)
			{
				if (Breakpoint.TriggerType != FBreakpoint::ETriggerType::EntityCreate)
				{
					continue;
				}

				if (Breakpoint.ApplyEntityFilterByFragments(Fragments))
				{
					++Breakpoint.HitCount;
					if (Breakpoint.bEnabled)
					{
						LastBreakpointHandle = Breakpoint.Handle;
						bShouldBreak = true;
					}
				}
			}
			if (bShouldBreak)
			{
				return true;
			}
		}

		return false;
	}

	bool FBreakpoint::CheckCreateEntityBreakpoints(const FMassArchetypeHandle& ArchetypeHandle)
	{
		if (LIKELY(!bHasBreakpoint))
		{
			return false;
		}

		for (const FMassDebugger::FEnvironment& Env : FMassDebugger::GetEnvironments())
		{
			if (!Env.bHasBreakpoint)
			{
				continue;
			}

			bool bShouldBreak = false;

			for (const FBreakpoint& Breakpoint : Env.Breakpoints)
			{
				if (Breakpoint.TriggerType != FBreakpoint::ETriggerType::EntityCreate)
				{
					continue;
				}

				if (Breakpoint.ApplyEntityFilterByArchetype(ArchetypeHandle))
				{
					++Breakpoint.HitCount;
					if (Breakpoint.bEnabled)
					{
						LastBreakpointHandle = Breakpoint.Handle;
						bShouldBreak = true;
					}
				}
			}
			if (bShouldBreak)
			{
				return true;
			}
		}

		return false;
	}

	void FBreakpoint::DisableActiveBreakpoint()
	{
		FMassDebugger::SetBreakpointEnabled(LastBreakpointHandle, false);
	}

} // namespace UE::Mass::Debug

#undef LOCTEXT_NAMESPACE
#endif // WITH_MASSENTITY_DEBUG