// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassElement.h"

DEFINE_TYPEBITSET(FMassExternalSubsystemBitSet);

namespace UE::Mass
{
	DEFINE_TYPEBITSET(FElementBitSetBase);

	TStaticArray<FElementBitSetBase, static_cast<uint8>(EElementType::MAX)> FElementBitSet::ElementTypeBitArrays;
	FElementBitSet FElementBitSet::AllSharedFragmentsBitSet;
	FElementBitSet FElementBitSet::AllFragmentsAndTagsBitSet;
	FElementBitSet FElementBitSet::AllSparseElementsBitSet;
	FDelegateHandle FElementBitSet::OnTypeRegisteredDelegateHandle;

	void FElementBitSet::OnTypeRegistered(const TNotNull<const UStruct*> Type, int32 Index)
	{
		const EElementType ElementType = DetermineElementType(Type);
		GetTypeBitArray(ElementType).AddAtIndex(Index);

		switch (ElementType)
		{
		case EElementType::Fragment: [[fallthrough]];
		case EElementType::Tag:
			AllFragmentsAndTagsBitSet.AddAtIndex(Index);
			if (IsSparse(Type))
			{
				AllSparseElementsBitSet.AddAtIndex(Index);
			}
			break;
		case EElementType::ChunkFragment:
			break;
		case EElementType::SharedFragment: [[fallthrough]];
		case EElementType::ConstSharedFragment:
			AllSharedFragmentsBitSet.AddAtIndex(Index);
			break;
		}
	}

	void FElementBitSet::OnModuleInitialized()
	{
		FStructTracker& Tracker = FElementBitSetBaseStructTrackerWrapper::StructTracker;
		OnTypeRegisteredDelegateHandle = Tracker.OnTypeRegistered.AddStatic(&FElementBitSet::OnTypeRegistered);

		// also check if there's already something registered in FElementBitSet's FStructTracker
		// in case there were types registered before this function call
		const int32 NumTypes = Tracker.Num();
		for (int32 Index = 0; Index < NumTypes; ++Index)
		{
			if (const UStruct* Type = Tracker.GetStructType(Index))
			{
				// OnTypeRegistered uses AddAtIndex, which is idempotent.
				OnTypeRegistered(Type, Index);
			}
		}
	}
} // namespace UE::Mass
