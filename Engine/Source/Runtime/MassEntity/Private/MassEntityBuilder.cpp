// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityBuilder.h"
#include "MassCommandBuffer.h"
#include "MassCommands.h"
#include "MassEntityManager.h"
#include "VisualLogger/VisualLogger.h"
#include "HAL/IConsoleManager.h"

namespace UE::Mass
{
#if WITH_MASSENTITY_DEBUG
	namespace Debug
	{
		bool bValidateEntityBuilderMakeInput = true;
		namespace
		{
			FAutoConsoleVariableRef AnonymousCVars[] = {
				{TEXT("mass.debug.ValidateEntityBuilderMakeInput"), bValidateEntityBuilderMakeInput
					, TEXT("When set, every call to FEntityBuilder::Make will verify if the struct values provided match declared entity composition.")
					, ECVF_Cheat}
			};
		}
	}
#endif // WITH_MASSENTITY_DEBUG

	namespace Tweakable
	{
		bool bForceDeferredEntityBuilderCommit = false;
		namespace
		{
			FAutoConsoleVariableRef AnonymousCVars[] = {
				{TEXT("mass.EntityBuilder.ForceDeferredCommit"), bForceDeferredEntityBuilderCommit
					, TEXT("When set will result in each EntityBuilder commit its work using commands, regardless of EntityManager.IsProcessing value.")}
			};
		}
	}

	namespace Private
	{
		struct FEntityBuilderHelper
		{
			template<typename T>
			static void AppendFromEntity(FEntityBuilder& Builder, const FMassEntityHandle SourceEntityHandle, const FMassArchetypeCompositionDescriptor& ArchetypeComposition)
			{
				auto SourceContainer = ArchetypeComposition.GetContainer<T>();
				TArray<FInstancedStruct>& ElementInstanceContainer = Builder.GetInstancedStructContainerInternal<T>();

				// remove all the existing entries that match SourceContainer, and then just copy.

				for (auto Iterator = SourceContainer.GetIndexIterator(); Iterator; ++Iterator)
				{
					// @todo we could use an iterator that can fetch the type by simply calling Iterator.GetType()
					TNotNull<const UScriptStruct*> ElementType = SourceContainer.GetTypeAtIndex(*Iterator);
					const int32 FoundIndex = ElementInstanceContainer.IndexOfByPredicate([&ElementType](const FInstancedStruct& ExistingElement)
					{
						return ExistingElement.GetScriptStruct() == ElementType;
					});
					if (FoundIndex != INDEX_NONE)
					{
						ElementInstanceContainer.RemoveAtSwap(FoundIndex, EAllowShrinking::No);
					}
				}
				CopyFromEntity<T>(Builder, SourceEntityHandle, ArchetypeComposition);
			}

			template<typename T>
			static void CopyFromEntity(FEntityBuilder& Builder, const FMassEntityHandle SourceEntityHandle, const FMassArchetypeCompositionDescriptor& ArchetypeComposition)
			{
				auto SourceContainer = ArchetypeComposition.GetContainer<T>();
				TArray<FInstancedStruct>& ElementInstanceContainer = Builder.GetInstancedStructContainerInternal<T>();

				ElementInstanceContainer.Reserve(ElementInstanceContainer.Num() + SourceContainer.CountStoredTypes());

				for (auto Iterator = SourceContainer.GetIndexIterator(); Iterator; ++Iterator)
				{
					// @todo we could use an iterator that can fetch the type by simply calling Iterator.GetType()
					TNotNull<const UScriptStruct*> Type = SourceContainer.GetTypeAtIndex(*Iterator);
					FConstStructView SourceElementView = Builder.EntityManager->GetElementDataStruct<T>(SourceEntityHandle, Type);

					// this happening is practically impossible, so testing only in debug
					checkSlow(SourceElementView.IsValid());
					ElementInstanceContainer.Emplace(SourceElementView);
				}
			}
		};

#if WITH_MASSENTITY_DEBUG
		template<typename TElement, typename TBitset, typename TWrapper>
		bool CheckStructContainer(TConstArrayView<TWrapper> Container, const TBitset& Bitset, const UObject* LogOwner)
		{
			bool bIssuesFound = false;

			for (const TWrapper& Element : Container)
			{
				if (Mass::IsA<TElement>(Element.GetScriptStruct()))
				{
					if (Bitset.Contains(Element.GetScriptStruct()) == false)
					{
						bIssuesFound = true;
						UE_VLOG_UELOG(LogOwner, LogMass, Error, TEXT("%hs: input Composition doesn't contain %s"), __FUNCTION__, *GetNameSafe(Element.GetScriptStruct()));
					}
				}
				else
				{
					bIssuesFound = true;
					UE_VLOG_UELOG(LogOwner, LogMass, Error, TEXT("%hs: %s is not a valid %s type")
						, __FUNCTION__, *GetNameSafe(Element.GetScriptStruct()), *TElement::StaticStruct()->GetName());
				}
			}

			return bIssuesFound;
		}

		bool ValidateMakeInput(const TSharedRef<FMassEntityManager>& InEntityManager, const FMassArchetypeCompositionDescriptor& Composition
			, TConstArrayView<FInstancedStruct> InitialFragmentValues, TConstArrayView<FConstSharedStruct> ConstSharedFragments, TConstArrayView<FSharedStruct> SharedFragments)
		{
			const UObject* LogOwner = InEntityManager->GetOwner();
			bool bIssuesFound = CheckStructContainer<FMassFragment>(InitialFragmentValues, Composition.GetElementsBitSet(), LogOwner);
			bIssuesFound |= CheckStructContainer<FMassConstSharedFragment>(ConstSharedFragments, Composition.GetElementsBitSet(), LogOwner);
			bIssuesFound |= CheckStructContainer<FMassSharedFragment>(SharedFragments, Composition.GetElementsBitSet(), LogOwner);
			
			return !bIssuesFound;
		}
#endif // WITH_MASSENTITY_DEBUG
	}

//-----------------------------------------------------------------------------
// FEntityBuilder
//-----------------------------------------------------------------------------
FEntityBuilder::FEntityBuilder(FMassEntityManager& InEntityManager)
	: EntityManager(InEntityManager.AsShared())
	, bForceDeferred(UE::Mass::Tweakable::bForceDeferredEntityBuilderCommit)
{	
}

FEntityBuilder::FEntityBuilder(const TSharedRef<FMassEntityManager>& InEntityManager)
	: EntityManager(InEntityManager)
	, bForceDeferred(UE::Mass::Tweakable::bForceDeferredEntityBuilderCommit)
{
	
}

FEntityBuilder::FEntityBuilder(const FEntityBuilder& Other)
	: EntityManager(Other.EntityManager)
{
	*this = Other;
}

FEntityBuilder& FEntityBuilder::operator=(const FEntityBuilder& Other)
{
	if (testableEnsureMsgf(Other.IsValid(), TEXT("Copying invalid entity builder instances is not supported")))
	{
		// if we already have an EntityHandle reserved we might want to keep it - why reserve a handle again
		// soon, the reserved handle doesn't have an archetype associated with it?
		// We do need to release the handle if we're dealing with a different entity manager (unexpected in practice, but possible [for now])
		if (EntityManager != Other.EntityManager)
		{
			ConditionallyReleaseEntityHandle();
			EntityManager = Other.EntityManager;
		}
		// we also reset the handle if this builder has already committed its entity - the entity needs to 
		// be forgotten by this builder, it's "out in the wild" now and should be safe from accidental destruction.
		else if (State == EState::Committed)
		{
			EntityHandle.Reset();
		}
			
		Composition = Other.Composition;
		Fragments = Other.Fragments;
		SharedFragments = Other.SharedFragments;
		ConstSharedFragments = Other.ConstSharedFragments;

		State = Composition.IsEmpty() ? EState::Empty : EState::ReadyToCommit;
		bForceDeferred = Other.bForceDeferred;
	}

	return *this;
}

FEntityBuilder& FEntityBuilder::operator=(FEntityBuilder&& Other)
{
	if (testableEnsureMsgf(Other.IsValid(), TEXT("Copying invalid entity builder instances is not supported")))
	{
		// if we already have an EntityHandle reserved we might want to keep it - why reserve a handle again
		// soon, the reserved handle doesn't have an archetype associated with it?
		// We do need to release the handle if we're dealing with a different entity manager (unexpected in practice, but possible [for now])
		if (EntityManager != Other.EntityManager)
		{
			ConditionallyReleaseEntityHandle();
			EntityManager = MoveTemp(Other.EntityManager);
		}
		Fragments = MoveTemp(Other.Fragments);
		SharedFragments = MoveTemp(Other.SharedFragments);
		ConstSharedFragments = MoveTemp(Other.ConstSharedFragments);

		// the main point of the elaborated logic below is to avoid needlessly releasing reserved entities.
		if (HasReservedEntityHandle())
		{
			if (Other.HasReservedEntityHandle())
			{
				ConditionallyReleaseEntityHandle();
				EntityHandle = Other.EntityHandle;
			}
			State = (Other.State == EState::Committed)
				? EState::ReadyToCommit // we have a reserved entity at hand, we can Commit again
				: Other.State;
		}
		else
		{
			// we just take everything as is
			EntityHandle = Other.EntityHandle;
			State = Other.State;
		}
		bForceDeferred = Other.bForceDeferred;

		Other.EntityHandle.Reset();
		Other.State = EState::Invalid;
	}

	return *this;
}

FEntityBuilder::~FEntityBuilder()
{
	ConditionallyReleaseEntityHandle();
}

FEntityBuilder FEntityBuilder::Make(const TSharedRef<FMassEntityManager>& InEntityManager, const FMassArchetypeCompositionDescriptor& Composition
	, TConstArrayView<FInstancedStruct> InitialFragmentValues, TConstArrayView<FConstSharedStruct> ConstSharedFragments, TConstArrayView<FSharedStruct> SharedFragments)
{
	FEntityBuilder Builder(InEntityManager);

#if WITH_MASSENTITY_DEBUG
	if (Debug::bValidateEntityBuilderMakeInput)
	{
		ensureMsgf(Private::ValidateMakeInput(InEntityManager, Composition, InitialFragmentValues, ConstSharedFragments, SharedFragments)
			, TEXT("%hs: failed input validation. See log for details."), __FUNCTION__);
	}
#endif // WITH_MASSENTITY_DEBUG

	Builder.Composition = Composition;
	Builder.Fragments = InitialFragmentValues;
	Builder.SharedFragments = SharedFragments;
	Builder.ConstSharedFragments = ConstSharedFragments;

	return Builder;
}

FEntityBuilder FEntityBuilder::Make(const TSharedRef<FMassEntityManager>& InEntityManager
		, const FMassArchetypeCompositionDescriptor& Composition
		, TArray<FInstancedStruct>&& InitialFragmentValues
		, TArray<FConstSharedStruct>&& ConstSharedFragments
		, TArray<FSharedStruct>&& SharedFragments)
{
	FEntityBuilder Builder(InEntityManager);

#if WITH_MASSENTITY_DEBUG
	if (Debug::bValidateEntityBuilderMakeInput)
	{
		ensureMsgf(Private::ValidateMakeInput(InEntityManager, Composition, InitialFragmentValues, ConstSharedFragments, SharedFragments)
			, TEXT("%hs: failed input validation. See log for details."), __FUNCTION__);
	}
#endif // WITH_MASSENTITY_DEBUG

	Builder.Composition = Composition;
	Builder.Fragments = InitialFragmentValues;
	Builder.SharedFragments.Append(Forward<TArray<FSharedStruct>>(SharedFragments));
	Builder.ConstSharedFragments.Append(Forward<TArray<FConstSharedStruct>>(ConstSharedFragments));

	return Builder;
}

FMassEntityHandle FEntityBuilder::Commit()
{
	// @todo consider locking every builder instance to a single thread to prevent concurrent add/flush

	if (!testableEnsureMsgf(State != EState::Committed, TEXT("Trying to commit an already committed EntityBuilder. This request will be ignored.")))
	{
		return EntityHandle;
	}
	if (Composition.IsEmpty())
	{
		UE_VLOG_UELOG(EntityManager->GetOwner(), LogMass, Warning, TEXT("%hs: Attempting to commit while no composition has been configured."), __FUNCTION__);
		UE_CVLOG_UELOG(EntityHandle.IsValid(), EntityManager->GetOwner(), LogMass, Error, TEXT("Failing to commit while the entity handle has already been reserved."));
		return FMassEntityHandle();
	}

	CacheEntityHandle();
	CacheSharedFragmentValue();
	CacheArchetypeHandle();

	if (bForceDeferred || EntityManager->IsProcessing())
	{
		// we need to issue commands in this case
		EntityManager->Defer().PushCommand<FMassDeferredCreateCommand>(
					[ReservedEntityHandle = EntityHandle, SharedFragmentValues = CachedSharedFragmentValues
					, ArchetypeHandle = CachedArchetypeHandle, FragmentsCopy = Fragments
					, RelationsParams = RelationsParams
					, CompositionBitSet = Composition.GetElementsBitSet()](FMassEntityManager& Manager)
					{
						FEntityBuilder::CreateEntityImpl(Manager, ReservedEntityHandle, ArchetypeHandle
							, SharedFragmentValues, FragmentsCopy, RelationsParams, CompositionBitSet);
					});
	}
	else
	{
		// directly create the entity, right now
		CreateEntityImpl(*EntityManager, EntityHandle, CachedArchetypeHandle, CachedSharedFragmentValues
			, Fragments, RelationsParams, Composition.GetElementsBitSet());
	}

	State = EState::Committed;

	return EntityHandle;
}

void FEntityBuilder::CreateEntityImpl(FMassEntityManager& EntityManager, FMassEntityHandle EntityHandle, const FMassArchetypeHandle& ArchetypeHandle
	, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, const TConstArrayView<FInstancedStruct> Fragments, TConstArrayView<FPendingRelationParams> RelationsParams
	, const FMassElementBitSet& CompositionBitSet)
{
	// creating creation context to block observers from triggering until the
	// values are set with SetEntityFragmentValues. The lock gets auto-released at the end of the scope
	// or persists, if anyone else has a shared ptr to it
	TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager.GetOrMakeCreationContext();

	EntityManager.BuildEntity(EntityHandle, ArchetypeHandle, SharedFragmentValues);
	EntityManager.SetEntityFragmentValues(EntityHandle, Fragments);

	FMassElementBitSet SparseElements = CompositionBitSet & FMassElementBitSet::GetAllSparseElements();
	if (SparseElements.IsEmpty() == false)
	{
		EntityManager.AddSparseElementsToEntity(EntityHandle, SparseElements);
	}

	for (const FPendingRelationParams& Relation : RelationsParams)
	{
		switch (Relation.OtherEntityRole)
		{
		case ERelationRole::Subject:
			EntityManager.GetRelationManager().CreateRelationInstance(Relation.RelationTypeHandle, Relation.OtherEntity, EntityHandle);
			break;
		case ERelationRole::Object:
			EntityManager.GetRelationManager().CreateRelationInstance(Relation.RelationTypeHandle, EntityHandle, Relation.OtherEntity);
			break;
		default:
			ensureMsgf(false, TEXT("Unhandled value %s"), *LexToString(Relation.OtherEntityRole));
		}
	}
}

FMassEntityHandle FEntityBuilder::CommitAndReprepare()
{
	FMassEntityHandle CreatedEntity = Commit();
	Reprepare();
	return CreatedEntity;
}

void FEntityBuilder::Reprepare()
{
	if (ensureMsgf(State == EState::Committed, TEXT("Expected to be called only on Committed builders")))
	{
		EntityHandle.Reset();
		State = EState::ReadyToCommit;
	}
}

void FEntityBuilder::Reset(const bool bReleaseEntityHandleIfReserved)
{
	if (bReleaseEntityHandleIfReserved)
	{
		ConditionallyReleaseEntityHandle();
	}

	if (State != EState::Empty)
	{
		InvalidateCachedData();

		State = EState::Empty;

		Composition.Reset();
		Fragments.Reset();
		SharedFragments.Reset();
		ConstSharedFragments.Reset();
		RelationsParams.Reset();
	}
}

bool FEntityBuilder::SetReservedEntityHandle(const FMassEntityHandle ReservedEntityHandle)
{
	if (!ensureMsgf(ReservedEntityHandle.IsValid() && EntityManager->IsEntityReserved(ReservedEntityHandle), TEXT("Input ReservedEntityHandle is expected to be valid and represent a reserved entity")))
	{
		return false;
	}

	if (EntityHandle.IsValid() && EntityManager->IsEntityReserved(EntityHandle))
	{
		checkf(IsCommitted() == false, TEXT("We only expect to be here when the entity builder has not been `Committed` yet"));
		EntityManager->ReleaseReservedEntity(EntityHandle);
	}

	EntityHandle = ReservedEntityHandle;
	return true;
}

bool FEntityBuilder::AppendDataFromEntity(const FMassEntityHandle SourceEntityHandle, EIncludeSparseElements IncludeSparseElements)
{
	if (!testableEnsureMsgf(EntityManager->IsEntityActive(SourceEntityHandle)
		, TEXT("%hs expecting a valid, built entity as input"), __FUNCTION__))
	{
		return false;
	}
	if (State == EState::Empty)
	{
		// copying is significantly more efficient (no lookups for existing data) 
		return CopyDataFromEntity(SourceEntityHandle, IncludeSparseElements);
	}

	InvalidateCachedData();

	// handling sparse elements first in case we need to look for existing sparse fragment instances in Fragments,
	// before we add more stuff to it
	if (IncludeSparseElements == EIncludeSparseElements::Yes)
	{
		const FMassElementBitSet SparseElements = EntityManager->GetSparseElementsBitSetForEntity(SourceEntityHandle);
		const FMassElementBitSet SparseFragments = SparseElements & FMassElementBitSet::GetTypeBitArray(UE::Mass::EElementType::Fragment);
		for (FMassElementBitSet::FIndexIterator It = SparseFragments.GetIndexIterator(); It; ++It)
		{
			const UScriptStruct* SparseElementType = FMassElementBitSet::GetTypeAtIndex(*It);
			CA_ASSUME(SparseElementType);
			const FConstStructView FragmentInstance = EntityManager->GetSparseElementDataForEntity(SparseElementType, SourceEntityHandle);

			if (Composition.GetElementsBitSet().IsBitSet(*It))
			{
				// it's possible there's an instance in Fragments already
				FInstancedStruct* ExistingStruct = Fragments.FindByPredicate([&SparseElementType](const FInstancedStruct& Element)
					{
						return Element.GetScriptStruct() == SparseElementType;
					});
				if (ExistingStruct)
				{
					*ExistingStruct = FragmentInstance;
					continue;
				}
			}
			
			Fragments.Emplace(FragmentInstance);
		}
		Composition.GetElementsBitSet() += SparseElements;
	}

	const FMassArchetypeHandle ArchetypeHandle = EntityManager->GetArchetypeForEntity(SourceEntityHandle);
	const FMassArchetypeCompositionDescriptor& ArchetypeComposition = EntityManager->GetArchetypeComposition(ArchetypeHandle);

	Private::FEntityBuilderHelper::AppendFromEntity<FMassFragment>(*this, SourceEntityHandle, ArchetypeComposition);
	Private::FEntityBuilderHelper::AppendFromEntity<FMassSharedFragment>(*this, SourceEntityHandle, ArchetypeComposition);
	Private::FEntityBuilderHelper::AppendFromEntity<FMassConstSharedFragment>(*this, SourceEntityHandle, ArchetypeComposition);

	Composition.Append(ArchetypeComposition);

	State = Composition.IsEmpty() ? EState::Empty : EState::ReadyToCommit;

	return true;
}

bool FEntityBuilder::CopyDataFromEntity(const FMassEntityHandle SourceEntityHandle, EIncludeSparseElements IncludeSparseElements)
{
	if (!testableEnsureMsgf(EntityManager->IsEntityActive(SourceEntityHandle)
		, TEXT("%hs expecting a valid, built entity as input"), __FUNCTION__))
	{
		return false;
	}

	Reset(/*bReleaseEntityHandleIfReserved=*/false);
	
	const FMassArchetypeHandle ArchetypeHandle = EntityManager->GetArchetypeForEntity(SourceEntityHandle);
	const FMassArchetypeCompositionDescriptor& ArchetypeComposition = EntityManager->GetArchetypeComposition(ArchetypeHandle);

	Private::FEntityBuilderHelper::CopyFromEntity<FMassFragment>(*this, SourceEntityHandle, ArchetypeComposition);
	Private::FEntityBuilderHelper::CopyFromEntity<FMassSharedFragment>(*this, SourceEntityHandle, ArchetypeComposition);
	Private::FEntityBuilderHelper::CopyFromEntity<FMassConstSharedFragment>(*this, SourceEntityHandle, ArchetypeComposition);

	Composition = ArchetypeComposition;

	if (IncludeSparseElements == EIncludeSparseElements::Yes)
	{
		const FMassElementBitSet SparseElements = EntityManager->GetSparseElementsBitSetForEntity(SourceEntityHandle);
		const FMassElementBitSet SparseFragments = SparseElements & FMassElementBitSet::GetTypeBitArray(UE::Mass::EElementType::Fragment);
		for (FMassElementBitSet::FIndexIterator It = SparseFragments.GetIndexIterator(); It; ++It)
		{
			const UScriptStruct* SparseElementType = FMassElementBitSet::GetTypeAtIndex(*It);
			CA_ASSUME(SparseElementType);
			const FConstStructView FragmentInstance = EntityManager->GetSparseElementDataForEntity(SparseElementType, SourceEntityHandle);
			// we can add elements without checking if instances of the given type has already been added
			// Due to the nature of how sparse fragments are stored it's guaranteed the Fragments container doesn't contain any
			Fragments.Emplace(FragmentInstance);
		}
		Composition.GetElementsBitSet() += SparseElements;
	}

	State = Composition.IsEmpty() ? EState::Empty : EState::ReadyToCommit;

	return true;
}

FMassEntityHandle FEntityBuilder::GetEntityHandle() const
{
	CacheEntityHandle();
	return EntityHandle;
}

void FEntityBuilder::ConditionallyReleaseEntityHandle()
{
	if ((EntityHandle.IsValid() == true) && (State != EState::Committed))
	{
		EntityManager->ReleaseReservedEntity(EntityHandle);
	}

	EntityHandle.Reset();
}

void FEntityBuilder::CacheEntityHandle() const
{
	if (EntityHandle.IsValid() == false)
	{
		checkf(State != EState::Committed, TEXT("Reserving an entity while the builder has already committed. This should not happen. Indicates an error during builder copying from another instance."));
		EntityHandle = EntityManager->ReserveEntity();
	}
}

void FEntityBuilder::CacheArchetypeHandle()
{
	if (CachedArchetypeHandle.IsValid() == false)
	{
		CachedArchetypeHandle = EntityManager->CreateArchetype(Composition, ArchetypeCreationParams);
	}
}

void FEntityBuilder::InvalidateCachedData()
{
	CachedArchetypeHandle = {};
	CachedSharedFragmentValues.Reset();
}

FMassArchetypeHandle FEntityBuilder::GetArchetypeHandle()
{
	CacheArchetypeHandle();
	return CachedArchetypeHandle;
}

void FEntityBuilder::CacheSharedFragmentValue()
{
	if (CachedSharedFragmentValues.IsEmpty())
	{
		for (FInstancedStruct& SharedFragmentInstance : SharedFragments)
		{
			check(SharedFragmentInstance.IsValid());
			const FSharedStruct& SharedStruct = EntityManager->GetOrCreateSharedFragment(*SharedFragmentInstance.GetScriptStruct(), SharedFragmentInstance.GetMemory());
			CachedSharedFragmentValues.Add(SharedStruct);
		}
		for (FInstancedStruct& ConstSharedFragmentInstance : ConstSharedFragments)
		{
			check(ConstSharedFragmentInstance.IsValid());
			const FConstSharedStruct& ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(*ConstSharedFragmentInstance.GetScriptStruct(), ConstSharedFragmentInstance.GetMemory());
			CachedSharedFragmentValues.Add(ConstSharedStruct);
		}

		CachedSharedFragmentValues.Sort();
	}
}

template<typename T>
bool FEntityBuilder::HandleTypeInstance(const UScriptStruct* Type, TArray<FInstancedStruct>*& OutTargetContainer, bool& bOutAlreadyInComposition)
{
	if (UE::Mass::IsA<T>(Type))
	{
		auto& BitSet = Composition.GetElementsBitSet();
		const int32 TypeIndex = BitSet.GetTypeIndex(Type);
		OutTargetContainer = &GetInstancedStructContainerInternal<T>();
		if (BitSet.IsBitSet(TypeIndex) == false)
		{
			BitSet.AddAtIndex(TypeIndex);
		}
		else
		{
			bOutAlreadyInComposition = true;
		}
		return true;
	}
	return false;
}

template<typename TInstancedStruct>
FEntityBuilder& FEntityBuilder::AddInternal(TInstancedStruct&& ElementInstance)
{
	if (const UScriptStruct* Type = ElementInstance.GetScriptStruct())
	{
		TArray<FInstancedStruct>* TargetContainer = nullptr;

		bool bAlreadyInComposition = false;
		// the following chain will stop at the first successful HandleTypeInstance call.
		// This is by design, since we don't support there being any overlap between the types.
		HandleTypeInstance<FMassFragment>(Type, TargetContainer, bAlreadyInComposition)
		|| HandleTypeInstance<FMassSharedFragment>(Type, TargetContainer, bAlreadyInComposition)
		|| HandleTypeInstance<FMassConstSharedFragment>(Type, TargetContainer, bAlreadyInComposition);

		if (TargetContainer)
		{
			auto* FoundElement = bAlreadyInComposition
				? TargetContainer->FindByPredicate(FStructInstanceFindingPredicate{Type})
				: nullptr;

			// note that it's perfectly fine for FoundElement being null while bAlreadyInComposition == true.
			// This will happen if the element type has been added just as a type first (which only annotates the bitset)
			// and then an instance of the same type gets added - the bitset already contains the bit, but
			// the container doesn't hold an instance yet.
			if (FoundElement)
			{
				*FoundElement = Forward<TInstancedStruct>(ElementInstance);
			}
			else
			{
				TargetContainer->Add(Forward<TInstancedStruct>(ElementInstance));
			}
		}
	}

	return *this;
}

FEntityBuilder& FEntityBuilder::Add(const FInstancedStruct& ElementInstance)
{
	return AddInternal(ElementInstance);
}

FEntityBuilder& FEntityBuilder::Add(FInstancedStruct&& ElementInstance)
{
	return AddInternal(ElementInstance);
}

FEntityBuilder& FEntityBuilder::Add(TNotNull<const UScriptStruct*> ElementType)
{
	if (UE::Mass::IsA<FMassElement>(ElementType))
	{
		Composition.GetElementsBitSet().Add(ElementType);
		State = EState::ReadyToCommit;
		CachedArchetypeHandle = FMassArchetypeHandle();
	}
	
	return *this;
}

FEntityBuilder& FEntityBuilder::AddRelation(const FTypeHandle RelationTypeHandle, const FMassEntityHandle OtherEntity, const ERelationRole InputEntityRole)
{
	RelationsParams.Add({RelationTypeHandle, OtherEntity, InputEntityRole});
	return *this;
}

void FEntityBuilder::ForEachRelation(const TFunctionRef<bool(FPendingRelationParams&)>& Operator)
{
	for (int32 ElementIndex = RelationsParams.Num() - 1; ElementIndex >= 0; --ElementIndex)
	{
		if (Operator(RelationsParams[ElementIndex]) == false)
		{
			RelationsParams.RemoveAt(ElementIndex, EAllowShrinking::No);
		}
	}
}

bool FEntityBuilder::GetDefaultValueOfForceDeferredCommit()
{
	return Tweakable::bForceDeferredEntityBuilderCommit;
}

} // namespace UE::Mass