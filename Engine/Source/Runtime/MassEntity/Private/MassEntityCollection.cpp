// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityCollection.h"
#include "MassEntityUtils.h"

namespace UE::Mass
{
	//-----------------------------------------------------------------------------
	// FEntityCollection
	//-----------------------------------------------------------------------------
	FEntityCollection::FEntityCollection(FMassArchetypeEntityCollection&& InEntityCollection)
	{
		AppendCollection(Forward<FMassArchetypeEntityCollection>(InEntityCollection));
	}

	FEntityCollection::FEntityCollection(const FMassArchetypeEntityCollection& InEntityCollection)
	{
		AppendCollection(InEntityCollection);
	}

	FEntityCollection::FEntityCollection(const TConstArrayView<FMassEntityHandle> InEntityHandles)
		: EntityHandles(InEntityHandles)
	{	
	}

	FEntityCollection::FEntityCollection(const TConstArrayView<FMassEntityHandle> InEntityHandles, FMassArchetypeEntityCollection&& InEntityCollection)
		: EntityHandles(InEntityHandles)
	{
		CachedCollections.Add(Forward<FMassArchetypeEntityCollection>(InEntityCollection));
	}

	void FEntityCollection::ConditionallyUpdate(const FMassEntityManager& EntityManager) const
	{
		if (IsUpToDate() == false)
		{
			ensureMsgf(CachedCollections.IsEmpty(), TEXT("Failing IsUpToDate test should result in clearing out the cached collections"));
			Utils::CreateEntityCollections(EntityManager, EntityHandles
				, CollectionCreationDuplicatesHandling
				, CachedCollections);
		}
	}

	void FEntityCollection::AppendHandles(TConstArrayView<FMassEntityHandle> Handles)
	{
		EntityHandles.Append(Handles);
		MarkDirty();
		CollectionCreationDuplicatesHandling = FMassArchetypeEntityCollection::FoldDuplicates;
	}

	void FEntityCollection::AppendHandles(TConstArrayView<FMassEntityHandle> Handles, FMassArchetypeEntityCollection&& InEntityCollection)
	{
		const bool bWasEmpty = EntityHandles.IsEmpty();
		EntityHandles.Append(Handles);
		ConditionallyStoreCollection(bWasEmpty, Forward<FMassArchetypeEntityCollection>(InEntityCollection));
	}

	void FEntityCollection::AppendHandles(TArray<FMassEntityHandle>&& Handles)
	{
		EntityHandles.Append(Forward<TArray<FMassEntityHandle>>(Handles));
		MarkDirty();
		CollectionCreationDuplicatesHandling = FMassArchetypeEntityCollection::FoldDuplicates;
	}

	void FEntityCollection::AddHandle(FMassEntityHandle Handle)
	{
		EntityHandles.Emplace(Handle);
		MarkDirty();
		CollectionCreationDuplicatesHandling = FMassArchetypeEntityCollection::FoldDuplicates;
	}

	bool FEntityCollection::UpdateAndRemoveDuplicates(const FMassEntityManager& EntityManager, bool bForceOperation)
	{
		const int32 StartingHandlesCount = EntityHandles.Num();
		if (bForceOperation || CollectionCreationDuplicatesHandling == FMassArchetypeEntityCollection::FoldDuplicates)
		{
			CachedCollections.Reset();

			Utils::CreateEntityCollections(EntityManager, EntityHandles
				, FMassArchetypeEntityCollection::FoldDuplicates
				, CachedCollections);

			EntityHandles.Reset();
			for (const FMassArchetypeEntityCollection& Collection : CachedCollections)
			{
				Collection.ExportEntityHandles(EntityHandles);
			}

			CollectionCreationDuplicatesHandling = FMassArchetypeEntityCollection::NoDuplicates;

			ensureMsgf(EntityHandles.Num() <= StartingHandlesCount, TEXT("We don't expect to gain new handles"));
		}
		return StartingHandlesCount != EntityHandles.Num();
	}

	bool FEntityCollection::IsUpToDate() const
	{
		if (CachedCollections.IsEmpty() != EntityHandles.IsEmpty())
		{
			ensureMsgf(CachedCollections.IsEmpty(), TEXT("Unexpected development. We don't expect to have cached collections without any stored handles"));
			CachedCollections.Reset();
			return false;
		}

		for (const FMassArchetypeEntityCollection& Collection : GetCachedPerArchetypeCollections())
		{
			if (Collection.IsUpToDate() == false)
			{
				CachedCollections.Reset();
				return false;
			}
		}
		return true;
	}
} // namespace UE::Mass