// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManager.h"
#include "MassEntityManagerConstants.h"
#include "MassArchetypeData.h"
#include "MassCommandBuffer.h"
#include "MassEntityManagerStorage.h"
#include "Engine/World.h"
#include "UObject/UObjectIterator.h"
#include "VisualLogger/VisualLogger.h"
#include "MassExecutionContext.h"
#include "MassDebugger.h"
#include "Misc/Fork.h"
#include "Misc/CoreDelegates.h"
#include "Algo/Find.h"
#include "MassEntityUtils.h"
#include "MassEntityBuilder.h"
#include "MassTypeManager.h"
#include "MassProcessingContext.h"
#include "MassObserverNotificationTypes.h"

#define CHECK_SYNC_API() testableCheckf(IsProcessing() == false, TEXT("Synchronous API function %hs called during mass processing. Use asynchronous API instead."), __FUNCTION__)
#define CHECK_SYNC_API_RETURN(ReturnValue) testableCheckfReturn(IsProcessing() == false, ReturnValue , TEXT("Synchronous API function %hs called during mass processing. Use asynchronous API instead."), __FUNCTION__)
#define CHECK_ELEMENT(ElementType) checkf(UE::Mass::Private::IsElement(ElementType), TEXT("%hs: Only tags and fragments are considered 'elements', type provided: %s") \
		, __FUNCTION__ , *ElementType->GetName())
#define CHECK_SPARSE_ELEMENT(ElementType) checkf(UE::Mass::Private::IsSparseElement(ElementType), TEXT("%hs: Only sparse tags and fragments can be added or removed via this API, type provided: %s") \
		, __FUNCTION__ , *ElementType->GetName())

const FMassEntityHandle FMassEntityManager::InvalidEntity;

namespace UE::Mass::Private
{
	// note: this function doesn't set EntityHandle.SerialNumber
	void ConvertArchetypelessSubchunksIntoEntityHandles(FMassArchetypeEntityCollection::FConstEntityRangeArrayView Subchunks, TArray<FMassEntityHandle>& OutEntityHandles)
	{
		int32 TotalCount = 0;
		for (const FMassArchetypeEntityCollection::FArchetypeEntityRange& Subchunk : Subchunks)
		{
			TotalCount += Subchunk.Length;
		}

		int32 Index = OutEntityHandles.Num();
		OutEntityHandles.AddDefaulted(TotalCount);

		for (const FMassArchetypeEntityCollection::FArchetypeEntityRange& Subchunk : Subchunks)
		{
			for (int i = Subchunk.SubchunkStart; i < Subchunk.SubchunkStart + Subchunk.Length; ++i)
			{
				OutEntityHandles[Index++].Index = i;
			}
		}
	}

	bool IsElement(TNotNull<const UScriptStruct*> ElementType)
	{
		return UE::Mass::IsA<FMassFragment>(ElementType)
			|| UE::Mass::IsA<FMassTag>(ElementType)
			|| UE::Mass::IsA<FMassSharedFragment>(ElementType)
			|| UE::Mass::IsA<FMassConstSharedFragment>(ElementType);
	}

	bool IsSparseElement(TNotNull<const UScriptStruct*> ElementType)
	{
		return ElementType->IsChildOf(FMassSparseFragment::StaticStruct())
			|| ElementType->IsChildOf(FMassSparseTag::StaticStruct());
	}
} // namespace UE::Mass::Private

//-----------------------------------------------------------------------------
// FMassEntityManager
//-----------------------------------------------------------------------------
template void FMassEntityManager::SetEntityFragmentValues<FInstancedStruct>(FMassEntityHandle EntityHandle, TConstArrayView<FInstancedStruct> FragmentInstanceList);
template void FMassEntityManager::SetEntityFragmentValues<FStructView>(FMassEntityHandle EntityHandle, TConstArrayView<FStructView> FragmentInstanceList);
template void FMassEntityManager::SetEntityFragmentValues<FConstStructView>(FMassEntityHandle EntityHandle, TConstArrayView<FConstStructView> FragmentInstanceList);
template void FMassEntityManager::BatchSetEntityFragmentValues<FInstancedStruct>(const FMassArchetypeEntityCollection& SparseEntities, TConstArrayView<FInstancedStruct> FragmentInstanceList);
template void FMassEntityManager::BatchSetEntityFragmentValues<FStructView>(const FMassArchetypeEntityCollection& SparseEntities, TConstArrayView<FStructView> FragmentInstanceList);
template void FMassEntityManager::BatchSetEntityFragmentValues<FConstStructView>(const FMassArchetypeEntityCollection& SparseEntities, TConstArrayView<FConstStructView> FragmentInstanceList);
template void FMassEntityManager::BatchSetEntityFragmentValues<FInstancedStruct>(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TConstArrayView<FInstancedStruct> FragmentInstanceList);
template void FMassEntityManager::BatchSetEntityFragmentValues<FStructView>(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TConstArrayView<FStructView> FragmentInstanceList);
template void FMassEntityManager::BatchSetEntityFragmentValues<FConstStructView>(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TConstArrayView<FConstStructView> FragmentInstanceList);

FMassEntityManager::FMassEntityManager(UObject* InOwner)
	: ObserverManager(*this)
	, TypeManager(new UE::Mass::FTypeManager(*this))
	, RelationManager(*this)
	, Owner(InOwner)
{
#if WITH_MASSENTITY_DEBUG
	DebugName = InOwner ? (InOwner->GetName() + TEXT("_EntityManager")) : TEXT("Unset");
#endif
}

FMassEntityManager::~FMassEntityManager()
{
	if (InitializationState == EInitializationState::Initialized)
	{
		Deinitialize();
	}
}

void FMassEntityManager::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	SIZE_T MyExtraSize = (InitializationState == EInitializationState::Initialized ? GetEntityStorageInterface().GetAllocatedSize() : 0)
		+ FragmentHashToArchetypeMap.GetAllocatedSize()
		+ FragmentTypeToArchetypeMap.GetAllocatedSize();

	for (const TSharedPtr<FMassCommandBuffer>& CommandBuffer : DeferredCommandBuffers)
	{
		MyExtraSize += (CommandBuffer ? CommandBuffer->GetAllocatedSize() : 0);
	}
	
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(MyExtraSize);

	for (const auto& KVP : FragmentHashToArchetypeMap)
	{
		for (const TSharedPtr<FMassArchetypeData>& ArchetypePtr : KVP.Value)
		{
			CumulativeResourceSize.AddDedicatedSystemMemoryBytes(ArchetypePtr->GetAllocatedSize());
		}
	}
}

void FMassEntityManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (InitializationState == EInitializationState::Uninitialized)
	{
		UE_VLOG_UELOG(GetOwner(), LogMass, Log, TEXT("AddReferencedObjects called before Initialize call"));
		return;
	}

	if (InitializationState == EInitializationState::Deinitialized)
	{
		// this is AddReferencedObjects called after Deinitialize call, which means we don't want to retain any object refs
		// since this FMassEntityManager instance is going away even if it's kept alive by some stored shared refs at the moment.
		return;
	}

	for (FConstSharedStruct& Struct : ConstSharedFragmentsContainer.GetAllInstances())
	{
		Struct.AddStructReferencedObjects(Collector);
	}

	for (FSharedStruct& Struct : SharedFragmentsContainer.GetAllInstances())
	{
		Struct.AddStructReferencedObjects(Collector);
	}
 
	const class UScriptStruct* ScriptStruct = FMassObserverManager::StaticStruct();
	TWeakObjectPtr<const UScriptStruct> ScriptStructPtr{ScriptStruct};
	Collector.AddReferencedObjects(ScriptStructPtr, &ObserverManager);
}

void FMassEntityManager::Initialize()
{
	FMassEntityManagerStorageInitParams InitializationParams;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InitializationParams.Emplace<FMassEntityManager_InitParams_SingleThreaded>();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Initialize(InitializationParams);
}

namespace UE::Mass::Private
{
	struct FEntityStorageInitializer
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		void operator()(const FMassEntityManager_InitParams_SingleThreaded& Params)

		{
			EntityStorage->Emplace<UE::Mass::FSingleThreadedEntityStorage>();
			EntityStorage->Get<FSingleThreadedEntityStorage>().Initialize(Params);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		void operator()(const FMassEntityManager_InitParams_Concurrent& Params)
		{
#if WITH_MASS_CONCURRENT_RESERVE
			EntityStorage->Emplace<UE::Mass::FConcurrentEntityStorage>();
			EntityStorage->Get<UE::Mass::FConcurrentEntityStorage>().Initialize(Params);
#else
			checkf(false, TEXT("Mass does not support this storage backend"));
#endif
		}
		
		FMassEntityManager::FEntityStorageContainerType* EntityStorage = nullptr;
	};
} // namespace UE::Mass::Private

void FMassEntityManager::Initialize(const FMassEntityManagerStorageInitParams& InitializationParams)
{
	if (InitializationState == EInitializationState::Initialized)
	{
		UE_VLOG_UELOG(GetOwner(), LogMass, Log, TEXT("Calling %hs on already initialized entity manager owned by %s")
			, __FUNCTION__, *GetNameSafe(Owner.Get()));
		return;
	}

	LLM_SCOPE_BYNAME(TEXT("Mass/EntityManager"));

	Visit(UE::Mass::Private::FEntityStorageInitializer{&EntityStorage}, InitializationParams);
#if WITH_MASSENTITY_DEBUG
	DebugEntityStoragePtr = &DebugGetEntityStorageInterface();
#endif // WITH_MASSENTITY_DEBUG

	for (TSharedPtr<FMassCommandBuffer>& CommandBuffer : DeferredCommandBuffers)
	{
		CommandBuffer = MakeShareable(new FMassCommandBuffer());
	}

	// if we get forked we need to update the CurrentThreadID of things like command buffers
	// and potentially active creation context
	if (FForkProcessHelper::IsForkRequested())
	{
		OnPostForkHandle = FCoreDelegates::OnPostFork.AddSP(AsShared(), &FMassEntityManager::OnPostFork);
	}

	// creating these bitset instances to populate respective bitset types' StructTrackers
	FMassFragmentBitSet Fragments;
	FMassTagBitSet Tags;
	FMassChunkFragmentBitSet ChunkFragments;
	FMassSharedFragmentBitSet LocalSharedFragments;
	FMassConstSharedFragmentBitSet LocalConstSharedFragments;

	for (TObjectIterator<UScriptStruct> StructIt; StructIt; ++StructIt)
	{
		if (UE::Mass::IsA<FMassFragment>(*StructIt))
		{
			if (*StructIt != FMassFragment::StaticStruct())
			{
				Fragments.Add(*StructIt);
			}
		}
		else if (UE::Mass::IsA<FMassTag>(*StructIt))
		{
			if (*StructIt != FMassTag::StaticStruct())
			{
				Tags.Add(*StructIt);
			}
		}
		else if (UE::Mass::IsA<FMassChunkFragment>(*StructIt))
		{
			if (*StructIt != FMassChunkFragment::StaticStruct())
			{
				ChunkFragments.Add(*StructIt);
			}
		}
		else if (UE::Mass::IsA<FMassSharedFragment>(*StructIt))
		{
			if (*StructIt != FMassSharedFragment::StaticStruct())
			{
				LocalSharedFragments.Add(*StructIt);
			}
		}
		else if (UE::Mass::IsA<FMassConstSharedFragment>(*StructIt))
		{
			if (*StructIt != FMassConstSharedFragment::StaticStruct())
			{
				LocalConstSharedFragments.Add(*StructIt);
			}
		}
	}

	InitializationState = EInitializationState::Initialized;
	bFirstCommandFlush = true;

#if WITH_MASSENTITY_DEBUG
	RequirementAccessDetector.Initialize();
	FMassDebugger::RegisterEntityManager(*this);
#endif // WITH_MASSENTITY_DEBUG
}

void FMassEntityManager::PostInitialize()
{
	ensureMsgf(InitializationState == EInitializationState::Initialized,
		TEXT("This needs to be done after all the subsystems have been initialized since some processors might want to access"
			" them during processors' initialization"));

	TypeManager->RegisterBuiltInTypes();
	// now hook-in all relation observers
	// note that we're doing it only after RegisterBuiltInTypes is done, as opposed to doing it on
	// every type as it gets added, because there are ways to override the traits of built-in types.
	// Once RegisterBuiltInTypes is done the traits are set in stone, and we can handle the types.
	for (UE::Mass::FTypeManager::FTypeInfoConstIterator It = TypeManager->MakeIterator(); It; ++It)
	{
		OnNewTypeRegistered(It->Key);
	}

	ObserverManager.Initialize();
}

void FMassEntityManager::Deinitialize()
{
	if (InitializationState == EInitializationState::Initialized)
	{
		FCoreDelegates::OnPostFork.Remove(OnPostForkHandle);

		// closing down so no point in actually flushing commands, but need to clean them up to avoid warnings on destruction
		for (TSharedPtr<FMassCommandBuffer>& CommandBuffer : DeferredCommandBuffers)
		{
			if (CommandBuffer)
			{
				CommandBuffer->CleanUp();
			}
		}

#if WITH_MASSENTITY_DEBUG
		FMassDebugger::UnregisterEntityManager(*this);
#endif // WITH_MASSENTITY_DEBUG

		EntityStorage.Emplace<FEmptyVariantState>();

		ObserverManager.DeInitialize();

		SparseElementsStorage.Deinitialize();

		InitializationState = EInitializationState::Deinitialized;
	}
	else
	{
		UE_VLOG_UELOG(GetOwner(), LogMass, Log, TEXT("Calling %hs on already deinitialized entity manager owned by %s")
			, __FUNCTION__, *GetNameSafe(Owner.Get()));
	}
}

void FMassEntityManager::OnPostFork(EForkProcessRole Role)
{
	if (Role == EForkProcessRole::Child)
	{
		LLM_SCOPE_BYNAME(TEXT("Mass/EntityManager"));
		for (TSharedPtr<FMassCommandBuffer>& CommandBuffer : DeferredCommandBuffers)
		{
			if (CommandBuffer)
			{
				CommandBuffer->ForceUpdateCurrentThreadID();
			}
			else
			{
				CommandBuffer = MakeShareable(new FMassCommandBuffer());
			}
		}

		ObserverManager.OnPostFork(Role);
	}
}

FMassArchetypeHandle FMassEntityManager::CreateArchetype(TConstArrayView<const UScriptStruct*> FragmentsAndTagsList, const FMassArchetypeCreationParams& CreationParams)
{
	FMassArchetypeCompositionDescriptor Composition;
	InternalAppendFragmentsAndTagsToArchetypeCompositionDescriptor(Composition, FragmentsAndTagsList);
	return CreateArchetype(Composition, CreationParams);
}

FMassArchetypeHandle FMassEntityManager::CreateArchetype(FMassArchetypeHandle SourceArchetype, TConstArrayView<const UScriptStruct*> FragmentsAndTagsList)
{
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype); 
	return CreateArchetype(SourceArchetype, FragmentsAndTagsList, FMassArchetypeCreationParams(ArchetypeData));
}

FMassArchetypeHandle FMassEntityManager::CreateArchetype(FMassArchetypeHandle SourceArchetype,
	TConstArrayView<const UScriptStruct*> FragmentsAndTagsList, const FMassArchetypeCreationParams& CreationParams)
{
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype);
	FMassArchetypeCompositionDescriptor Composition = FMassArchetypeCompositionDescriptor(ArchetypeData.GetCompositionBitSet());
	InternalAppendFragmentsAndTagsToArchetypeCompositionDescriptor(Composition, FragmentsAndTagsList);
	return CreateArchetype(Composition, CreationParams);
}

FMassArchetypeHandle FMassEntityManager::CreateArchetype(const TSharedPtr<FMassArchetypeData>& SourceArchetype, const FMassFragmentBitSet& AddedFragments)
{
	return CreateArchetype(SourceArchetype, AddedFragments, FMassArchetypeCreationParams(*SourceArchetype));
}

FMassArchetypeHandle FMassEntityManager::CreateArchetype(const TSharedPtr<FMassArchetypeData>& SourceArchetype, const FMassFragmentBitSet& AddedFragments, const FMassArchetypeCreationParams& CreationParams)
{
	check(SourceArchetype.IsValid());
	checkf(AddedFragments.IsEmpty() == false, TEXT("%hs Adding an empty fragment list to an archetype is not supported."), __FUNCTION__);

	FMassArchetypeCompositionDescriptor Composition(SourceArchetype->GetCompositionBitSet() + AddedFragments);
	return CreateArchetype(Composition, CreationParams);
}

FMassArchetypeHandle FMassEntityManager::GetOrCreateSuitableArchetype(const FMassArchetypeHandle& ArchetypeHandle
	, const FMassSharedFragmentBitSet& SharedFragmentBitSet
	, const FMassConstSharedFragmentBitSet& ConstSharedFragmentBitSet
	, const FMassArchetypeCreationParams& CreationParams)
{
	return GetOrCreateSuitableArchetype(ArchetypeHandle, FMassElementBitSet(SharedFragmentBitSet) + ConstSharedFragmentBitSet, CreationParams);
}

FMassArchetypeHandle FMassEntityManager::GetOrCreateSuitableArchetype(const FMassArchetypeHandle& ArchetypeHandle
	, const FMassElementBitSet& SharedElementsBitSet, const FMassArchetypeCreationParams& CreationParams)
{
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	if (UE::Mass::DoContainEquivalentSharedFragments(ArchetypeData.GetCompositionBitSet(), SharedElementsBitSet) == false)
	{
		checkf(UE::Mass::DoesContainOnlySharedFragments(SharedElementsBitSet)
			, TEXT("SharedElementsBitSet is expected to contain only shared fragments (const and non-const)"));

		FMassArchetypeCompositionDescriptor NewDescriptor(ArchetypeData.GetCompositionBitSet());
		NewDescriptor.SetAllSharedElements(SharedElementsBitSet);
		return CreateArchetype(NewDescriptor, CreationParams);
	}
	return ArchetypeHandle;
}

FMassArchetypeHandle FMassEntityManager::CreateArchetype(const FMassArchetypeCompositionDescriptor& Composition, const FMassArchetypeCreationParams& CreationParams)
{
	return CreateArchetype(Composition.GetElementsBitSet(), CreationParams);
}

FMassArchetypeHandle FMassEntityManager::CreateArchetype(const FMassElementBitSet& ElementBitSet, const FMassArchetypeCreationParams& CreationParams)
{
	LLM_SCOPE_BYNAME(TEXT("Mass/EntityManager"));
	const FMassElementBitSet NonSparseComposition = ElementBitSet.AndNot(FMassElementBitSet::GetAllSparseElements());
	const uint32 TypeHash = HashCombine(GetTypeHash(NonSparseComposition), GetTypeHash(UE::Mass::FArchetypeGroups()));

	TArray<TSharedPtr<FMassArchetypeData>>& HashRow = FragmentHashToArchetypeMap.FindOrAdd(TypeHash);

	TSharedPtr<FMassArchetypeData> ArchetypeDataPtr;
	for (const TSharedPtr<FMassArchetypeData>& Ptr : HashRow)
	{
		if (Ptr->IsEquivalent(NonSparseComposition, /*Groups=*/{}))
		{
#if WITH_MASSENTITY_DEBUG
			// Keep track of all names for this archetype.
			if (!CreationParams.DebugName.IsNone())
			{
				Ptr->AddUniqueDebugName(CreationParams.DebugName);
			}
#endif // WITH_MASSENTITY_DEBUG
			if (CreationParams.ChunkMemorySize > 0 && CreationParams.ChunkMemorySize != Ptr->GetChunkAllocSize())
			{
				UE_LOGF(LogMass, Warning, "Reusing existing Archetype, but the requested ChunkMemorySize is different. Requested %d, existing: %llu"
					, CreationParams.ChunkMemorySize, Ptr->GetChunkAllocSize());
			}
			ArchetypeDataPtr = Ptr;
			break;
		}
	}

	if (!ArchetypeDataPtr.IsValid())
	{
		// Important to pre-increment the version as the queries will use this value to do incremental updates
		++ArchetypeDataVersion;

		// Create a new archetype
		FMassArchetypeData* NewArchetype = new FMassArchetypeData(CreationParams);
		NewArchetype->Initialize(*this, NonSparseComposition, ArchetypeDataVersion, SparseElementsStorage);
		ArchetypeDataPtr = HashRow.Add_GetRef(MakeShareable(NewArchetype));
		AllArchetypes.Add(ArchetypeDataPtr);
		ensure(AllArchetypes.Num() == ArchetypeDataVersion);

		for (const FMassArchetypeFragmentConfig& FragmentConfig : NewArchetype->GetFragmentConfigs())
		{
			checkSlow(FragmentConfig.FragmentType)
			FragmentTypeToArchetypeMap.FindOrAdd(FragmentConfig.FragmentType).Add(ArchetypeDataPtr);
		}

		OnNewArchetypeEvent.Broadcast(FMassArchetypeHandle(ArchetypeDataPtr));
		UE_TRACE_MASS_ARCHETYPE_CREATED(ArchetypeDataPtr)
	}

	return FMassArchetypeHelper::ArchetypeHandleFromData(ArchetypeDataPtr);
}

FMassArchetypeHandle FMassEntityManager::InternalCreateSimilarArchetype(const TSharedPtr<FMassArchetypeData>& SourceArchetype, FMassElementBitSet&& NewCompositionBitSet)
{
	checkSlow(SourceArchetype.IsValid());
	const FMassArchetypeData& SourceArchetypeRef = *SourceArchetype.Get();

	return InternalCreateSimilarArchetype(SourceArchetypeRef, MoveTemp(NewCompositionBitSet), SourceArchetypeRef.GetGroups());
}

FMassArchetypeHandle FMassEntityManager::InternalCreateSimilarArchetype(const TSharedPtr<FMassArchetypeData>& SourceArchetype, const UE::Mass::FArchetypeGroups& GroupsOverride)
{
	checkSlow(SourceArchetype.IsValid());
	const FMassArchetypeData& SourceArchetypeRef = *SourceArchetype.Get();
	return InternalCreateSimilarArchetype(SourceArchetypeRef, FMassArchetypeCompositionDescriptor(SourceArchetype->GetCompositionBitSet()), GroupsOverride);
}

FMassArchetypeHandle FMassEntityManager::InternalCreateSimilarArchetype(const FMassArchetypeData& SourceArchetypeRef, FMassArchetypeCompositionDescriptor&& NewComposition, const UE::Mass::FArchetypeGroups& Groups)
{
	LLM_SCOPE_BYNAME(TEXT("Mass/EntityManager"));
	// we require Groups to be already shrunk. Shrinking is required to remove any trailing, invalid group IDs that would
	// be there if IDs were added and removed to this specific Groups container instance
	checkf(Groups.IsShrunk(), TEXT("A group container with invalid trailing IDs has been passed to archetype creation - this is not expected and will cause issues. Make sure to Shrink your Groups before passing to %hs"), __FUNCTION__);

	FMassElementBitSet NonSparseComposition = NewComposition.GetElementsBitSet().AndNot(FMassElementBitSet::GetAllSparseElements());
	const uint32 TypeHash = HashCombine(GetTypeHash(NonSparseComposition), GetTypeHash(UE::Mass::FArchetypeGroups()));

	TArray<TSharedPtr<FMassArchetypeData>>& HashRow = FragmentHashToArchetypeMap.FindOrAdd(TypeHash);

	TSharedPtr<FMassArchetypeData> ArchetypeDataPtr;
	for (const TSharedPtr<FMassArchetypeData>& Ptr : HashRow)
	{
		if (Ptr->IsEquivalent(NonSparseComposition, Groups))
		{
			ArchetypeDataPtr = Ptr;
			break;
		}
	}

	if (!ArchetypeDataPtr.IsValid())
	{
		// Important to pre-increment the version as the queries will use this value to do incremental updates
		++ArchetypeDataVersion;

		// Create a new archetype
		FMassArchetypeData* NewArchetype = new FMassArchetypeData(FMassArchetypeCreationParams(SourceArchetypeRef));
		NewArchetype->InitializeWithSimilar(*this, SourceArchetypeRef, MoveTemp(NonSparseComposition), Groups, ArchetypeDataVersion, SparseElementsStorage);
		NewArchetype->CopyDebugNamesFrom(SourceArchetypeRef);

		ArchetypeDataPtr = HashRow.Add_GetRef(MakeShareable(NewArchetype));
		AllArchetypes.Add(ArchetypeDataPtr);
		ensure(AllArchetypes.Num() == ArchetypeDataVersion);

		for (const FMassArchetypeFragmentConfig& FragmentConfig : NewArchetype->GetFragmentConfigs())
		{
			checkSlow(FragmentConfig.FragmentType)
			FragmentTypeToArchetypeMap.FindOrAdd(FragmentConfig.FragmentType).Add(ArchetypeDataPtr);
		}

		OnNewArchetypeEvent.Broadcast(FMassArchetypeHandle(ArchetypeDataPtr));
		UE_TRACE_MASS_ARCHETYPE_CREATED(ArchetypeDataPtr)
	}

	return FMassArchetypeHelper::ArchetypeHandleFromData(ArchetypeDataPtr);
}

void FMassEntityManager::InternalAppendFragmentsAndTagsToArchetypeCompositionDescriptor(
	FMassArchetypeCompositionDescriptor& InOutComposition, TConstArrayView<const UScriptStruct*> FragmentsAndTagsList) const
{
	for (const UScriptStruct* Type : FragmentsAndTagsList)
	{
		if (UE::Mass::IsA<FMassFragment>(Type)
			|| UE::Mass::IsA<FMassTag>(Type)
			|| UE::Mass::IsA<FMassChunkFragment>(Type))
		{
			InOutComposition.Add(Type);
		}
		else
		{
			UE_LOGF(LogMass, Warning, "%s: %ls is not a valid fragment nor tag type. Ignoring."
				, __FUNCTION__, *GetNameSafe(Type));
		}
	}
}

FMassArchetypeHandle FMassEntityManager::GetArchetypeForEntity(FMassEntityHandle Entity) const
{
	if (IsEntityValid(Entity))
	{
		return FMassArchetypeHelper::ArchetypeHandleFromData(GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index));
	}
	return FMassArchetypeHandle();
}

FMassArchetypeHandle FMassEntityManager::GetArchetypeForEntityUnsafe(FMassEntityHandle Entity) const
{
	check(GetEntityStorageInterface().IsValidIndex(Entity.Index));
	return FMassArchetypeHelper::ArchetypeHandleFromData(GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index));
}

void FMassEntityManager::GetMatchingArchetypes(const FMassFragmentRequirements& Requirements, TArray<FMassArchetypeHandle>& OutValidArchetypes) const
{
	GetMatchingArchetypes(Requirements, OutValidArchetypes, 0);
}

void FMassEntityManager::ForEachArchetypeElementType(const FMassArchetypeHandle& ArchetypeHandle, TFunctionRef< bool(const UScriptStruct*)> Function)
{
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	ArchetypeData.GetCompositionBitSet().ExportTypes(Function);
}

void FMassEntityManager::ForEachArchetypeFragmentType(const FMassArchetypeHandle& ArchetypeHandle, TFunction< void(const UScriptStruct*)> Function)
{
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	ArchetypeData.ForEachFragmentType(Function);
}

void FMassEntityManager::ForEachArchetypeChunkEntityList(const FMassArchetypeHandle& ArchetypeHandle, TFunctionRef<void(TConstArrayView<FMassEntityHandle>)> Function)
{
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	ArchetypeData.ListEntitiesForEachChunk(Function);
}

void FMassEntityManager::DoEntityCompaction(const double TimeAllowed)
{
	int32 TotalEntitiesMoved = 0;
	const double TimeAllowedEnd = FPlatformTime::Seconds() + TimeAllowed;

	bool bReachedTimeLimit = false;
	for (const auto& KVP : FragmentHashToArchetypeMap)
	{
		for (const TSharedPtr<FMassArchetypeData>& ArchetypePtr : KVP.Value)
		{
			const double TimeAllowedLeft = TimeAllowedEnd - FPlatformTime::Seconds();
			bReachedTimeLimit = TimeAllowedLeft <= 0.0;
			if (bReachedTimeLimit)
			{
 				break;
			}
			TotalEntitiesMoved += ArchetypePtr->CompactEntities(TimeAllowedLeft);
		}
		if (bReachedTimeLimit)
		{
			break;
		}
	}

	UE_CVLOG(TotalEntitiesMoved, GetOwner(), LogMass, Verbose, TEXT("Entity Compaction: moved %d entities"), TotalEntitiesMoved);
}

FMassEntityHandle FMassEntityManager::CreateEntity(const FMassArchetypeHandle& ArchetypeHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues)
{
	CHECK_SYNC_API_RETURN(return {});
	check(ArchetypeHandle.IsValid());
	
	MASS_BREAKPOINT(UE::Mass::Debug::FBreakpoint::CheckCreateEntityBreakpoints(ArchetypeHandle));

	const FMassEntityHandle Entity = ReserveEntity();
	InternalBuildEntity(Entity
		, GetOrCreateSuitableArchetype(ArchetypeHandle, SharedFragmentValues.GetBitSet())
		, SharedFragmentValues);

	return Entity;
}

FMassEntityHandle FMassEntityManager::CreateEntity(TConstArrayView<FInstancedStruct> FragmentInstanceList, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, const FMassArchetypeCreationParams& CreationParams)
{
	CHECK_SYNC_API_RETURN(return {});
	check(FragmentInstanceList.Num() > 0);

	const FMassArchetypeHandle& ArchetypeHandle = CreateArchetype(FMassArchetypeCompositionDescriptor(FragmentInstanceList,
		FMassTagBitSet(), FMassChunkFragmentBitSet(), FMassSharedFragmentBitSet(), FMassConstSharedFragmentBitSet()), CreationParams);
	check(ArchetypeHandle.IsValid());

	const FMassEntityHandle Entity = ReserveEntity();

	// Using a creation context to prevent InternalBuildEntity from notifying observers before we set fragments data
	const TSharedRef<FEntityCreationContext> CreationContext = ObserverManager.GetOrMakeCreationContext();

	InternalBuildEntity(Entity, ArchetypeHandle, SharedFragmentValues);

	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
	check(CurrentArchetype);
	CurrentArchetype->SetFragmentsData(Entity, FragmentInstanceList);

	return Entity;
}

FMassEntityHandle FMassEntityManager::CreateEntityAsyncUnsafe(const FMassArchetypeHandle& ArchetypeHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues)
{
	check(ArchetypeHandle.IsValid());

	MASS_BREAKPOINT(UE::Mass::Debug::FBreakpoint::CheckCreateEntityBreakpoints(ArchetypeHandle));

	const FMassEntityHandle Entity = ReserveEntity();
	InternalBuildEntity(Entity
		, GetOrCreateSuitableArchetype(ArchetypeHandle, SharedFragmentValues.GetBitSet())
		, SharedFragmentValues);
	return Entity;
}

void FMassEntityManager::BuildEntityAsyncUnsafe(const FMassArchetypeHandle& ArchetypeHandle, const FMassEntityHandle& ReservedEntityHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues)
{
	check(ArchetypeHandle.IsValid());

	MASS_BREAKPOINT(UE::Mass::Debug::FBreakpoint::CheckCreateEntityBreakpoints(ArchetypeHandle));

	InternalBuildEntity(ReservedEntityHandle
		, GetOrCreateSuitableArchetype(ArchetypeHandle, SharedFragmentValues.GetBitSet())
		, SharedFragmentValues);
}

FMassEntityHandle FMassEntityManager::ReserveEntity()
{
	FMassEntityHandle Result = GetEntityStorageInterface().AcquireOne();

	return Result;
}

void FMassEntityManager::ReleaseReservedEntity(FMassEntityHandle Entity)
{
	checkf(!IsEntityBuilt(Entity), TEXT("Entity is already built, use DestroyEntity() instead"));

	InternalReleaseEntity(Entity);
}

void FMassEntityManager::BuildEntity(FMassEntityHandle Entity, const FMassArchetypeHandle& ArchetypeHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues)
{
	CHECK_SYNC_API();
	checkf(!IsEntityBuilt(Entity), TEXT("Expecting an entity that is not already built"));
	check(ArchetypeHandle.IsValid());

	InternalBuildEntity(Entity, ArchetypeHandle, SharedFragmentValues);
}

void FMassEntityManager::BuildEntity(FMassEntityHandle Entity, TConstArrayView<FInstancedStruct> FragmentInstanceList, const FMassArchetypeSharedFragmentValues& SharedFragmentValues)
{
	CHECK_SYNC_API();
	check(FragmentInstanceList.Num() > 0);
	checkf(!IsEntityBuilt(Entity), TEXT("Expecting an entity that is not already built"));

	checkf(SharedFragmentValues.IsSorted(), TEXT("Expecting shared fragment values to be previously sorted"));
	FMassArchetypeCompositionDescriptor Composition(FragmentInstanceList, FMassTagBitSet(), FMassChunkFragmentBitSet(), FMassSharedFragmentBitSet(), FMassConstSharedFragmentBitSet());
	for (const FConstSharedStruct& SharedFragment : SharedFragmentValues.GetConstSharedFragments())
	{
		Composition.Add(SharedFragment.GetScriptStruct());
	}
	for (const FSharedStruct& SharedFragment : SharedFragmentValues.GetSharedFragments())
	{
		Composition.Add(SharedFragment.GetScriptStruct());
	}

	const FMassArchetypeHandle& ArchetypeHandle = CreateArchetype(Composition);
	check(ArchetypeHandle.IsValid());

	// Using a creation context to prevent InternalBuildEntity from notifying observers before we set fragments data
	const TSharedRef<FEntityCreationContext> CreationContext = ObserverManager.GetOrMakeCreationContext();

	InternalBuildEntity(Entity, ArchetypeHandle, SharedFragmentValues);

	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
	check(CurrentArchetype);
	CurrentArchetype->SetFragmentsData(Entity, FragmentInstanceList);
}

TConstArrayView<FMassEntityHandle> FMassEntityManager::BatchReserveEntities(const int32 Count, TArray<FMassEntityHandle>& InOutEntities)
{
	const int32 Index = InOutEntities.Num();
	const int32 NumAdded = GetEntityStorageInterface().Acquire(Count, InOutEntities);
	ensureMsgf(NumAdded == Count, TEXT("Failed to reserve %d entities, was able to only reserve %d"), Count, NumAdded);

	return MakeArrayView(InOutEntities.GetData() + Index, NumAdded);
}

int32 FMassEntityManager::BatchReserveEntities(TArrayView<FMassEntityHandle> InOutEntities)
{
	return GetEntityStorageInterface().Acquire(InOutEntities);
}

TSharedRef<FMassEntityManager::FEntityCreationContext> FMassEntityManager::BatchBuildEntities(
	const FMassArchetypeEntityCollectionWithPayload& EncodedEntitiesWithPayload
	, const FMassFragmentBitSet& FragmentsAffected
	, const FMassArchetypeSharedFragmentValues& SharedFragmentValues
	, const FMassArchetypeCreationParams& CreationParams)
{
	CHECK_SYNC_API_RETURN(return FMassObserverManager::FCreationContext::DebugCreateDummyCreationContext());
	check(SharedFragmentValues.IsSorted());

	FMassArchetypeCompositionDescriptor Composition(FragmentsAffected, FMassTagBitSet(), FMassChunkFragmentBitSet(), FMassSharedFragmentBitSet(), FMassConstSharedFragmentBitSet());
	for (const FConstSharedStruct& SharedFragment : SharedFragmentValues.GetConstSharedFragments())
	{
		Composition.Add(SharedFragment.GetScriptStruct());
	}
	for (const FSharedStruct& SharedFragment : SharedFragmentValues.GetSharedFragments())
	{
		Composition.Add(SharedFragment.GetScriptStruct());
	}

	return BatchBuildEntities(EncodedEntitiesWithPayload, MoveTemp(Composition), SharedFragmentValues, CreationParams);
}

TSharedRef<FMassEntityManager::FEntityCreationContext> FMassEntityManager::BatchBuildEntities(const FMassArchetypeEntityCollectionWithPayload& EncodedEntitiesWithPayload
	, const FMassArchetypeCompositionDescriptor& Composition
	, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, const FMassArchetypeCreationParams& CreationParams)
{
	return BatchBuildEntities(EncodedEntitiesWithPayload, Composition.GetElementsBitSet(), SharedFragmentValues, CreationParams);
}

TSharedRef<FMassEntityManager::FEntityCreationContext> FMassEntityManager::BatchBuildEntities(
	const FMassArchetypeEntityCollectionWithPayload& EncodedEntitiesWithPayload
	, const FMassElementBitSet& Composition
	, const FMassArchetypeSharedFragmentValues& SharedFragmentValues
	, const FMassArchetypeCreationParams& CreationParams)
{
	CHECK_SYNC_API_RETURN(return FMassObserverManager::FCreationContext::DebugCreateDummyCreationContext());

	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchBuildEntities);

	FMassArchetypeEntityCollection::FEntityRangeArray TargetArchetypeEntityRanges;

	// "built" entities case, this is verified during FMassArchetypeEntityCollectionWithPayload construction
	FMassArchetypeHandle TargetArchetypeHandle = CreateArchetype(Composition, CreationParams);
	check(TargetArchetypeHandle.IsValid());

	// there are some extra steps in creating EncodedEntities from the original given entity handles and then back
	// to handles here, but this way we're consistent in how stuff is handled, and there are some slight benefits 
	// to having entities ordered by their index (like accessing the Entities data below).
	TArray<FMassEntityHandle> EntityHandles;
	UE::Mass::Private::ConvertArchetypelessSubchunksIntoEntityHandles(EncodedEntitiesWithPayload.GetEntityCollection().GetRanges(), EntityHandles);

	// since the handles encoded via FMassArchetypeEntityCollectionWithPayload miss the SerialNumber we need to update it
	// before passing over the new archetype. Thankfully we need to iterate over all the entity handles anyway
	// to update the manager's information on these entities (stored in FMassEntityManager::Entities)
	for (FMassEntityHandle& Entity : EntityHandles)
	{
		check(GetEntityStorageInterface().IsValidIndex(Entity.Index));

		const UE::Mass::IEntityStorageInterface::EEntityState EntityState = GetEntityStorageInterface().GetEntityState(Entity.Index);
		checkf(EntityState == UE::Mass::IEntityStorageInterface::EEntityState::Reserved, TEXT("Trying to build entities that are not reserved. Check all handles are reserved or consider using BatchCreateEntities"));

		const int32 SerialNumber = GetEntityStorageInterface().GetSerialNumber(Entity.Index);
		Entity.SerialNumber = SerialNumber;

		GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, TargetArchetypeHandle.DataPtr);
	}

	TargetArchetypeHandle.DataPtr->BatchAddEntities(EntityHandles, SharedFragmentValues, TargetArchetypeEntityRanges);
	UE_TRACE_MASS_ENTITIES_CREATED(EntityHandles, *TargetArchetypeHandle.DataPtr.Get());

	if (EncodedEntitiesWithPayload.GetPayload().IsEmpty() == false)
	{
		// at this point all the entities are in the target archetype, we can set the values
		// note that even though the "subchunk" information could have changed the order of entities is the same and 
		// corresponds to the order in FMassArchetypeEntityCollectionWithPayload's payload
		TargetArchetypeHandle.DataPtr->BatchSetFragmentValues(TargetArchetypeEntityRanges, EncodedEntitiesWithPayload.GetPayload());
	}

	// With this call we're either creating a fresh context populated with EntityHandles, or it will append 
	// EntityHandles to active context.
	// Not creating the context sooner since we want to reuse TargetArchetypeEntityRanges by moving it over to the context.
	// Note that we can afford to create this context so late since all previous operations were on the archetype level
	// and as such won't cause observers triggering (which usually is prevented by context's existence), and that we 
	// strongly assume the all entity creation/building (not to be mistaken with "reserving") takes place in a single thread
	// @todo add checks/ensures enforcing the assumption mentioned above.
	return ObserverManager.GetOrMakeCreationContext(EntityHandles, FMassArchetypeEntityCollection(TargetArchetypeHandle, MoveTemp(TargetArchetypeEntityRanges)));
}

TSharedRef<FMassEntityManager::FEntityCreationContext> FMassEntityManager::BatchCreateReservedEntities(const FMassArchetypeHandle& ArchetypeHandle
	, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, TConstArrayView<FMassEntityHandle> ReservedEntities)
{
	CHECK_SYNC_API_RETURN(return FMassObserverManager::FCreationContext::DebugCreateDummyCreationContext());
	checkf(!ReservedEntities.IsEmpty(), TEXT("No reserved entities given to batch create."));

	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchCreateReservedEntities);

	return InternalBatchCreateReservedEntities(
		GetOrCreateSuitableArchetype(ArchetypeHandle, SharedFragmentValues.GetBitSet())
		, SharedFragmentValues, ReservedEntities, /*bCreateContext=*/true).ToSharedRef();
}

TSharedRef<FMassEntityManager::FEntityCreationContext> FMassEntityManager::BatchCreateEntities(const FMassArchetypeHandle& ArchetypeHandle
	, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, const int32 Count, TArray<FMassEntityHandle>& InOutEntities)
{
	CHECK_SYNC_API_RETURN(return FMassObserverManager::FCreationContext::DebugCreateDummyCreationContext());
	testableCheckfReturn(ArchetypeHandle.IsValid(), return FMassObserverManager::FCreationContext::DebugCreateDummyCreationContext()
		, TEXT("%hs expecting a valid ArchetypeHandle"), __FUNCTION__);

	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchCreateEntities);

	MASS_BREAKPOINT(UE::Mass::Debug::FBreakpoint::CheckCreateEntityBreakpoints(ArchetypeHandle));

	TConstArrayView<FMassEntityHandle> ReservedEntities = BatchReserveEntities(Count, InOutEntities);

	return InternalBatchCreateReservedEntities(
		GetOrCreateSuitableArchetype(ArchetypeHandle, SharedFragmentValues.GetBitSet())
		, SharedFragmentValues, ReservedEntities, /*bCreateContext=*/true).ToSharedRef();
}

void FMassEntityManager::BatchCreateEntitiesAsyncUnsafe(const FMassArchetypeHandle& ArchetypeHandle
	, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, const int32 Count, TArray<FMassEntityHandle>& InOutEntities)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchCreateEntitiesAsyncUnsafe);

	testableCheckfReturn(ArchetypeHandle.IsValid(), return, TEXT("%hs expecting a valid ArchetypeHandle"), __FUNCTION__);

	MASS_BREAKPOINT(UE::Mass::Debug::FBreakpoint::CheckCreateEntityBreakpoints(ArchetypeHandle));

	TConstArrayView<FMassEntityHandle> ReservedEntities = BatchReserveEntities(Count, InOutEntities);

	InternalBatchCreateReservedEntities(ArchetypeHandle, SharedFragmentValues, ReservedEntities, /*bCreateContext=*/false);
}

TSharedPtr<FMassEntityManager::FEntityCreationContext> FMassEntityManager::InternalBatchCreateReservedEntities(const FMassArchetypeHandle& ArchetypeHandle
	, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, TConstArrayView<FMassEntityHandle> ReservedEntities
	, const bool bCreateContext)
{
	// Functions calling into this one are required to verify that the archetype handle is valid
	FMassArchetypeData* ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandle(ArchetypeHandle);
	checkf(ArchetypeData, TEXT("Functions calling into this one are required to verify that the archetype handle is valid"));

	MASS_BREAKPOINT(UE::Mass::Debug::FBreakpoint::CheckCreateEntityBreakpoints(ArchetypeHandle));

	for (FMassEntityHandle Entity : ReservedEntities)
	{
		check(IsEntityValid(Entity));
		const UE::Mass::IEntityStorageInterface::EEntityState EntityState = GetEntityStorageInterface().GetEntityState(Entity.Index);
		checkf(EntityState == UE::Mass::IEntityStorageInterface::EEntityState::Reserved, TEXT("Trying to build entities that are not reserved. Check all handles are reserved or consider using BatchCreateEntities"));

		GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, ArchetypeHandle.DataPtr);
	}

	FMassArchetypeEntityCollection::FEntityRangeArray TargetArchetypeEntityRanges;
	ArchetypeData->BatchAddEntities(ReservedEntities, SharedFragmentValues, TargetArchetypeEntityRanges);

	UE_TRACE_MASS_ENTITIES_CREATED(ReservedEntities, *ArchetypeData)

	if (bCreateContext)
	{
		return ObserverManager.GetOrMakeCreationContext(ReservedEntities, FMassArchetypeEntityCollection(ArchetypeHandle, MoveTemp(TargetArchetypeEntityRanges)));
	}

	return nullptr;
}

void FMassEntityManager::DestroyEntity(FMassEntityHandle Entity)
{
	CHECK_SYNC_API();
	
	CheckIfEntityIsActive(Entity);

	FMassArchetypeData* Archetype = GetEntityStorageInterface().GetArchetype(Entity.Index);

	MASS_BREAKPOINT(UE::Mass::Debug::FBreakpoint::CheckDestroyEntityBreakpoints(Entity));

	if (Archetype)
	{
		ObserverManager.OnPreEntityDestroyed(Archetype->GetCompositionDescriptor(), Entity);
		Archetype->RemoveEntity(Entity);
	}
	
	UE_TRACE_MASS_ENTITY_DESTROYED(Entity)

	InternalReleaseEntity(Entity);
}

void FMassEntityManager::BatchDestroyEntities(TConstArrayView<FMassEntityHandle> InEntities)
{
	CHECK_SYNC_API();
	checkf(ObserverManager.IsLocked() == false, TEXT("%hs: Trying to destroy entities while observers are locked - remove-observers won't get triggered in time to read fragments being removed."), __FUNCTION__);

	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchDestroyEntities);

	MASS_BREAKPOINT(UE::Mass::Debug::FBreakpoint::CheckDestroyEntityBreakpoints(InEntities));

	TArray<FMassEntityHandle> ValidEntityHandles;
	ValidEntityHandles.Reserve(InEntities.Num());

	for (const FMassEntityHandle Entity : InEntities)
	{
		if (GetEntityStorageInterface().IsValidIndex(Entity.Index) == false)
		{
			continue;
		}

		const int32 SerialNumber = GetEntityStorageInterface().GetSerialNumber(Entity.Index);
		if (SerialNumber != Entity.SerialNumber)
		{
			continue;
		}

		if (FMassArchetypeData* Archetype = GetEntityStorageInterface().GetArchetype(Entity.Index))
		{
			ObserverManager.OnPreEntityDestroyed(Archetype->GetCompositionDescriptor(), Entity);
			Archetype->RemoveEntity(Entity);
			ValidEntityHandles.Add(Entity);
		}
		// else it's a "reserved" entity so it has not been assigned to an archetype yet, no archetype nor observers to notify
	}
	
	UE_TRACE_MASS_ENTITIES_DESTROYED(InEntities)

	GetEntityStorageInterface().Release(InEntities);
}

void FMassEntityManager::BatchDestroyEntityChunks(const FMassArchetypeEntityCollection& EntityCollection)
{
	BatchDestroyEntityChunks(MakeArrayView(&EntityCollection, 1));
}

void FMassEntityManager::BatchDestroyEntityChunks(TConstArrayView<FMassArchetypeEntityCollection> Collections)
{
	CHECK_SYNC_API();
	checkf(ObserverManager.IsLocked() == false, TEXT("%hs: Trying to destroy entities while observers are locked - remove-observers won't get triggered in time to read fragments being removed."), __FUNCTION__);

	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchDestroyEntityChunks);

	TArray<FMassEntityHandle> EntitiesRemoved;
	FMassProcessingContext ProcessingContext(*this);

	for (const FMassArchetypeEntityCollection& EntityCollection : Collections)
	{
		EntitiesRemoved.Reset();
		if (ensureMsgf(EntityCollection.GetArchetype().IsValid() && EntityCollection.IsUpToDate(), TEXT("Provided collection is out of data")))
		{
			ObserverManager.OnPreEntitiesDestroyed(ProcessingContext, EntityCollection);

			checkf(EntityCollection.IsUpToDate(), TEXT("Remove-type observers resulted in additional mutating of entity composition. This is not allowed."))

			FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(EntityCollection.GetArchetype());
			ArchetypeData.BatchDestroyEntityChunks(EntityCollection.GetRanges(), EntitiesRemoved);

			GetEntityStorageInterface().Release(EntitiesRemoved);
		}
		else
		{
			UE::Mass::Private::ConvertArchetypelessSubchunksIntoEntityHandles(EntityCollection.GetRanges(), EntitiesRemoved);
			GetEntityStorageInterface().ForceRelease(EntitiesRemoved);
		}
	}
}

void FMassEntityManager::BatchGroupEntities(const UE::Mass::FArchetypeGroupHandle GroupHandle, TConstArrayView<FMassArchetypeEntityCollection> Collections)
{
	CHECK_SYNC_API();

	if (GroupHandle.IsValid() == false)
	{
		UE_LOGF(LogMass, Warning, "%s called with an invalid GroupHandle", __FUNCTION__);
		return;
	}

	TArray<FMassEntityHandle> EntitiesBeingMoved;

	for (const FMassArchetypeEntityCollection& EntityCollection : Collections)
	{
		if (EntityCollection.GetArchetype().IsValid())
		{
			FMassArchetypeData& CurrentArchetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(EntityCollection.GetArchetype());
			if (CurrentArchetype.IsInGroup(GroupHandle) == false)
			{
				UE::Mass::FArchetypeGroups NewGroups = CurrentArchetype.GetGroups();
				NewGroups.Add(GroupHandle);

				const FMassArchetypeHandle NewArchetypeHandle = InternalCreateSimilarArchetype(EntityCollection.GetArchetype().DataPtr, MoveTemp(NewGroups));

				EntitiesBeingMoved.Reset();
				CurrentArchetype.BatchMoveEntitiesToAnotherArchetype(EntityCollection, *NewArchetypeHandle.DataPtr.Get(), EntitiesBeingMoved
				// we need something like the following to support observers
				//, bTagsAddedAreObserved ? &NewArchetypeEntityRanges : nullptr
				);

				for (const FMassEntityHandle& Entity : EntitiesBeingMoved)
				{
					check(GetEntityStorageInterface().IsValidIndex(Entity.Index));
					GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);
				}
			}
		}
	}
}

void FMassEntityManager::BatchGroupEntities(const UE::Mass::FArchetypeGroupHandle GroupHandle, TConstArrayView<FMassEntityHandle> InEntities)
{
	TArray<FMassArchetypeEntityCollection> Collections;
	UE::Mass::Utils::CreateEntityCollections(*this, InEntities, FMassArchetypeEntityCollection::FoldDuplicates, Collections);
	BatchGroupEntities(GroupHandle, Collections);
}

void FMassEntityManager::RemoveEntityFromGroupType(FMassEntityHandle EntityHandle, UE::Mass::FArchetypeGroupType GroupType)
{
	CHECK_SYNC_API();

	const FMassArchetypeHandle CurrentArchetypeHandle = GetArchetypeForEntity(EntityHandle);
	if (FMassArchetypeData* CurrentArchetype = CurrentArchetypeHandle.DataPtr.Get())
	{
		if (CurrentArchetype->IsInGroupOfType(GroupType))
		{
			const FMassArchetypeHandle NewArchetypeHandle = InternalCreateSimilarArchetype(CurrentArchetypeHandle.DataPtr, CurrentArchetype->GetGroups().Remove(GroupType));

			CurrentArchetype->MoveEntityToAnotherArchetype(EntityHandle, *NewArchetypeHandle.DataPtr.Get());
			
			GetEntityStorageInterface().SetArchetypeFromShared(EntityHandle.Index, NewArchetypeHandle.DataPtr);
		}
	}
}

UE::Mass::FArchetypeGroupHandle FMassEntityManager::GetGroupForEntity(FMassEntityHandle EntityHandle, UE::Mass::FArchetypeGroupType GroupType) const
{
	if (FMassArchetypeData* CurrentArchetype = GetArchetypeForEntity(EntityHandle).DataPtr.Get())
	{
		return UE::Mass::FArchetypeGroupHandle(GroupType, CurrentArchetype->GetGroups().GetID(GroupType));
	}

	return UE::Mass::FArchetypeGroupHandle();
}

UE::Mass::FArchetypeGroupType FMassEntityManager::FindOrAddArchetypeGroupType(const FName GroupName)
{
	LLM_SCOPE_BYNAME(TEXT("Mass/EntityManager"));
	const int32* FoundGroupIndex = GroupNameToTypeIndex.Find(GroupName);
	if (LIKELY(FoundGroupIndex))
	{
		return UE::Mass::FArchetypeGroupType(*FoundGroupIndex);
	}

	const int32 NewGroupIndex = GroupTypes.Add(GroupName);
	GroupNameToTypeIndex.Add(GroupName, NewGroupIndex);
	return UE::Mass::FArchetypeGroupType(NewGroupIndex);
}

const UE::Mass::FArchetypeGroups& FMassEntityManager::GetGroupsForArchetype(const FMassArchetypeHandle& ArchetypeHandle) const
{
	if (ArchetypeHandle.IsValid() == false)
	{
		static UE::Mass::FArchetypeGroups DummyGroups;
		return DummyGroups;
	}

	return ArchetypeHandle.DataPtr->GetGroups();
}

void FMassEntityManager::AddFragmentToEntity(FMassEntityHandle Entity, TNotNull<const UScriptStruct*> FragmentType)
{
	CHECK_SYNC_API();

	CheckIfEntityIsActive(Entity);

	FMassFragmentBitSet AddedBits = InternalAddFragmentListToEntityChecked(Entity, FMassFragmentBitSet(FragmentType));

	ObserverManager.OnPostCompositionAdded(Entity, AddedBits);
}

void FMassEntityManager::AddFragmentToEntity(FMassEntityHandle Entity, TNotNull<const UScriptStruct*> FragmentType, const FStructInitializationCallback& Initializer)
{
	CHECK_SYNC_API();

	CheckIfEntityIsActive(Entity);

	FMassFragmentBitSet Fragments = InternalAddFragmentListToEntityChecked(Entity, FMassFragmentBitSet(FragmentType));
	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
	check(CurrentArchetype);
	void* FragmentData = CurrentArchetype->GetMutableFragmentDataForEntity(FragmentType, Entity);
	Initializer(FragmentData, *FragmentType);

	ObserverManager.OnPostCompositionAdded(Entity, Fragments);
}

void FMassEntityManager::AddFragmentListToEntity(FMassEntityHandle Entity, TConstArrayView<const UScriptStruct*> FragmentList)
{
	CHECK_SYNC_API();

	CheckIfEntityIsActive(Entity);

	const FMassElementBitSet AddedElements(InternalAddFragmentListToEntityChecked(Entity, FMassElementBitSet(FragmentList)));
	
	ObserverManager.OnPostCompositionAdded(Entity, AddedElements);
}

void FMassEntityManager::AddCompositionToEntity_GetDelta(FMassEntityHandle Entity, FMassArchetypeCompositionDescriptor& InOutDescriptor, const FMassArchetypeSharedFragmentValues* AddedSharedFragmentValues)
{
	CHECK_SYNC_API();

	CheckIfEntityIsActive(Entity);

	FMassArchetypeData* OldArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
	check(OldArchetype);

	InOutDescriptor.Remove(OldArchetype->GetCompositionDescriptor());

	ensureMsgf(InOutDescriptor.HasAny<FMassChunkFragment>() == false, TEXT("Adding new chunk fragments is not supported"));
	ensureMsgf(InOutDescriptor.HasAny<FMassSharedFragment>() == false
		|| (AddedSharedFragmentValues && AddedSharedFragmentValues->DoesMatchComposition(InOutDescriptor))
		, TEXT("When adding new shared fragments it's required to provide values for said fragments"));

	if (InOutDescriptor.IsEmpty() == false)
	{
		FMassArchetypeCompositionDescriptor NewDescriptor = OldArchetype->GetCompositionDescriptor();
		NewDescriptor.Append(InOutDescriptor);

		const FMassArchetypeHandle NewArchetypeHandle = CreateArchetype(NewDescriptor, FMassArchetypeCreationParams(*OldArchetype));

		if (ensure(NewArchetypeHandle.DataPtr.Get() != OldArchetype))
		{
			// Move the entity over
			FMassArchetypeData& NewArchetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(NewArchetypeHandle);
			NewArchetype.CopyDebugNamesFrom(*OldArchetype);
			if (AddedSharedFragmentValues)
			{
				// we need to merge AddedSharedFragmentValues with OldArchetype's shared fragments
				FMassArchetypeSharedFragmentValues CurrentSharedFragment = OldArchetype->GetSharedFragmentValues(Entity);
				CurrentSharedFragment.Append(*AddedSharedFragmentValues);
				CurrentSharedFragment.Sort();
				OldArchetype->MoveEntityToAnotherArchetype(Entity, NewArchetype, &CurrentSharedFragment);
			}
			else
			{
				OldArchetype->MoveEntityToAnotherArchetype(Entity, NewArchetype);
			}

			GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);

			ObserverManager.OnPostCompositionAdded(Entity, InOutDescriptor);
		}
	}
}

void FMassEntityManager::RemoveCompositionFromEntity(FMassEntityHandle Entity, const FMassArchetypeCompositionDescriptor& InDescriptor)
{
	CHECK_SYNC_API();

	CheckIfEntityIsActive(Entity);

	if(InDescriptor.IsEmpty() == false)
	{
		FMassArchetypeData* OldArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
		check(OldArchetype);

		FMassArchetypeCompositionDescriptor NewDescriptor = OldArchetype->GetCompositionDescriptor();
		NewDescriptor.Remove(InDescriptor);

		ensureMsgf(InDescriptor.HasAny<FMassChunkFragment>() == false, TEXT("Removing chunk fragments is not supported"));

		if (NewDescriptor.IsEquivalent(OldArchetype->GetCompositionDescriptor()) == false)
		{
			ObserverManager.OnPreCompositionRemoved(Entity, InDescriptor);

			const FMassArchetypeHandle NewArchetypeHandle = CreateArchetype(NewDescriptor, FMassArchetypeCreationParams(*OldArchetype));

			if (ensure(NewArchetypeHandle.DataPtr.Get() != OldArchetype))
			{
				// Move the entity over
				FMassArchetypeData& NewArchetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(NewArchetypeHandle);
				NewArchetype.CopyDebugNamesFrom(*OldArchetype);
				if (InDescriptor.GetElementsBitSet().HasAnySharedElements() == false)
				{
					OldArchetype->MoveEntityToAnotherArchetype(Entity, NewArchetype);
				}
				else
				{
					// we need to remove the shared fragment values to match the new composition
					FMassArchetypeSharedFragmentValues CurrentSharedFragment = OldArchetype->GetSharedFragmentValues(Entity);
					CurrentSharedFragment.Remove(InDescriptor);
					CurrentSharedFragment.Sort();
					OldArchetype->MoveEntityToAnotherArchetype(Entity, NewArchetype, &CurrentSharedFragment);
				}
				GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);
			}
		}
	}
}

FMassArchetypeCompositionDescriptor FMassEntityManager::GetArchetypeComposition(const FMassArchetypeHandle& ArchetypeHandle) const
{
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	return ArchetypeData.GetCompositionDescriptor();
}

void FMassEntityManager::InternalBuildEntity(FMassEntityHandle Entity, const FMassArchetypeHandle& ArchetypeHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues)
{
	const TSharedPtr<FMassArchetypeData>& NewArchetype = ArchetypeHandle.DataPtr;
	GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, ArchetypeHandle.DataPtr);
	NewArchetype->AddEntity(Entity, SharedFragmentValues);
	
	UE_TRACE_MASS_ENTITY_CREATED(Entity, *NewArchetype)
	ObserverManager.OnPostEntityCreated(Entity, NewArchetype->GetCompositionDescriptor());
}

void FMassEntityManager::InternalReleaseEntity(FMassEntityHandle Entity)
{
	// Using force release by bypass serial number check since we have verified the validity of the handle earlier.
	GetEntityStorageInterface().ForceReleaseOne(Entity);
}

FMassElementBitSet FMassEntityManager::InternalAddFragmentListToEntityChecked(FMassEntityHandle Entity, const FMassElementBitSet& FragmentList)
{
	TSharedPtr<FMassArchetypeData>& OldArchetype = GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index);
	check(OldArchetype);

	FMassElementBitSet SparseElements = FMassElementBitSet::GetAllSparseElements() & FragmentList;
	FMassElementBitSet RegularElements = FragmentList - SparseElements;

	FMassFragmentBitSet NewFragments = RegularElements - OldArchetype->GetCompositionBitSet();
	if (NewFragments.IsEmpty() == false)
	{
		InternalAddFragmentListToEntity(Entity, NewFragments);
	}
	if (SparseElements.IsEmpty() == false)
	{
		AddSparseElementsToEntity(Entity, SparseElements);
	}

	return NewFragments + SparseElements;
}

void FMassEntityManager::InternalAddFragmentListToEntity(FMassEntityHandle Entity, const FMassElementBitSet& FragmentList)
{
	checkf(FragmentList.IsEmpty() == false, TEXT("%hs is intended for internal calls with non empty NewFragments parameter"), __FUNCTION__);
	check(GetEntityStorageInterface().IsValidIndex(Entity.Index));
	TSharedPtr<FMassArchetypeData>& OldArchetype = GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index);
	check(OldArchetype.IsValid());

	// fetch or create the new archetype
	const FMassArchetypeHandle NewArchetypeHandle = CreateArchetype(OldArchetype, FragmentList);
	checkf(NewArchetypeHandle.DataPtr != OldArchetype, TEXT("%hs is intended for internal calls with non overlapping fragment list."), __FUNCTION__);

	// Move the entity over
	FMassArchetypeData& NewArchetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(NewArchetypeHandle);
	NewArchetype.CopyDebugNamesFrom(*OldArchetype);
	OldArchetype->MoveEntityToAnotherArchetype(Entity, NewArchetype);

	GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);
}

void FMassEntityManager::AddFragmentInstanceListToEntity(FMassEntityHandle Entity, TConstArrayView<FInstancedStruct> FragmentInstanceList)
{
	CHECK_SYNC_API();

	CheckIfEntityIsActive(Entity);
	checkf(FragmentInstanceList.Num() > 0, TEXT("Need to specify at least one fragment instances for this operation"));

	const FMassArchetypeCompositionDescriptor Descriptor(InternalAddFragmentListToEntityChecked(Entity, FMassFragmentBitSet(FragmentInstanceList)));
	
	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
	check(CurrentArchetype);
	CurrentArchetype->SetFragmentsData(Entity, FragmentInstanceList);

	ObserverManager.OnPostCompositionAdded(Entity, Descriptor);
}

void FMassEntityManager::RemoveFragmentFromEntity(FMassEntityHandle Entity, const UScriptStruct* FragmentType)
{
	RemoveFragmentListFromEntity(Entity, MakeArrayView(&FragmentType, 1));
}

void FMassEntityManager::RemoveFragmentListFromEntity(FMassEntityHandle EntityHandle, TConstArrayView<const UScriptStruct*> FragmentList)
{
	CHECK_SYNC_API();

	CheckIfEntityIsActive(EntityHandle);

	FMassArchetypeData* OldArchetype = GetEntityStorageInterface().GetArchetype(EntityHandle.Index);
	check(OldArchetype);

	const FMassElementBitSet FragmentsToRemove(FragmentList);
	const FMassElementBitSet SparseElements = FragmentsToRemove.GetSparseElements();
	const bool bRegularFragmentsChange = OldArchetype->GetCompositionBitSet().HasAny(FragmentsToRemove);

	if (bRegularFragmentsChange)
	{
		// If all the fragments got removed this will result in fetching of the empty archetype
		const FMassArchetypeCompositionDescriptor NewComposition(OldArchetype->GetCompositionBitSet() - FragmentsToRemove);
		const FMassArchetypeHandle NewArchetypeHandle = CreateArchetype(NewComposition, FMassArchetypeCreationParams(*OldArchetype));

		// Find overlap.  It isn't guaranteed that the old archetype has all the fragments being removed.
		ObserverManager.OnPreCompositionRemoved(EntityHandle, OldArchetype->GetCompositionBitSet().GetOverlap(FragmentsToRemove));

		// Move the entity over and its new shared fragments
		FMassArchetypeData& NewArchetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(NewArchetypeHandle);
		NewArchetype.CopyDebugNamesFrom(*OldArchetype);
		const FMassArchetypeSharedFragmentValues& OldSharedFragmentValues = OldArchetype->GetSharedFragmentValues(EntityHandle.Index);
		FMassArchetypeSharedFragmentValues NewSharedFragmentValues(OldSharedFragmentValues);
		NewSharedFragmentValues.Remove(FragmentsToRemove.GetSharedFragments());
		// @Todo, optimization, sorting should not be required when only removing elements.
		NewSharedFragmentValues.Sort();

		OldArchetype->MoveEntityToAnotherArchetype(EntityHandle, NewArchetype, &NewSharedFragmentValues);

		GetEntityStorageInterface().SetArchetypeFromShared(EntityHandle.Index, NewArchetypeHandle.DataPtr);
	}
	if (SparseElements.IsEmpty() == false)
	{
		// if there were regular fragments removed then the entity's archetype has changed.
		FMassArchetypeData* CurrentArchetype = bRegularFragmentsChange
			? GetEntityStorageInterface().GetArchetype(EntityHandle.Index)
			: OldArchetype;

		check(CurrentArchetype);

		CurrentArchetype->RemoveSparseElementFromEntity(EntityHandle, SparseElements);
	}
}

void FMassEntityManager::SwapTagsForEntity(FMassEntityHandle Entity, TNotNull<const UScriptStruct*> OldTagType, TNotNull<const UScriptStruct*> NewTagType)
{
	CHECK_SYNC_API();

	checkf(UE::Mass::IsA<FMassTag>(OldTagType), TEXT("%hs works only with tags while '%s' is not one."), __FUNCTION__, *GetPathNameSafe(OldTagType));
	checkf(UE::Mass::IsA<FMassTag>(NewTagType), TEXT("%hs works only with tags while '%s' is not one."), __FUNCTION__, *GetPathNameSafe(NewTagType));

	if (OldTagType == NewTagType
		|| !ensureMsgf(UE::Mass::IsSparse(OldTagType) == false, TEXT("%hs: %s is a sparse element, not supported"), __FUNCTION__, *OldTagType->GetName())
		|| !ensureMsgf(UE::Mass::IsSparse(NewTagType) == false, TEXT("%hs: %s is a sparse element, not supported"), __FUNCTION__, *NewTagType->GetName()))
	{
		return;
	}

	CheckIfEntityIsActive(Entity);

	TSharedPtr<FMassArchetypeData>& CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index);
	check(CurrentArchetype);

	FMassElementBitSet NewCompositionBitSet = CurrentArchetype->GetCompositionBitSet();
	NewCompositionBitSet.Remove(OldTagType);
	NewCompositionBitSet.Add(NewTagType);

	ensureMsgf(NewCompositionBitSet.IsEquivalent(CurrentArchetype->GetCompositionBitSet()) == false, TEXT("Failing this test at this point is unexpected."));

	const FMassArchetypeHandle NewArchetypeHandle = InternalCreateSimilarArchetype(CurrentArchetype, MoveTemp(NewCompositionBitSet));
	checkSlow(NewArchetypeHandle.IsValid());

	ObserverManager.OnPreCompositionRemoved(Entity, FMassElementBitSet(OldTagType));

	// Move the entity over
	CurrentArchetype->MoveEntityToAnotherArchetype(Entity, *NewArchetypeHandle.DataPtr.Get());

	// Change the entity archetype
	GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);

	ObserverManager.OnPostCompositionAdded(Entity, FMassElementBitSet(NewTagType));
}

void FMassEntityManager::AddTagToEntity(FMassEntityHandle Entity, TNotNull<const UScriptStruct*> TagType)
{
	CHECK_SYNC_API();
	checkf(UE::Mass::IsA<FMassTag>(TagType), TEXT("%hs works only with tags while '%s' is not one."), __FUNCTION__, *GetPathNameSafe(TagType));

	CheckIfEntityIsActive(Entity);

	if (UE::Mass::IsSparse(TagType) == false)
	{
		TSharedPtr<FMassArchetypeData>& CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index);
		check(CurrentArchetype);

		if (CurrentArchetype->HasTagType(TagType) == false)
		{
			FMassElementBitSet NewCompositionBitSet = CurrentArchetype->GetCompositionBitSet();
			NewCompositionBitSet.Add(TagType);
			const FMassArchetypeHandle NewArchetypeHandle = InternalCreateSimilarArchetype(CurrentArchetype, MoveTemp(NewCompositionBitSet));
			checkSlow(NewArchetypeHandle.IsValid());

			// Move the entity over
			CurrentArchetype->MoveEntityToAnotherArchetype(Entity, *NewArchetypeHandle.DataPtr.Get());
			GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);

			ObserverManager.OnPostCompositionAdded(Entity, FMassElementBitSet(TagType));
		}
	}
	else
	{
		FStructView _ = AddSparseElementToEntity(Entity, TagType);
	}
}
	
void FMassEntityManager::RemoveTagFromEntity(FMassEntityHandle Entity, TNotNull<const UScriptStruct*> TagType)
{
	CHECK_SYNC_API();
	checkf(UE::Mass::IsA<FMassTag>(TagType), TEXT("%hs works only with tags while '%s' is not one."), __FUNCTION__, *GetPathNameSafe(TagType));

	CheckIfEntityIsActive(Entity);

	if (UE::Mass::IsSparse(TagType) == false)
	{
		TSharedPtr<FMassArchetypeData>& CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index);
		check(CurrentArchetype);

		if (CurrentArchetype->HasTagType(TagType))
		{
			FMassElementBitSet TagDelta(TagType);
			FMassElementBitSet NewComposition = CurrentArchetype->GetCompositionBitSet() - TagDelta;

			ObserverManager.OnPreCompositionRemoved(Entity, MoveTemp(TagDelta));

			const FMassArchetypeHandle NewArchetypeHandle = InternalCreateSimilarArchetype(CurrentArchetype, MoveTemp(NewComposition));
			checkSlow(NewArchetypeHandle.IsValid());

			// Move the entity over
			CurrentArchetype->MoveEntityToAnotherArchetype(Entity, *NewArchetypeHandle.DataPtr.Get());
			GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);
		}
	}
	else
	{
		RemoveSparseElementFromEntity(Entity, TagType);
	}
}

FStructView FMassEntityManager::AddSparseElementToEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType)
{
	CHECK_SPARSE_ELEMENT(ElementType);

	CHECK_SYNC_API_RETURN(return {});
	CheckIfEntityIsActive(EntityHandle);

	TSharedPtr<FMassArchetypeData>& CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(EntityHandle.Index);
	check(CurrentArchetype);

	return CurrentArchetype->AddSparseElementToEntity(EntityHandle, ElementType);
}

void FMassEntityManager::AddSparseElementsToEntity(FMassEntityHandle EntityHandle, const FMassElementBitSet& ElementsBitset)
{
	CHECK_SYNC_API_RETURN(return);
	CheckIfEntityIsActive(EntityHandle);

	TSharedPtr<FMassArchetypeData>& CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(EntityHandle.Index);
	check(CurrentArchetype);

	for (FMassElementBitSet::FIndexIterator It = ElementsBitset.GetIndexIterator(); It; ++It)
	{
		const UScriptStruct* SparseElementType = FMassElementBitSet::GetTypeAtIndex(*It);
		CA_ASSUME(SparseElementType);
		// assuming SparseElementType is indeed sparse, as tested by UE::Mass::IsSparse,
		// AddSparseElementToEntity will ensure that. 
		FStructView _ = CurrentArchetype->AddSparseElementToEntity(EntityHandle, SparseElementType);
	}
}

void FMassEntityManager::RemoveSparseElementFromEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType)
{
	CHECK_SPARSE_ELEMENT(ElementType);

	CHECK_SYNC_API();
	CheckIfEntityIsActive(EntityHandle);

	TSharedPtr<FMassArchetypeData>& CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(EntityHandle.Index);
	check(CurrentArchetype);

	CurrentArchetype->RemoveSparseElementFromEntity(EntityHandle, ElementType);
}

bool FMassEntityManager::DoesEntityHaveElement(FMassEntityHandle Entity, TNotNull<const UScriptStruct*> ElementType) const
{
	CHECK_ELEMENT(ElementType);

	CheckIfEntityIsActive(Entity);
	const TSharedPtr<FMassArchetypeData>& CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index);
	check(CurrentArchetype);
	if (UE::Mass::IsSparse(ElementType))
	{
		return CurrentArchetype->HasSparseElementForEntity(ElementType, Entity);
	}
	
	return CurrentArchetype->GetCompositionBitSet().Contains(ElementType);
}

void FMassEntityManager::BatchAddElementToEntities(TConstArrayView<FMassEntityHandle> Entities, TNotNull<const UScriptStruct*> ElementType)
{
	testableCheckfReturn(UE::Mass::Private::IsElement(ElementType)
		&& !UE::Mass::IsA<FMassSharedFragment>(ElementType)
		&& !UE::Mass::IsA<FMassConstSharedFragment>(ElementType)
		&& !UE::Mass::IsA<FMassChunkFragment>(ElementType)
		, return
		, TEXT("%hs: Only tags, fragments, and sparse elements can be added via this API."
			" Shared fragments require a value, chunk fragments require dedicated APIs."
			" Type provided: %s")
		, __FUNCTION__, *ElementType->GetName());

	TArray<FMassArchetypeEntityCollection> EntityCollections;
	UE::Mass::Utils::CreateEntityCollections(*this, Entities, FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates, EntityCollections);

	if (UE::Mass::IsSparse(ElementType))
	{
		BatchAddSparseElementToEntities(EntityCollections, ElementType);
	}
	else if (UE::Mass::IsA<FMassTag>(ElementType))
	{
		BatchChangeTagsForEntities(EntityCollections, FMassTagBitSet(ElementType), {});
	}
	else
	{
		BatchChangeFragmentCompositionForEntities(EntityCollections, FMassFragmentBitSet(ElementType), {});
	}
}

void FMassEntityManager::AddElementToEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType)
{
	testableCheckfReturn(UE::Mass::Private::IsElement(ElementType)
		&& !UE::Mass::IsA<FMassSharedFragment>(ElementType)
		&& !UE::Mass::IsA<FMassConstSharedFragment>(ElementType)
		&& !UE::Mass::IsA<FMassChunkFragment>(ElementType)
		, return
		, TEXT("%hs: Only tags, fragments, and sparse elements can be added via this API."
			" Shared fragments require a value, chunk fragments require dedicated APIs."
			" Type provided: %s")
		, __FUNCTION__, *ElementType->GetName());

	if (UE::Mass::IsSparse(ElementType))
	{
		TSharedPtr<FMassArchetypeData>& CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(EntityHandle.Index);
		checkf(CurrentArchetype, TEXT("Entity (ID: %d, SN:%d) not assigned an archetype yet"), EntityHandle.Index, EntityHandle.SerialNumber);

		FStructView _ = CurrentArchetype->AddSparseElementToEntity(EntityHandle, ElementType);
	}
	else if (UE::Mass::IsA<FMassTag>(ElementType))
	{
		AddTagToEntity(EntityHandle, ElementType);
	}
	else
	{
		AddFragmentToEntity(EntityHandle, ElementType);
	}
}

void FMassEntityManager::BatchRemoveElementFromEntities(TConstArrayView<FMassEntityHandle> Entities, TNotNull<const UScriptStruct*> ElementType)
{
	CHECK_ELEMENT(ElementType);

	TArray<FMassArchetypeEntityCollection> EntityCollections;
	UE::Mass::Utils::CreateEntityCollections(*this, Entities, FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates, EntityCollections);
	if (UE::Mass::IsSparse(ElementType))
	{
		BatchRemoveSparseElementFromEntities(EntityCollections, ElementType);
	}
	else if (UE::Mass::IsA<FMassTag>(ElementType))
	{
		BatchChangeTagsForEntities(EntityCollections, {}, FMassTagBitSet(ElementType));
	}
	else if (UE::Mass::IsA<FMassSharedFragment>(ElementType))
	{
		BatchRemoveSharedFragmentFromEntities(EntityCollections, FMassSharedFragmentBitSet(ElementType));
	}
	else if (UE::Mass::IsA<FMassConstSharedFragment>(ElementType))
	{
		BatchRemoveConstSharedFragmentFromEntities(EntityCollections, FMassConstSharedFragmentBitSet(ElementType));
	}
	else
	{
		BatchChangeFragmentCompositionForEntities(EntityCollections, {}, FMassFragmentBitSet(ElementType));
	}
}

void FMassEntityManager::RemoveElementFromEntity(FMassEntityHandle Entity, TNotNull<const UScriptStruct*> ElementType)
{
	CHECK_ELEMENT(ElementType);

	if (UE::Mass::IsSparse(ElementType))
	{
		RemoveSparseElementFromEntity(Entity, ElementType);
	}
	else if (UE::Mass::IsA<FMassTag>(ElementType))
	{
		RemoveTagFromEntity(Entity, ElementType);
	}
	else if (UE::Mass::IsA<FMassSharedFragment>(ElementType))
	{
		RemoveSharedFragmentFromEntity(Entity, ElementType);
	}
	else if (UE::Mass::IsA<FMassConstSharedFragment>(ElementType))
	{
		RemoveConstSharedFragmentFromEntity(Entity, ElementType);
	}
	else
	{
		RemoveFragmentFromEntity(Entity, ElementType);
	}
}

bool FMassEntityManager::SwapConstSharedFragmentForEntity(const FMassEntityHandle Entity, const FConstSharedStruct& InConstSharedFragment)
{
	CHECK_SYNC_API_RETURN(return false);

	if (!ensureMsgf(InConstSharedFragment.IsValid(), TEXT("%hs parameter Fragment is expected to be valid"), __FUNCTION__))
	{
		return false;
	}

	testableCheckfReturn(ConstSharedFragmentsContainer.Find(InConstSharedFragment.GetScriptStruct()) != nullptr
		, return false
		, TEXT("Provided shared struct requires to be registered first (i.e., GetGetOrCreateConstSharedFragment)"));

	CheckIfEntityIsActive(Entity);

	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index).Get();
	check(CurrentArchetype);

	const UScriptStruct* FragmentType = InConstSharedFragment.GetScriptStruct();
	if (!CurrentArchetype->GetCompositionBitSet().Contains(FragmentType))
	{
		// Observers will get notify by the call to AddConstSharedFragmentToEntity
		AddConstSharedFragmentToEntity(Entity, InConstSharedFragment);
	}
	else
	{
		// SetSharedFragmentsData won't notify observers so we have to do it here to allow observers to handle the change
		ObserverManager.OnPreCompositionRemoved(Entity, FMassArchetypeCompositionDescriptor(FMassElementBitSet{FragmentType}));

		CurrentArchetype->SetSharedFragmentsData(Entity, MakeArrayView<FConstSharedStruct>({ InConstSharedFragment }));

		ObserverManager.OnPostCompositionAdded(Entity, FMassElementBitSet(FragmentType));
	}

	return true;
}

bool FMassEntityManager::AddConstSharedFragmentToEntity(const FMassEntityHandle Entity, const FConstSharedStruct& InConstSharedFragment)
{
	CHECK_SYNC_API_RETURN(return false);

	if (!ensureMsgf(InConstSharedFragment.IsValid(), TEXT("%hs parameter Fragment is expected to be valid"), __FUNCTION__))
	{
		return false;
	}

	testableCheckfReturn(ConstSharedFragmentsContainer.Find(InConstSharedFragment.GetScriptStruct()) != nullptr
		, return false
		, TEXT("Provided shared struct requires to be registered first (i.e., GetGetOrCreateConstSharedFragment)"));

	CheckIfEntityIsActive(Entity);

	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index).Get();
	check(CurrentArchetype);

	const UScriptStruct* StructType = InConstSharedFragment.GetScriptStruct();
	CA_ASSUME(StructType);
	if (CurrentArchetype->GetCompositionDescriptor().Contains(StructType))
	{
		const FMassArchetypeSharedFragmentValues& SharedFragmentValues = CurrentArchetype->GetSharedFragmentValues(Entity);
		FConstSharedStruct ExistingConstSharedStruct = SharedFragmentValues.GetConstSharedFragmentStruct(StructType);
		if (ExistingConstSharedStruct == InConstSharedFragment || ExistingConstSharedStruct.CompareStructValues(InConstSharedFragment))
		{
			// nothing to do
			return true;
		}
		UE_LOGF(LogMass, Warning, "Changing shared fragment value of entities is not supported");
		return false;
	}
	
	FMassArchetypeCompositionDescriptor NewComposition(CurrentArchetype->GetCompositionDescriptor());
	NewComposition.Add(StructType);
	const FMassArchetypeHandle NewArchetypeHandle = CreateArchetype(NewComposition, FMassArchetypeCreationParams(*CurrentArchetype));
	check(NewArchetypeHandle.IsValid());
	FMassArchetypeData* NewArchetype = NewArchetypeHandle.DataPtr.Get();
	check(NewArchetype);

	NewArchetype->CopyDebugNamesFrom(*CurrentArchetype);

	const FMassArchetypeSharedFragmentValues& OldSharedFragmentValues = CurrentArchetype->GetSharedFragmentValues(Entity.Index);
	check(!OldSharedFragmentValues.ContainsType(StructType));
	FMassArchetypeSharedFragmentValues NewSharedFragmentValues(OldSharedFragmentValues);
	NewSharedFragmentValues.Add(InConstSharedFragment);
	NewSharedFragmentValues.Sort();

	// Move the entity over
	CurrentArchetype->MoveEntityToAnotherArchetype(Entity, *NewArchetype, &NewSharedFragmentValues);

	// Change the entity archetype
	GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);

	ObserverManager.OnPostCompositionAdded(Entity, FMassElementBitSet(StructType));

	return true;
}

bool FMassEntityManager::RemoveConstSharedFragmentFromEntity(const FMassEntityHandle Entity, TNotNull<const UScriptStruct*> ConstSharedFragmentType)
{
	CHECK_SYNC_API_RETURN(return false);

	if (!ensureMsgf(UE::Mass::IsA<FMassConstSharedFragment>(ConstSharedFragmentType), TEXT("%hs parameter ConstSharedFragmentType is expected to be a FMassConstSharedFragment"), __FUNCTION__))
	{
		return false;
	}
	
	CheckIfEntityIsActive(Entity);
	
	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index).Get();
	check(CurrentArchetype);
	
	if (!CurrentArchetype->GetCompositionDescriptor().Contains(ConstSharedFragmentType))
	{
		// Nothing to do. Returning false to indicate nothing has been removed, as per function's documentation 
		return false;
	}

	FMassArchetypeCompositionDescriptor NewComposition(CurrentArchetype->GetCompositionDescriptor());
	NewComposition.GetElementsBitSet().Remove(ConstSharedFragmentType);
	const FMassArchetypeHandle NewArchetypeHandle = CreateArchetype(NewComposition);
	check(NewArchetypeHandle.IsValid());
	FMassArchetypeData* NewArchetype = NewArchetypeHandle.DataPtr.Get();
	check(NewArchetype);

	NewArchetype->CopyDebugNamesFrom(*CurrentArchetype);
	
	const FMassArchetypeSharedFragmentValues& OldSharedFragmentValues = CurrentArchetype->GetSharedFragmentValues(Entity.Index);
	check(OldSharedFragmentValues.ContainsType(ConstSharedFragmentType));
	FMassArchetypeSharedFragmentValues NewSharedFragmentValues(OldSharedFragmentValues);
	
	const FMassConstSharedFragmentBitSet ToRemove(ConstSharedFragmentType);
	NewSharedFragmentValues.Remove(ToRemove);
	NewSharedFragmentValues.Sort();

	ObserverManager.OnPreCompositionRemoved(Entity, FMassElementBitSet(ToRemove));

	// Move the entity over
	CurrentArchetype->MoveEntityToAnotherArchetype(Entity, *NewArchetype, &NewSharedFragmentValues);

	// Change the entity archetype
	GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);
	
	return true;
}

bool FMassEntityManager::AddSharedFragmentToEntity(const FMassEntityHandle Entity, const FSharedStruct& InSharedFragment)
{
	CHECK_SYNC_API_RETURN(return false);

	if (!ensureMsgf(InSharedFragment.IsValid(), TEXT("%hs parameter Fragment is expected to be valid"), __FUNCTION__))
	{
		return false;
	}

	testableCheckfReturn(SharedFragmentsContainer.Find(InSharedFragment.GetScriptStruct()) != nullptr
		, return false
		, TEXT("Provided shared struct requires to be registered first (i.e., GetGetOrCreateSharedFragment)"));

	CheckIfEntityIsActive(Entity);

	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index).Get();
	check(CurrentArchetype);

	const UScriptStruct* StructType = InSharedFragment.GetScriptStruct();
	CA_ASSUME(StructType);
	if (CurrentArchetype->GetCompositionBitSet().Contains(StructType))
	{
		const FMassArchetypeSharedFragmentValues& SharedFragmentValues = CurrentArchetype->GetSharedFragmentValues(Entity);
		FConstSharedStruct ExistingSharedStruct = SharedFragmentValues.GetSharedFragmentStruct(StructType);
		if (ExistingSharedStruct == InSharedFragment || ExistingSharedStruct.CompareStructValues(InSharedFragment))
		{
			// nothing to do
			return true;
		}
		UE_LOGF(LogMass, Warning, "Changing shared fragment value of entities is not supported");
		return false;
	}
	
	FMassArchetypeCompositionDescriptor NewComposition(CurrentArchetype->GetCompositionDescriptor());
	NewComposition.GetElementsBitSet().Add(StructType);
	const FMassArchetypeHandle NewArchetypeHandle = CreateArchetype(NewComposition, FMassArchetypeCreationParams(*CurrentArchetype));
	check(NewArchetypeHandle.IsValid());
	FMassArchetypeData* NewArchetype = NewArchetypeHandle.DataPtr.Get();
	check(NewArchetype);

	NewArchetype->CopyDebugNamesFrom(*CurrentArchetype);

	const FMassArchetypeSharedFragmentValues& OldSharedFragmentValues = CurrentArchetype->GetSharedFragmentValues(Entity.Index);
	check(!OldSharedFragmentValues.ContainsType(StructType));
	FMassArchetypeSharedFragmentValues NewSharedFragmentValues(OldSharedFragmentValues);
	NewSharedFragmentValues.Add(InSharedFragment);
	NewSharedFragmentValues.Sort();

	// Move the entity over
	CurrentArchetype->MoveEntityToAnotherArchetype(Entity, *NewArchetype, &NewSharedFragmentValues);

	// Change the entity archetype
	GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);

	ObserverManager.OnPostCompositionAdded(Entity, FMassElementBitSet(StructType));

	return true;
}

bool FMassEntityManager::RemoveSharedFragmentFromEntity(const FMassEntityHandle Entity, TNotNull<const UScriptStruct*> SharedFragmentType)
{
	CHECK_SYNC_API_RETURN(return false);

	if (!ensureMsgf(UE::Mass::IsA<FMassSharedFragment>(SharedFragmentType), TEXT("%hs parameter SharedFragmentType is expected to be a FMassSharedFragment"), __FUNCTION__))
	{
		return false;
	}
	
	CheckIfEntityIsActive(Entity);
	
	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetypeAsShared(Entity.Index).Get();
	check(CurrentArchetype);
	
	if (!CurrentArchetype->GetCompositionBitSet().Contains(SharedFragmentType))
	{
		// Nothing to do. Returning false to indicate nothing has been removed, as per function's documentation 
		return false;
	}

	FMassArchetypeCompositionDescriptor NewComposition(CurrentArchetype->GetCompositionDescriptor());
	NewComposition.GetElementsBitSet().Remove(SharedFragmentType);
	const FMassArchetypeHandle NewArchetypeHandle = CreateArchetype(NewComposition);
	check(NewArchetypeHandle.IsValid());
	FMassArchetypeData* NewArchetype = NewArchetypeHandle.DataPtr.Get();
	check(NewArchetype);

	NewArchetype->CopyDebugNamesFrom(*CurrentArchetype);

	const FMassArchetypeSharedFragmentValues& OldSharedFragmentValues = CurrentArchetype->GetSharedFragmentValues(Entity.Index);
	check(OldSharedFragmentValues.ContainsType(SharedFragmentType));
	FMassArchetypeSharedFragmentValues NewSharedFragmentValues(OldSharedFragmentValues);

	const FMassSharedFragmentBitSet ToRemove(SharedFragmentType);
	NewSharedFragmentValues.Remove(ToRemove);
	NewSharedFragmentValues.Sort();

	ObserverManager.OnPreCompositionRemoved(Entity, FMassElementBitSet(ToRemove));

	// Move the entity over
	CurrentArchetype->MoveEntityToAnotherArchetype(Entity, *NewArchetype, &NewSharedFragmentValues);

	// Change the entity archetype
	GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);

	return true;
}

void FMassEntityManager::BatchChangeCompositionForEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, const FMassElementBitSet& ElementsToAdd, const FMassElementBitSet& ElementsToRemove)
{
	// Adding shared or sparse element types via this overload is not supported -- shared fragments require
	// values (use the overload with FMassArchetypeSharedFragmentValues, or AddSharedFragmentToEntity),
	// sparse elements have separate storage, and chunk fragments require dedicated APIs.
	// Removing shared fragments is supported (extracted in the implementation and passed to BatchMoveEntitiesToAnotherArchetype).
	testableCheckfReturn(ElementsToAdd.ContainsOnlyTagsAndFragments() && ElementsToAdd.GetSparseElements().IsEmpty()
		, return
		, TEXT("%hs: ElementsToAdd contains shared, chunk, or sparse element types."
			" Only non-sparse tags and fragments can be added via this API.")
		, __FUNCTION__);

	static const FMassArchetypeSharedFragmentValues EmptySharedValues;
	BatchChangeCompositionForEntities(EntityCollections, ElementsToAdd, ElementsToRemove, EmptySharedValues);
}

void FMassEntityManager::BatchChangeCompositionForEntities(
	TConstArrayView<FMassArchetypeEntityCollection> EntityCollections,
	const FMassElementBitSet& ElementsToAdd,
	const FMassElementBitSet& ElementsToRemove,
	const FMassArchetypeSharedFragmentValues& SharedFragmentValuesToAdd)
{
	CHECK_SYNC_API();

	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchChangeCompositionForEntitiesWithSharedFragments);

	testableCheckfReturn(ElementsToAdd.GetSparseElements().IsEmpty()
		, return
		, TEXT("%hs: ElementsToAdd contains sparse element types."
			" Sparse elements have separate storage and cannot be added via composition change.")
		, __FUNCTION__);

	// Extract shared fragment removal bitsets for BatchMoveEntitiesToAnotherArchetype
	const FMassSharedFragmentBitSet SharedToRemove = ElementsToRemove.Get<FMassSharedFragmentBitSet>();
	const FMassConstSharedFragmentBitSet ConstSharedToRemove = ElementsToRemove.Get<FMassConstSharedFragmentBitSet>();
	const bool bRemoveSharedFragments = !SharedToRemove.IsEmpty() || !ConstSharedToRemove.IsEmpty();

	const bool bElementsAddedAreObserved = ObserverManager.HasObserversForBitSet(ElementsToAdd, EMassObservedOperation::AddElement);
	const bool bElementsRemovedAreObserved = ObserverManager.HasObserversForBitSet(ElementsToRemove, EMassObservedOperation::RemoveElement);

	for (const FMassArchetypeEntityCollection& Collection : EntityCollections)
	{
		if (FMassArchetypeData* CurrentArchetype = Collection.GetArchetype().DataPtr.Get())
		{
			FMassElementBitSet ElementsAdded;
			FMassElementBitSet NewComposition = CurrentArchetype->GetCompositionBitSet() + ElementsToAdd - ElementsToRemove;
			check(NewComposition.IsEmpty() == false);
			if (UNLIKELY(CurrentArchetype->GetCompositionBitSet().IsEquivalent(NewComposition)))
			{
				continue;
			}

			if (bElementsAddedAreObserved)
			{
				ElementsAdded = ElementsToAdd - CurrentArchetype->GetCompositionBitSet();
			}

			if (bElementsRemovedAreObserved)
			{
				FMassElementBitSet ElementsRemoved = ElementsToRemove.GetOverlap(CurrentArchetype->GetCompositionBitSet());
				if (ElementsRemoved.IsEmpty() == false)
				{
					ObserverManager.OnCompositionChanged(Collection, MoveTemp(ElementsRemoved), EMassObservedOperation::RemoveElement);
				}
			}

			FMassArchetypeHandle NewArchetypeHandle = InternalCreateSimilarArchetype(Collection.GetArchetype().DataPtr, MoveTemp(NewComposition));
			checkSlow(NewArchetypeHandle.IsValid());

			// Move the entity over; single move handles fragment/tag changes + shared fragment removal
			FMassArchetypeEntityCollection::FEntityRangeArray NewArchetypeEntityRanges;
			TArray<FMassEntityHandle> EntitiesBeingMoved;
			const bool bAddSharedFragments = !SharedFragmentValuesToAdd.IsEmpty();
			CurrentArchetype->BatchMoveEntitiesToAnotherArchetype(Collection, *NewArchetypeHandle.DataPtr.Get(), EntitiesBeingMoved
				, bElementsAddedAreObserved && (ElementsAdded.IsEmpty() == false) ? &NewArchetypeEntityRanges : nullptr
				, bAddSharedFragments ? &SharedFragmentValuesToAdd : nullptr
				, bRemoveSharedFragments ? &SharedToRemove : nullptr
				, bRemoveSharedFragments ? &ConstSharedToRemove : nullptr);

			for (const FMassEntityHandle& Entity : EntitiesBeingMoved)
			{
				check(GetEntityStorageInterface().IsValidIndex(Entity.Index));

				GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);
			}

			if (ElementsAdded.IsEmpty() == false)
			{
				ObserverManager.OnCompositionChanged(
					FMassArchetypeEntityCollection(NewArchetypeHandle, MoveTemp(NewArchetypeEntityRanges))
					, MoveTemp(ElementsAdded)
					, EMassObservedOperation::AddElement);
			}
		}
		else // only valid if there are fragments in NewComposition
		{
			FMassElementBitSet NewComposition = (ElementsToAdd - ElementsToRemove);
			BatchBuildEntities(FMassArchetypeEntityCollectionWithPayload(Collection), MoveTemp(NewComposition), SharedFragmentValuesToAdd);
		}
	}
}

void FMassEntityManager::BatchAddFragmentInstancesForEntities(TConstArrayView<FMassArchetypeEntityCollectionWithPayload> EntityCollections, const FMassFragmentBitSet& FragmentsAffected)
{
	static const FMassArchetypeSharedFragmentValues EmptySharedValues;
	BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected, FMassTagBitSet(), EmptySharedValues);
}

void FMassEntityManager::BatchAddFragmentInstancesForEntities(TConstArrayView<FMassArchetypeEntityCollectionWithPayload> EntityCollections
	, const FMassFragmentBitSet& FragmentsAffected, const FMassTagBitSet& TagsToAdd)
{
	static const FMassArchetypeSharedFragmentValues EmptySharedValues;
	BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected, TagsToAdd, EmptySharedValues);
}

void FMassEntityManager::BatchAddFragmentInstancesForEntities(TConstArrayView<FMassArchetypeEntityCollectionWithPayload> EntityCollections
	, const FMassFragmentBitSet& FragmentsAffected, const FMassArchetypeSharedFragmentValues& SharedFragmentValuesToAdd)
{
	BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected, FMassTagBitSet(), SharedFragmentValuesToAdd);
}

void FMassEntityManager::BatchAddFragmentInstancesForEntities(TConstArrayView<FMassArchetypeEntityCollectionWithPayload> EntityCollections
	, const FMassFragmentBitSet& FragmentsAffected, const FMassTagBitSet& TagsToAdd, const FMassArchetypeSharedFragmentValues& SharedFragmentValuesToAdd)
{
	CHECK_SYNC_API();

	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchAddFragmentInstancesForEntities);

	testableCheckfReturn(FMassElementBitSet(TagsToAdd).GetSparseElements().IsEmpty()
		, return
		, TEXT("%hs: TagsToAdd contains sparse tag types."
			" Sparse elements must be handled at the command level via BatchAddSparseElementToEntities before calling this function.")
		, __FUNCTION__);

	// Sparse fragments are excluded from composition but their values are set via BatchSetFragmentValues.
	const FMassElementBitSet SparseElementsAffected = FragmentsAffected & FMassElementBitSet::GetAllSparseElements();
	FMassElementBitSet RegularFragmentsBits = FragmentsAffected;
	if (SparseElementsAffected.IsEmpty() == false)
	{
		RegularFragmentsBits -= SparseElementsAffected;
	}

	// here's the scenario:
	// * we get entities from potentially different archetypes
	// * adding a fragment instance consists of two operations: A) add fragment type & B) set fragment value
	//		* some archetypes might already have the "added" fragments so no need for step A
	//		* there might be an "empty" archetype in the mix - then step A results in archetype creation and assigning
	//		* if step A is required then the initial FMassArchetypeEntityCollection instance is no longer valid
	// * setting value can be done uniformly for all entities, remembering some might be in different chunks already
	// * @todo note that after adding fragment type some entities originally in different archetypes end up in the same
	//		archetype. This could be utilized as a basis for optimization. To be investigated.
	//
	const FMassElementBitSet AllRegularElementsToAdd = RegularFragmentsBits + TagsToAdd + SharedFragmentValuesToAdd.GetBitSet();
	const bool bElementsAddedAreObserved = ObserverManager.HasObserversForBitSet(AllRegularElementsToAdd, EMassObservedOperation::AddElement);

	for (const FMassArchetypeEntityCollectionWithPayload& EntityRangesWithPayload : EntityCollections)
	{
		FMassArchetypeHandle TargetArchetypeHandle = EntityRangesWithPayload.GetEntityCollection().GetArchetype();
		FMassArchetypeData* CurrentArchetype = TargetArchetypeHandle.DataPtr.Get();

		if (CurrentArchetype)
		{
			FMassArchetypeEntityCollection::FEntityRangeArray TargetArchetypeEntityRanges;
			FMassElementBitSet ElementsAdded;

			// New composition includes regular fragments, non-sparse tags, and shared fragment type bits
			FMassElementBitSet NewComposition = AllRegularElementsToAdd + CurrentArchetype->GetCompositionBitSet();

			if (CurrentArchetype->GetCompositionBitSet().IsEquivalent(NewComposition) == false)
			{
				if (bElementsAddedAreObserved)
				{
					ElementsAdded = AllRegularElementsToAdd - CurrentArchetype->GetCompositionBitSet();
				}

				FMassArchetypeHandle NewArchetypeHandle = InternalCreateSimilarArchetype(TargetArchetypeHandle.DataPtr, MoveTemp(NewComposition));
				checkSlow(NewArchetypeHandle.IsValid());

				TArray<FMassEntityHandle> EntitiesBeingMoved;
				const bool bAddSharedFragments = !SharedFragmentValuesToAdd.IsEmpty();
				CurrentArchetype->BatchMoveEntitiesToAnotherArchetype(EntityRangesWithPayload.GetEntityCollection(), *NewArchetypeHandle.DataPtr.Get()
					, EntitiesBeingMoved, &TargetArchetypeEntityRanges, bAddSharedFragments ? &SharedFragmentValuesToAdd : nullptr);

				for (const FMassEntityHandle& Entity : EntitiesBeingMoved)
				{
					check(GetEntityStorageInterface().IsValidIndex(Entity.Index));
					GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);
				}

				TargetArchetypeHandle = NewArchetypeHandle;
			}
			else
			{
				// @todo consider adding sparse elements to ElementsAdded once we start observing those.
				TargetArchetypeEntityRanges = EntityRangesWithPayload.GetEntityCollection().GetRanges();
				// The entity's composition already contains all target element types (including the shared fragment type bits from SharedFragmentValuesToAdd),
				// so no archetype move occurs. Updating shared fragment VALUES for already-present types requires a chunk move,
				// which BatchMoveEntitiesToAnotherArchetype does not support within the same archetype.
				// This matches the explicit limitation in BatchAddSharedFragmentsForEntities.
				testableCheckfReturn(SharedFragmentValuesToAdd.IsEmpty()
					, continue
					, TEXT("%hs: SharedFragmentValuesToAdd is non-empty but the entity composition is unchanged."
						" Updating shared fragment values for already-present types is not supported via this API.")
					, __FUNCTION__);
			}

			// Set per-entity fragment values (order of entities is preserved through the move).
			// At this point all the entities are in the target archetype, we can set the values.
			// Note that even though the "subchunk" information could have changed, the order of entities is the same and 
			// corresponds to the order in FMassArchetypeEntityCollectionWithPayload's payload
			// The payload must NOT contain tag arrays; tags have zero-size storage and are not in FragmentIndexMap.
			TargetArchetypeHandle.DataPtr->BatchSetFragmentValues(TargetArchetypeEntityRanges, EntityRangesWithPayload.GetPayload());

			if (bElementsAddedAreObserved && (ElementsAdded.IsEmpty() == false))
			{
				ObserverManager.OnCompositionChanged(
					FMassArchetypeEntityCollection(TargetArchetypeHandle, MoveTemp(TargetArchetypeEntityRanges))
					, MoveTemp(ElementsAdded)
					, EMassObservedOperation::AddElement);
			}
		}
		else
		{
			// Archetype-less entity: build with full composition (sparse elements handled at command level).
			// AllRegularElementsToAdd already combines non-sparse fragments, tags, and shared fragment type bits,
			// so a single path handles both the tag and no-tag cases.
			BatchBuildEntities(EntityRangesWithPayload, AllRegularElementsToAdd, SharedFragmentValuesToAdd);
		}
	}
}

void FMassEntityManager::BatchAddSharedFragmentsForEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
	, const FMassArchetypeSharedFragmentValues& AddedFragmentValues)
{
	CHECK_SYNC_API();

	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchAddConstSharedFragmentForEntities);

	const FMassElementBitSet ElementsToAddBitSet = AddedFragmentValues.GetBitSet();
	const bool bElementsAddedAreObserved = ObserverManager.HasObserversForBitSet(ElementsToAddBitSet, EMassObservedOperation::AddElement);

	for (const FMassArchetypeEntityCollection& Collection : EntityCollections)
	{
		FMassArchetypeData* CurrentArchetype = Collection.GetArchetype().DataPtr.Get();
		testableCheckfReturn(CurrentArchetype, continue, TEXT("Adding shared fragments to archetype-less entities is not supported"));

		FMassElementBitSet NewComposition(CurrentArchetype->GetCompositionBitSet());
		NewComposition += ElementsToAddBitSet;

		const FMassArchetypeHandle NewArchetypeHandle = CreateArchetype(NewComposition, FMassArchetypeCreationParams(*CurrentArchetype));
		check(NewArchetypeHandle.IsValid());
		FMassArchetypeData* NewArchetype = NewArchetypeHandle.DataPtr.Get();
		check(NewArchetype);
		if (!testableEnsureMsgf(CurrentArchetype != NewArchetype, TEXT("Setting shared fragment values without archetype change is not supported")))
		{
			UE_LOGF(LogMass, Warning, "Trying to set shared fragment values, without adding new shared fragments, is not supported.");
			continue;
		}

		// Compute BitSet of the elements actually added (i.e., not already present)
		FMassElementBitSet ElementsAdded;
		if (bElementsAddedAreObserved)
		{
			ElementsAdded = ElementsToAddBitSet - CurrentArchetype->GetCompositionBitSet();
		}

		TArray<FMassEntityHandle> EntitiesBeingMoved;
		FMassArchetypeEntityCollection::FEntityRangeArray NewArchetypeEntityRanges;

		CurrentArchetype->BatchMoveEntitiesToAnotherArchetype(Collection
			, *NewArchetype
			, EntitiesBeingMoved
			, !ElementsAdded.IsEmpty() ? &NewArchetypeEntityRanges : nullptr
			, &AddedFragmentValues);

		for (const FMassEntityHandle& Entity : EntitiesBeingMoved)
		{
			check(GetEntityStorageInterface().IsValidIndex(Entity.Index));

			GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);
		}

		if (!NewArchetypeEntityRanges.IsEmpty())
		{
			ObserverManager.OnCompositionChanged(
				FMassArchetypeEntityCollection(NewArchetypeHandle, MoveTemp(NewArchetypeEntityRanges))
				, MoveTemp(ElementsAdded)
				, EMassObservedOperation::AddElement);
		}
	}
}

void FMassEntityManager::BatchAddSparseElementToEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TNotNull<const UScriptStruct*> ElementType)
{
	CHECK_SPARSE_ELEMENT(ElementType);
	CHECK_SYNC_API();

	for (const FMassArchetypeEntityCollection& Collection : EntityCollections)
	{
		if (FMassArchetypeData* CurrentArchetype = Collection.GetArchetype().DataPtr.Get())
		{
			CurrentArchetype->BatchAddSparseElementToEntities(Collection, ElementType);
		}
	}
}

void FMassEntityManager::BatchRemoveSparseElementFromEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TNotNull<const UScriptStruct*> ElementType)
{
	CHECK_SPARSE_ELEMENT(ElementType);
	CHECK_SYNC_API();

	for (const FMassArchetypeEntityCollection& Collection : EntityCollections)
	{
		if (FMassArchetypeData* CurrentArchetype = Collection.GetArchetype().DataPtr.Get())
		{
			CurrentArchetype->BatchRemoveSparseElementFromEntities(Collection, ElementType);
		}
	}
}

void FMassEntityManager::BatchRemoveSharedFragmentFromEntities(
	TConstArrayView<FMassArchetypeEntityCollection> EntityCollections,
	const FMassSharedFragmentBitSet& ToRemove)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchRemoveSharedFragmentFromEntities);
	InternalBatchRemoveSharedFragmentsFromEntities(EntityCollections, FMassElementBitSet(ToRemove)
		, &ToRemove, /*ConstSharedFragmentToRemoveBitSet=*/nullptr);
}

void FMassEntityManager::BatchRemoveConstSharedFragmentFromEntities(
	TConstArrayView<FMassArchetypeEntityCollection> EntityCollections,
	const FMassConstSharedFragmentBitSet& ToRemove)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchRemoveConstSharedFragmentFromEntities);
	InternalBatchRemoveSharedFragmentsFromEntities(EntityCollections, FMassElementBitSet(ToRemove)
		, /*SharedFragmentToRemoveBitSet=*/nullptr, &ToRemove);
}

void FMassEntityManager::InternalBatchRemoveSharedFragmentsFromEntities(
	TConstArrayView<FMassArchetypeEntityCollection> EntityCollections,
	const FMassElementBitSet& ElementsToRemoveBitSet,
	const FMassSharedFragmentBitSet* SharedFragmentToRemoveBitSet,
	const FMassConstSharedFragmentBitSet* ConstSharedFragmentToRemoveBitSet)
{
	CHECK_SYNC_API();

	const bool bElementsRemovedAreObserved = ObserverManager.HasObserversForBitSet(
		ElementsToRemoveBitSet, EMassObservedOperation::RemoveElement);

	for (const FMassArchetypeEntityCollection& Collection : EntityCollections)
	{
		FMassArchetypeData* CurrentArchetype = Collection.GetArchetype().DataPtr.Get();
		if (!CurrentArchetype)
		{
			continue;
		}

		if (!CurrentArchetype->GetCompositionBitSet().HasAny(ElementsToRemoveBitSet))
		{
			continue;
		}

		if (bElementsRemovedAreObserved)
		{
			FMassElementBitSet ElementsRemoved = ElementsToRemoveBitSet.GetOverlap(CurrentArchetype->GetCompositionBitSet());
			if (ElementsRemoved.IsEmpty() == false)
			{
				ObserverManager.OnCompositionChanged(Collection, MoveTemp(ElementsRemoved), EMassObservedOperation::RemoveElement);
			}
		}

		FMassArchetypeCompositionDescriptor NewComposition(CurrentArchetype->GetCompositionDescriptor());
		NewComposition.GetElementsBitSet().Remove(ElementsToRemoveBitSet);
		const FMassArchetypeHandle NewArchetypeHandle = CreateArchetype(NewComposition);
		check(NewArchetypeHandle.IsValid());
		FMassArchetypeData* NewArchetype = NewArchetypeHandle.DataPtr.Get();
		check(NewArchetype);

		NewArchetype->CopyDebugNamesFrom(*CurrentArchetype);

		TArray<FMassEntityHandle> EntitiesBeingMoved;
		CurrentArchetype->BatchMoveEntitiesToAnotherArchetype(Collection,
			*NewArchetype,
			EntitiesBeingMoved, /*OutNewRanges=*/nullptr,
			/*SharedFragmentValuesToAdd=*/nullptr,
			SharedFragmentToRemoveBitSet, ConstSharedFragmentToRemoveBitSet);

		for (const FMassEntityHandle& Entity : EntitiesBeingMoved)
		{
			check(GetEntityStorageInterface().IsValidIndex(Entity.Index));
			GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);
		}
	}
}

void FMassEntityManager::MoveEntityToAnotherArchetype(FMassEntityHandle Entity, FMassArchetypeHandle NewArchetypeHandle,
	const FMassArchetypeSharedFragmentValues* SharedFragmentValuesOverride)
{
	CHECK_SYNC_API();

	CheckIfEntityIsActive(Entity);

	FMassArchetypeData& NewArchetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(NewArchetypeHandle);

	// Move the entity over
	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
	check(CurrentArchetype);

	FMassArchetypeCompositionDescriptor CompositionRemoved = CurrentArchetype->GetCompositionDescriptor().CalculateDifference(NewArchetype.GetCompositionDescriptor());
	ObserverManager.OnCompositionChanged(Entity, MoveTemp(CompositionRemoved), EMassObservedOperation::RemoveElement);
	
	CurrentArchetype->MoveEntityToAnotherArchetype(Entity, NewArchetype, SharedFragmentValuesOverride);
	GetEntityStorageInterface().SetArchetypeFromShared(Entity.Index, NewArchetypeHandle.DataPtr);

	FMassArchetypeCompositionDescriptor CompositionAdded = NewArchetype.GetCompositionDescriptor().CalculateDifference(CurrentArchetype->GetCompositionDescriptor());
	ObserverManager.OnCompositionChanged(Entity, MoveTemp(CompositionAdded), EMassObservedOperation::AddElement);
}

template<UE::Mass::CStructType TStruct>
void FMassEntityManager::SetEntityFragmentValues(FMassEntityHandle Entity, TConstArrayView<TStruct> FragmentInstanceList)
{
	CheckIfEntityIsActive(Entity);

	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
	check(CurrentArchetype);
	CurrentArchetype->SetFragmentsData(Entity, FragmentInstanceList);
}

template<UE::Mass::CStructType TStruct>
void FMassEntityManager::BatchSetEntityFragmentValues(const FMassArchetypeEntityCollection& SparseEntities, TConstArrayView<TStruct> FragmentInstanceList)
{
	if (FragmentInstanceList.Num())
	{
		BatchSetEntityFragmentValues(MakeArrayView(&SparseEntities, 1), FragmentInstanceList);
	}
}

template<UE::Mass::CStructType TStruct>
void FMassEntityManager::BatchSetEntityFragmentValues(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TConstArrayView<TStruct> FragmentInstanceList)
{
	CHECK_SYNC_API();

	if (FragmentInstanceList.IsEmpty())
	{
		return;
	}

	for (const FMassArchetypeEntityCollection& SparseEntities : EntityCollections)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchSetEntityFragmentValues);

		FMassArchetypeData* Archetype = SparseEntities.GetArchetype().DataPtr.Get();
		check(Archetype);

		for (const TStruct& FragmentTemplate : FragmentInstanceList)
		{
			Archetype->SetFragmentData(SparseEntities.GetRanges(), FragmentTemplate);
		}
	}
}

void* FMassEntityManager::InternalGetFragmentDataChecked(FMassEntityHandle Entity, const UScriptStruct* FragmentType) const
{
	// note that FragmentType is guaranteed to be of valid type - it's either statically checked by the template versions
	// or `checkf`ed by the non-template one
	CheckIfEntityIsActive(Entity);
	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
	check(CurrentArchetype);
	return CurrentArchetype->GetMutableFragmentDataForEntityChecked(FragmentType, Entity);
}

void* FMassEntityManager::InternalGetFragmentDataPtr(FMassEntityHandle Entity, const UScriptStruct* FragmentType) const
{
	// note that FragmentType is guaranteed to be of valid type - it's either statically checked by the template versions
	// or `checkf`ed by the non-template one
	FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
	check(CurrentArchetype);
	return CurrentArchetype->GetMutableFragmentDataForEntity(FragmentType, Entity);
}

const FConstSharedStruct* FMassEntityManager::InternalGetConstSharedFragmentPtr(FMassEntityHandle Entity, const UScriptStruct* ConstSharedFragmentType) const
{
	// note that ConstSharedFragmentType is guaranteed to be of valid type - it's either statically checked by the template versions
	// or `checkf`ed by the non-template one
	const FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
	check(CurrentArchetype);
	const FConstSharedStruct* SharedFragment = CurrentArchetype->GetSharedFragmentValues(Entity).GetConstSharedFragments().FindByPredicate(FStructTypeEqualOperator(ConstSharedFragmentType));
	return SharedFragment;
}

const FSharedStruct* FMassEntityManager::InternalGetSharedFragmentPtr(FMassEntityHandle Entity, const UScriptStruct* SharedFragmentType) const
{
	// note that SharedFragmentType is guaranteed to be of valid type - it's either statically checked by the template versions
	// or `checkf`ed by the non-template one
	const FMassArchetypeData* CurrentArchetype = GetEntityStorageInterface().GetArchetype(Entity.Index);
	check(CurrentArchetype);
	const FSharedStruct* SharedFragment = CurrentArchetype->GetSharedFragmentValues(Entity).GetSharedFragments().FindByPredicate(FStructTypeEqualOperator(SharedFragmentType));
	return SharedFragment;
}

bool FMassEntityManager::IsEntityActive(FMassEntityHandle Entity) const
{
	if (Entity.Index != UE::Mass::Private::InvalidEntityIndex)
	{
		const UE::Mass::FStorageType& EntityStorageInterface = GetEntityStorageInterface();
		return EntityStorageInterface.IsEntityActive(Entity);
	}
	return false;
}

bool FMassEntityManager::IsEntityValid(FMassEntityHandle Entity) const
{
	return (Entity.Index != UE::Mass::Private::InvalidEntityIndex) 
		&& GetEntityStorageInterface().IsValidIndex(Entity.Index) 
		&& (GetEntityStorageInterface().GetSerialNumber(Entity.Index) == Entity.SerialNumber);
}

bool FMassEntityManager::IsEntityBuilt(FMassEntityHandle Entity) const
{
	CheckIfEntityIsValid(Entity);
	const UE::Mass::IEntityStorageInterface::EEntityState CurrentState = GetEntityStorageInterface().GetEntityState(Entity.Index);
	return CurrentState == UE::Mass::IEntityStorageInterface::EEntityState::Created;
}

bool FMassEntityManager::IsEntityReserved(FMassEntityHandle EntityHandle) const
{
	CheckIfEntityIsValid(EntityHandle);
	return GetEntityStorageInterface().GetEntityState(EntityHandle.Index) == UE::Mass::IEntityStorageInterface::EEntityState::Reserved;
}

FMassEntityHandle FMassEntityManager::CreateEntityIndexHandle(const int32 EntityIndex) const
{
	return (GetEntityStorageInterface().IsValidIndex(EntityIndex)
		&& GetEntityStorageInterface().GetEntityState(EntityIndex) == UE::Mass::IEntityStorageInterface::EEntityState::Created)
		? FMassEntityHandle(EntityIndex, GetEntityStorageInterface().GetSerialNumber(EntityIndex))
		: FMassEntityHandle();
}

void FMassEntityManager::GetMatchingArchetypes(const FMassFragmentRequirements& Requirements, TArray<FMassArchetypeHandle>& OutValidArchetypes, const uint32 FromArchetypeDataVersion) const
{
	for (int32 ArchetypeIndex = FromArchetypeDataVersion; ArchetypeIndex < AllArchetypes.Num(); ++ArchetypeIndex)
	{
		checkf(AllArchetypes[ArchetypeIndex].IsValid(), TEXT("We never expect to get any invalid shared ptrs in AllArchetypes"));

		FMassArchetypeData& Archetype = *(AllArchetypes[ArchetypeIndex].Get());

		// Only return archetypes with a newer created version than the specified version, this is for incremental query updates
		ensureMsgf(Archetype.GetCreatedArchetypeDataVersion() > FromArchetypeDataVersion
			, TEXT("There's a stron assumption that archetype's data version corresponds to its index in AllArchetypes"));

		if (Requirements.DoesArchetypeMatchRequirements(Archetype.GetCompositionDescriptor()))
		{
			OutValidArchetypes.Add(AllArchetypes[ArchetypeIndex]);
		}
#if WITH_MASSENTITY_DEBUG
		else
		{
			UE_VLOG_UELOG(GetOwner(), LogMass, VeryVerbose, TEXT("%s")
				, *FMassDebugger::GetArchetypeRequirementCompatibilityDescription(Requirements, Archetype.GetCompositionDescriptor()));
		}
#endif // WITH_MASSENTITY_DEBUG
	}
}

FStructView FMassEntityManager::GetMutableSparseElementDataForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle)
{
	check(UE::Mass::IsA<FMassTag>(ElementType) == false);
	return SparseElementsStorage.GetMutableElementDataForEntity(EntityHandle, ElementType);
}

FConstStructView FMassEntityManager::GetSparseElementDataForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle) const
{
	check(UE::Mass::IsA<FMassTag>(ElementType) == false);
	return SparseElementsStorage.GetElementDataForEntity(EntityHandle, ElementType);
}

FMassElementBitSet FMassEntityManager::GetSparseElementsBitSetForEntity(const FMassEntityHandle EntityHandle) const
{
	FMassArchetypeHandle ArchetypeHandle = GetArchetypeForEntity(EntityHandle);
	return ArchetypeHandle.IsValid()
		? ArchetypeHandle.DataPtr->GetSparseElementsBitSetForEntity(EntityHandle)
		: FMassElementBitSet();
}

FMassExecutionContext FMassEntityManager::CreateExecutionContext(const float DeltaSeconds)
{
	FMassExecutionContext ExecutionContext(*this, DeltaSeconds);
	ExecutionContext.SetDeferredCommandBuffer(DeferredCommandBuffers[OpenedCommandBufferIndex]);
	return MoveTemp(ExecutionContext);
}

void FMassEntityManager::FlushCommands(const TSharedPtr<FMassCommandBuffer>& InCommandBuffer)
{
	if (!ensureMsgf(IsInGameThread(), TEXT("Calling %hs is supported only on the Game Tread"), __FUNCTION__))
	{
		return;
	}

	if (!ensureMsgf(IsProcessing() == false, TEXT("Calling %hs is not supported while Mass Processing is active. Call FMassEntityManager::AppendCommands instead."), __FUNCTION__))
	{
		return;
	}

	if (UNLIKELY(InitializationState != EInitializationState::Initialized))
	{
		UE_CVLOG_UELOG(InitializationState == EInitializationState::Uninitialized, GetOwner(), LogMass, Warning
			, TEXT("FlushCommands called before Initialize call, which means this FMassEntityManager instance is not ready to process commands and will cancel them."));
		UE_CVLOG_UELOG(InitializationState == EInitializationState::Deinitialized, GetOwner(), LogMass, Log
			, TEXT("FlushCommands called after Deinitialize call, which means this FMassEntityManager instance is going away, can't process commands and will cancel them."));
		InCommandBuffer->CancelCommands();
		return;
	}
	
	if (InCommandBuffer && InCommandBuffer->HasPendingCommands()
		&& (Algo::Find(DeferredCommandBuffers, InCommandBuffer) == nullptr))
	{
		AppendCommands(InCommandBuffer);
	}
	FlushCommands();
}

void FMassEntityManager::FlushCommands()
{
	constexpr int32 MaxIterations = 5;

	if (!ensureMsgf(IsInGameThread(), TEXT("Calling %hs is supported only on the Game Tread"), __FUNCTION__))
	{
		return;
	}
	if (!ensureMsgf(IsProcessing() == false, TEXT("Calling %hs is not supported while Mass Processing is active. Call FMassEntityManager::AppendCommands instead."), __FUNCTION__))
	{
		return;
	}

	if (bCommandBufferFlushingInProgress == false && IsProcessing() == false)
	{
		ON_SCOPE_EXIT
		{
			bCommandBufferFlushingInProgress = false;
		};
		bCommandBufferFlushingInProgress = true;

		int32 IterationCount = 0;
		do 
		{
			const int32 CommandBufferIndexToFlush = OpenedCommandBufferIndex;

			// buffer swap. Code instigated by observers can still use Defer() to push commands.
			OpenedCommandBufferIndex = (OpenedCommandBufferIndex + 1) % DeferredCommandBuffers.Num();
			ensureMsgf(DeferredCommandBuffers[OpenedCommandBufferIndex]->HasPendingCommands() == false
				, TEXT("The freshly opened command buffer is expected to be empty upon switching"));

			DeferredCommandBuffers[CommandBufferIndexToFlush]->Flush(*this);

			// repeat if there were commands submitted while commands were being flushed (by observers for example)
		} while (DeferredCommandBuffers[OpenedCommandBufferIndex]->HasPendingCommands() && ++IterationCount < MaxIterations);

		UE_CVLOG_UELOG(IterationCount >= MaxIterations, GetOwner(), LogMass, Error, TEXT("Reached loop count limit while flushing commands. Limiting the number of commands pushed during commands flushing could help."));
	}
}

void FMassEntityManager::AppendCommands(const TSharedPtr<FMassCommandBuffer>& InOutCommandBuffer)
{
	if (!ensureMsgf(Algo::Find(DeferredCommandBuffers, InOutCommandBuffer) == nullptr
		, TEXT("We don't expect AppendCommands to be called with EntityManager's command buffer as the input parameter")))
	{
		return;
	}
	LLM_SCOPE_BYNAME(TEXT("Mass/EntityManager"));
	Defer().MoveAppend(*InOutCommandBuffer.Get());
}

TSharedRef<FMassEntityManager::FEntityCreationContext> FMassEntityManager::GetOrMakeCreationContext()
{
	return ObserverManager.GetOrMakeCreationContext();
}

UE::Mass::FEntityBuilder FMassEntityManager::MakeEntityBuilder()
{
	return UE::Mass::FEntityBuilder(AsShared());
}

void FMassEntityManager::OnNewTypeRegistered(UE::Mass::FTypeHandle RegisteredTypeHandle)
{
	const UE::Mass::FTypeInfo* NewTypeInfo = TypeManager->GetTypeInfo(RegisteredTypeHandle);
	if (testableEnsureMsgf(NewTypeInfo, TEXT("%hs input handle doesn't represent a registered type"), __FUNCTION__))
	{
		if (const UE::Mass::FRelationTypeTraits* RelationTypeTraits = NewTypeInfo->GetAsRelationTraits())
		{
			OnRelationTypeRegistered(RegisteredTypeHandle, *RelationTypeTraits);
		}
	}
}

void FMassEntityManager::OnRelationTypeRegistered(UE::Mass::FTypeHandle RegisteredTypeHandle, const UE::Mass::FRelationTypeTraits& RelationTypeTraits)
{
	if (RelationTypeTraits.RegisterObserversDelegate.IsSet() == false
		|| RelationTypeTraits.RegisterObserversDelegate(*this) == true)
	{
		RelationManager.OnRelationTypeRegistered(RegisteredTypeHandle, RelationTypeTraits);
	}
}

bool FMassEntityManager::BatchCreateRelations(const UE::Mass::FTypeHandle RelationTypeHandle, TArrayView<FMassEntityHandle> Subjects, TArrayView<FMassEntityHandle> Objects)
{
	return RelationManager.CreateRelationInstances(RelationTypeHandle, Subjects, Objects).Num() > 0;
}

//-----------------------------------------------------------------------------
// DEBUG stuff
//-----------------------------------------------------------------------------
bool FMassEntityManager::DebugDoCollectionsOverlapCreationContext(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections) const
{
	if (TSharedPtr<FMassObserverManager::FCreationContext> AsSharedPtr = ObserverManager.GetCreationContext())
	{
		TArray<FMassArchetypeEntityCollection> CreationCollections = AsSharedPtr->GetEntityCollections(*this);
		return CreationCollections.GetData() <= EntityCollections.GetData()
			&& EntityCollections.GetData() <= CreationCollections.GetData() + CreationCollections.Num();
	}

	return false;
}

void FMassEntityManager::SetDebugName(const FString& NewDebugGame) 
{ 
#if WITH_MASSENTITY_DEBUG
	DebugName = NewDebugGame; 
#endif // WITH_MASSENTITY_DEBUG
}

#if WITH_MASSENTITY_DEBUG
void FMassEntityManager::DebugPrintArchetypes(FOutputDevice& Ar, const bool bIncludeEmpty) const
{
	Ar.Logf(ELogVerbosity::Log, TEXT("Listing archetypes contained in EntityManager owned by %s"), *GetPathNameSafe(GetOwner()));

	int32 NumBuckets = 0;
	int32 NumArchetypes = 0;
	int32 LongestArchetypeBucket = 0;
	for (const auto& KVP : FragmentHashToArchetypeMap)
	{
		for (const TSharedPtr<FMassArchetypeData>& ArchetypePtr : KVP.Value)
		{
			if (ArchetypePtr.IsValid() && (bIncludeEmpty == true || ArchetypePtr->GetChunkCount() > 0))
			{
				ArchetypePtr->DebugPrintArchetype(Ar);
			}
		}

		const int32 NumArchetypesInBucket = KVP.Value.Num();
		LongestArchetypeBucket = FMath::Max(LongestArchetypeBucket, NumArchetypesInBucket);
		NumArchetypes += NumArchetypesInBucket;
		++NumBuckets;
	}

	Ar.Logf(ELogVerbosity::Log, TEXT("FragmentHashToArchetypeMap: %d archetypes across %d buckets, longest bucket is %d"),
		NumArchetypes, NumBuckets, LongestArchetypeBucket);
}

void FMassEntityManager::DebugGetArchetypesStringDetails(FOutputDevice& Ar, const bool bIncludeEmpty) const
{
	Ar.SetAutoEmitLineTerminator(true);
	for (auto Pair : FragmentHashToArchetypeMap)
	{
		Ar.Logf(ELogVerbosity::Log, TEXT("\n-----------------------------------\nHash: %u"), Pair.Key);
		for (TSharedPtr<FMassArchetypeData> Archetype : Pair.Value)
		{
			if (Archetype.IsValid() && (bIncludeEmpty == true || Archetype->GetChunkCount() > 0))
			{
				Archetype->DebugPrintArchetype(Ar);
				Ar.Logf(ELogVerbosity::Log, TEXT("+++++++++++++++++++++++++\n"));
			}
		}
	}
}

void FMassEntityManager::DebugGetArchetypeFragmentTypes(const FMassArchetypeHandle& Archetype, TArray<const UScriptStruct*>& InOutFragmentList) const
{
	if (Archetype.IsValid())
	{
		const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);
		ArchetypeData.GetCompositionDescriptor().Get<FMassFragmentBitSet>().ExportTypes(InOutFragmentList);
	}
}

int32 FMassEntityManager::DebugGetArchetypeEntitiesCount(const FMassArchetypeHandle& Archetype) const
{
	return Archetype.IsValid() ? FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype).GetNumEntities() : 0;
}

int32 FMassEntityManager::DebugGetArchetypeEntitiesCountPerChunk(const FMassArchetypeHandle& Archetype) const
{
	return Archetype.IsValid() ? FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype).GetNumEntitiesPerChunk() : 0;
}

int32 FMassEntityManager::DebugGetEntityCount() const
{
	return GetEntityStorageInterface().Num() - NumReservedEntities - GetEntityStorageInterface().ComputeFreeSize();
}

int32 FMassEntityManager::DebugGetArchetypesCount() const
{
	return AllArchetypes.Num();
}

void FMassEntityManager::DebugRemoveAllEntities()
{
	for (int EntityIndex = NumReservedEntities, EndIndex = GetEntityStorageInterface().Num(); EntityIndex < EndIndex; ++EntityIndex)
	{
		if (GetEntityStorageInterface().IsValid(EntityIndex) == false)
		{
			// already dead
			continue;
		}
		FMassArchetypeData* Archetype = GetEntityStorageInterface().GetArchetype(EntityIndex);
		check(Archetype);
		FMassEntityHandle Entity;
		Entity.Index = EntityIndex;
		Entity.SerialNumber = GetEntityStorageInterface().GetSerialNumber(EntityIndex);
		Archetype->RemoveEntity(Entity);

		GetEntityStorageInterface().ForceReleaseOne(Entity);
	}
}

void FMassEntityManager::DebugForceArchetypeDataVersionBump()
{
	++ArchetypeDataVersion;
}

void FMassEntityManager::DebugGetArchetypeStrings(const FMassArchetypeHandle& Archetype, TArray<FName>& OutFragmentNames, TArray<FName>& OutTagNames)
{
	if (Archetype.IsValid() == false)
	{
		return;
	}

	const FMassArchetypeData& ArchetypeRef = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);
	
	OutFragmentNames.Reserve(ArchetypeRef.GetFragmentConfigs().Num());
	for (const FMassArchetypeFragmentConfig& FragmentConfig : ArchetypeRef.GetFragmentConfigs())
	{
		checkSlow(FragmentConfig.FragmentType);
		OutFragmentNames.Add(FragmentConfig.FragmentType->GetFName());
	}

	ArchetypeRef.GetCompositionBitSet().Get<FMassTagBitSet>().DebugGetIndividualNames(OutTagNames);
}

FMassEntityHandle FMassEntityManager::DebugGetEntityIndexHandle(const int32 EntityIndex) const
{
	return CreateEntityIndexHandle(EntityIndex);
}

const FString& FMassEntityManager::DebugGetName() const
{
	return DebugName;
}

void FMassEntityManager::DebugEnableDebugFeature(EDebugFeatures Features)
{
	EnumAddFlags(EnabledDebugFeatures, Features);
}

void FMassEntityManager::DebugDisableDebugFeature(EDebugFeatures Features)
{
	EnumRemoveFlags(EnabledDebugFeatures, Features);
}

bool FMassEntityManager::DebugHasAllDebugFeatures(EDebugFeatures Features) const
{
	return EnumHasAllFlags(EnabledDebugFeatures, Features);
}

FMassRequirementAccessDetector& FMassEntityManager::GetRequirementAccessDetector()
{
	return RequirementAccessDetector;
}

UE::Mass::FStorageType& FMassEntityManager::DebugGetEntityStorageInterface()
{
	return GetEntityStorageInterface();
}

const UE::Mass::FStorageType& FMassEntityManager::DebugGetEntityStorageInterface() const
{
	return GetEntityStorageInterface();
}

bool FMassEntityManager::DebugHasCommandsToFlush() const
{
	checkfSlow(NumCommandBuffers == 2, TEXT("This check relies on there being two command buffers."));
	return DeferredCommandBuffers[0]->HasPendingCommands() || DeferredCommandBuffers[1]->HasPendingCommands();
}

UE::Mass::FSparseElementsStorage& FMassEntityManager::DebugGetSparseElementsStorage()
{
	return SparseElementsStorage;
}
#endif // WITH_MASSENTITY_DEBUG

#undef CHECK_SYNC_API
#undef CHECK_SYNC_API_RETURN
#undef CHECK_ELEMENT