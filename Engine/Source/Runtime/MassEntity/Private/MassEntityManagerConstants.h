// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::Mass::Private
{
	// Index 0 is a sentinel for an Empty/Unset EntityHandle
	constexpr int32 InvalidEntityIndex = 0;
} // namespace UE::Mass::Private