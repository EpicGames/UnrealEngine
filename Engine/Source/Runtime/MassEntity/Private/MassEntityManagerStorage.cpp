// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManagerStorage.h"
#include "MassEntityManagerConstants.h"
#include "Mass/EntityHandle.h"
#include "MassEntityTypes.h"
#include "Templates/SharedPointer.h"
#include "Async/UniqueLock.h"

namespace UE::Mass
{
	//-----------------------------------------------------------------------------
	// IEntityStorageInterface
	//-----------------------------------------------------------------------------
	int32 IEntityStorageInterface::Acquire(const int32 Count, TArray<FMassEntityHandle>& OutEntityHandles)
	{
		if (Count)
		{
			const int32 StartingIndex = OutEntityHandles.Num();
			OutEntityHandles.AddZeroed(Count);
			const int32 NumberAdded = Acquire(MakeArrayView(&OutEntityHandles[StartingIndex], Count));
			if (UNLIKELY(NumberAdded < Count))
			{
				// need to remove the redundantly reserved entries
				OutEntityHandles.RemoveAt(StartingIndex + NumberAdded, Count - NumberAdded, EAllowShrinking::No);
			}
			return NumberAdded;
		}
		return 0;
	}

	//-----------------------------------------------------------------------------
	// FSingleThreadedEntityStorage
	//-----------------------------------------------------------------------------
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FSingleThreadedEntityStorage::Initialize(const FMassEntityManager_InitParams_SingleThreaded&)
	{
		// Index 0 is reserved so we can treat that index as an invalid entity handle
		const FMassEntityHandle SentinelEntity = AcquireOne();
		check(SentinelEntity.Index == UE::Mass::Private::InvalidEntityIndex);
	}

	FMassArchetypeData* FSingleThreadedEntityStorage::GetArchetype(int32 Index)
	{
		return Entities[Index].CurrentArchetype.Get();
	}

	const FMassArchetypeData* FSingleThreadedEntityStorage::GetArchetype(int32 Index) const
	{
		return Entities[Index].CurrentArchetype.Get();
	}

	TSharedPtr<FMassArchetypeData>& FSingleThreadedEntityStorage::GetArchetypeAsShared(int32 Index)
	{
		return Entities[Index].CurrentArchetype;
	}

	const TSharedPtr<FMassArchetypeData>& FSingleThreadedEntityStorage::GetArchetypeAsShared(int32 Index) const
	{
		return Entities[Index].CurrentArchetype;
	}

	void FSingleThreadedEntityStorage::SetArchetypeFromShared(int32 Index, TSharedPtr<FMassArchetypeData>& Archetype)
	{
		Entities[Index].CurrentArchetype = Archetype;
	}

	void FSingleThreadedEntityStorage::SetArchetypeFromShared(int32 Index, const TSharedPtr<FMassArchetypeData>& Archetype)
	{
		Entities[Index].CurrentArchetype = Archetype;
	}

	IEntityStorageInterface::EEntityState FSingleThreadedEntityStorage::GetEntityState(int32 Index) const
	{
		const uint32 CurrentSerialNumber = Entities[Index].SerialNumber;

		if (CurrentSerialNumber != 0)
		{
			return Entities[Index].CurrentArchetype.Get()
				? EEntityState::Created 
				: EEntityState::Reserved;
		}

		return EEntityState::Free;	
	}

	int32 FSingleThreadedEntityStorage::GetSerialNumber(int32 Index) const
	{
		return Entities[Index].SerialNumber;
	}

	bool FSingleThreadedEntityStorage::IsValidIndex(int32 Index) const
	{
		return Entities.IsValidIndex(Index);
	}

	bool FSingleThreadedEntityStorage::IsValidHandle(FMassEntityHandle EntityHandle) const
	{
		return Entities.IsValidIndex(EntityHandle.Index)
			&& Entities[EntityHandle.Index].SerialNumber == EntityHandle.SerialNumber;
	}

	bool FSingleThreadedEntityStorage::IsEntityActive(FMassEntityHandle EntityHandle) const
	{
		return IsValidIndex(EntityHandle.Index)
			&& GetSerialNumber(EntityHandle.Index) == EntityHandle.SerialNumber
			&& GetEntityState(EntityHandle.Index) == UE::Mass::IEntityStorageInterface::EEntityState::Created;
	}

	SIZE_T FSingleThreadedEntityStorage::GetAllocatedSize() const
	{
		return Entities.GetAllocatedSize() + EntityFreeIndexList.GetAllocatedSize();
	}

	bool FSingleThreadedEntityStorage::IsValid(int32 Index) const
	{
		return Entities[Index].IsValid();
	}

	FMassEntityHandle FSingleThreadedEntityStorage::AcquireOne()
	{
		LLM_SCOPE_BYNAME(TEXT("Mass/SingleThreadedStorage"));
		const int32 SerialNumber = GenerateSerialNumber();
		const int32 Index = (EntityFreeIndexList.Num() > 0) ? EntityFreeIndexList.Pop(EAllowShrinking::No) : Entities.Add();
		Entities[Index].SerialNumber = SerialNumber;

		FMassEntityHandle Handle;
		Handle.SerialNumber = SerialNumber;
		Handle.Index = Index;
		return Handle;
	}

	int32 FSingleThreadedEntityStorage::Acquire(TArrayView<FMassEntityHandle> OutEntityHandles)
	{
		LLM_SCOPE_BYNAME(TEXT("Mass/SingleThreadedStorage"));
		const int32 NumToAdd = OutEntityHandles.Num();

		const int32 SerialNumber = GenerateSerialNumber();

		int32 NumAdded = 0;
		int32 CurrentEntityHandleIndex = 0;

		const int32 NumAvailableFromFreeList = FMath::Min(NumToAdd, EntityFreeIndexList.Num());
		if (NumAvailableFromFreeList > 0)
		{
			const int32 FirstIndexToUse = EntityFreeIndexList.Num() - NumAvailableFromFreeList;
			for (int32 Index = FirstIndexToUse; Index < EntityFreeIndexList.Num(); ++Index)
			{
				const int32 EntityIndex = EntityFreeIndexList[Index];
				Entities[EntityIndex].SerialNumber = SerialNumber;
				OutEntityHandles[CurrentEntityHandleIndex++] = { EntityIndex, SerialNumber };
			}
			EntityFreeIndexList.RemoveAt(FirstIndexToUse, NumAvailableFromFreeList, EAllowShrinking::No);
			NumAdded = NumAvailableFromFreeList;
		}

		if (NumAdded < NumToAdd)
		{
			const int32 RemainingCount = NumToAdd - NumAdded;
			const int32 StartingIndex = Entities.Num();
			Entities.Add(RemainingCount);
			for (int32 EntityIndex = StartingIndex; EntityIndex < Entities.Num(); ++EntityIndex)
			{
				Entities[EntityIndex].SerialNumber = SerialNumber;
				OutEntityHandles[CurrentEntityHandleIndex++] = { EntityIndex, SerialNumber };
			}
			NumAdded += RemainingCount;
		}

		return NumAdded;
	}

	int32 FSingleThreadedEntityStorage::Release(TConstArrayView<FMassEntityHandle> Handles)
	{
		LLM_SCOPE_BYNAME(TEXT("Mass/SingleThreadedStorage"));
		int DeallocateCount = 0;

		EntityFreeIndexList.Reserve(EntityFreeIndexList.Num() + Handles.Num());

		for (const FMassEntityHandle& Handle : Handles)
		{
			FEntityData& EntityData = Entities[Handle.Index];
			if (EntityData.SerialNumber == Handle.SerialNumber)
			{
				EntityData.Reset();
				EntityFreeIndexList.Add(Handle.Index);
				++DeallocateCount;
			}
		}
	
		return DeallocateCount;
	}

	int32 FSingleThreadedEntityStorage::ReleaseOne(FMassEntityHandle Handle)
	{
		return Release(MakeArrayView(&Handle, 1));
	}

	int32 FSingleThreadedEntityStorage::ForceRelease(TConstArrayView<FMassEntityHandle> Handles)
	{
		LLM_SCOPE_BYNAME(TEXT("Mass/SingleThreadedStorage"));
		EntityFreeIndexList.Reserve(EntityFreeIndexList.Num() + Handles.Num());
		for (const FMassEntityHandle& Handle : Handles)
		{
			FEntityData& EntityData = Entities[Handle.Index];
			EntityData.Reset();
			EntityFreeIndexList.Add(Handle.Index);
		}
		return Handles.Num();
	}

	int32 FSingleThreadedEntityStorage::ForceReleaseOne(FMassEntityHandle Handle)
	{
		return ForceRelease(MakeArrayView(&Handle, 1));
	}

	int32 FSingleThreadedEntityStorage::Num() const
	{
		return Entities.Num();
	}

	int32 FSingleThreadedEntityStorage::ComputeFreeSize() const
	{
		return EntityFreeIndexList.Num();
	}

	//-----------------------------------------------------------------------------
	// FSingleThreadedEntityStorage::FEntityData
	//-----------------------------------------------------------------------------

	FSingleThreadedEntityStorage::FEntityData::~FEntityData() = default;

	void FSingleThreadedEntityStorage::FEntityData::Reset()
	{
		CurrentArchetype.Reset();
		SerialNumber = 0;
	}

	bool FSingleThreadedEntityStorage::FEntityData::IsValid() const
	{
		return SerialNumber != 0 && CurrentArchetype.IsValid();
	}
	
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	//-----------------------------------------------------------------------------
	// FConcurrentEntityStorage
	//-----------------------------------------------------------------------------

	void FConcurrentEntityStorage::Initialize(const FMassEntityManager_InitParams_Concurrent& InInitializationParams)
	{
		LLM_SCOPE_BYNAME(TEXT("Mass/ConcurrentStorage"));
		// Compute number of pages required
		check(FMath::IsPowerOfTwo(InInitializationParams.MaxEntitiesPerPage));
		check(FMath::IsPowerOfTwo(InInitializationParams.MaxEntityCount));
		MaxEntitiesPerPage = InInitializationParams.MaxEntitiesPerPage;
		MaxEntitiesPerPageShift = FMath::FloorLog2(InInitializationParams.MaxEntitiesPerPage);
		MaximumEntityCountShift = FMath::FloorLog2(InInitializationParams.MaxEntityCount);
		checkf(MaximumEntityCountShift < 32, TEXT("Invalid maximum entity count, cannot exceed 31 bits"));

		const uint64 PagePointerCount = InInitializationParams.MaxEntityCount / InInitializationParams.MaxEntitiesPerPage;

		const uint64 EntityPageSize = sizeof(void*) * PagePointerCount;
		EntityPages = static_cast<FEntityData**>(FMemory::Malloc(EntityPageSize, alignof(FEntityData**)));
		FMemory::Memzero(EntityPages, EntityPageSize);	
	}

	FConcurrentEntityStorage::~FConcurrentEntityStorage()
	{
		if (EntityPages != nullptr)
		{
			for (uint32 Index = 0; Index < PageCount; ++Index)
			{
				FMemory::Free(EntityPages[Index]);
				EntityPages[Index] = nullptr;
			}
			FMemory::Free(EntityPages);
			EntityPages = nullptr;
		}
	}

	FMassArchetypeData* FConcurrentEntityStorage::GetArchetype(int32 Index)
	{
		return LookupEntity(Index).CurrentArchetype.Get();
	}

	const FMassArchetypeData* FConcurrentEntityStorage::GetArchetype(int32 Index) const
	{
		return LookupEntity(Index).CurrentArchetype.Get();
	}

	TSharedPtr<FMassArchetypeData>& FConcurrentEntityStorage::GetArchetypeAsShared(int32 Index)
	{
		return LookupEntity(Index).CurrentArchetype;
	}

	const TSharedPtr<FMassArchetypeData>& FConcurrentEntityStorage::GetArchetypeAsShared(int32 Index) const
	{
		return LookupEntity(Index).CurrentArchetype;
	}

	void FConcurrentEntityStorage::SetArchetypeFromShared(int32 Index, TSharedPtr<FMassArchetypeData>& Archetype)
	{
		LookupEntity(Index).CurrentArchetype = Archetype;
	}

	void FConcurrentEntityStorage::SetArchetypeFromShared(int32 Index, const TSharedPtr<FMassArchetypeData>& Archetype)
	{
		LookupEntity(Index).CurrentArchetype = Archetype;
	}

	IEntityStorageInterface::EEntityState FConcurrentEntityStorage::GetEntityStateInternal(const FEntityData& EntityData) const
	{
		//
		// || Archetype || bIsAllocated || Result    |
		//  |  nullptr   |      0       |  Free     |
		//  |  nullptr   |      1       |  Reserved |
		//  | !nullptr   |      1       |  Created  |
		//	
		if (EntityData.CurrentArchetype != nullptr)
		{
			return EEntityState::Created;
		}

		return EntityData.bIsAllocated
			? EEntityState::Reserved
			: EEntityState::Free;
	}

	IEntityStorageInterface::EEntityState FConcurrentEntityStorage::GetEntityState(int32 Index) const
	{
		return GetEntityStateInternal(LookupEntity(Index));
	}

	int32 FConcurrentEntityStorage::GetSerialNumber(int32 Index) const
	{
		return LookupEntity(Index).GenerationId;
	}

	bool FConcurrentEntityStorage::IsValidIndex(int32 Index) const
	{
		// Page Index is which page in the array of pages we need to access
		if (Index >= 0)
		{
			const uint32 PageIndex = static_cast<uint32>(Index) >> MaxEntitiesPerPageShift;
			return PageIndex < PageCount;
		}
		return false;
	}

	bool FConcurrentEntityStorage::IsValidHandle(FMassEntityHandle EntityHandle) const
	{
		return IsValidIndex(EntityHandle.Index)
			&& LookupEntity(EntityHandle.Index).GetSerialNumber() == EntityHandle.SerialNumber;
	}

	bool FConcurrentEntityStorage::IsEntityActive(FMassEntityHandle EntityHandle) const
	{
		if (IsValidIndex(EntityHandle.Index))
		{
			const FEntityData& EntityData = LookupEntity(EntityHandle.Index);
			return EntityData.GetSerialNumber() == EntityHandle.SerialNumber
				&& GetEntityStateInternal(EntityData) == UE::Mass::IEntityStorageInterface::EEntityState::Created;
		}
		return false;
	}

	SIZE_T FConcurrentEntityStorage::GetAllocatedSize() const
	{
		const SIZE_T EntityFreeListSizeBytes = EntityFreeIndexList.GetAllocatedSize();

		// Allocated size to pages
		const SIZE_T PageSizeBytes = ComputePageSize();
		const SIZE_T PageAllocatedSizeBytes = PageCount * PageSizeBytes;

		// Size of page pointer array
		const uint32 MaxEntities = 1 << MaximumEntityCountShift;
		const uint32 MaxPageCount = (MaxEntities / MaxEntitiesPerPage);
		const SIZE_T PagePointerArraySizeBytes = MaxPageCount * sizeof(FEntityData**);
	
		return PageAllocatedSizeBytes + PagePointerArraySizeBytes + EntityFreeListSizeBytes;
	}

	bool FConcurrentEntityStorage::IsValid(int32 Index) const
	{
		return LookupEntity(Index).CurrentArchetype != nullptr;
	}

	bool FConcurrentEntityStorage::AddPage()
	{
		LLM_SCOPE_BYNAME(TEXT("Mass/ConcurrentStorage"));
		check(FreeListMutex.IsLocked());
		UE::TUniqueLock PageAllocateLock(PageAllocateMutex);

		// Allocate new page
		const uint32 NewPageIndex = PageCount;
		checkf(((NewPageIndex + 1) << MaxEntitiesPerPageShift) < (1u << MaximumEntityCountShift), TEXT("Exhausted number of entities"));

		const uint64 PageSize = ComputePageSize();
		FEntityData* Page = static_cast<FEntityData*>(FMemory::Malloc(PageSize, alignof(FEntityData)));
		
		if (Page == nullptr)
		{
			return false;
		}

		/*for (int32 Index = 0, End = MaxEntitiesPerPage; Index < End; ++Index)
		{
			new (Page + Index) FEntityData();
		}*/
		FMemory::Memzero(Page, PageSize);

		EntityPages[PageCount] = Page;
		++PageCount;

		// Somewhat tricksy thing here to be aware of
		// MassEntityManager expects the very first allocated entity to be at index 0
		static_assert(UE::Mass::Private::InvalidEntityIndex == 0, "Free Entity list algorithm depends on InvalidEntityIndex being 0");
		int32 NewEntityIndexStart;
		if (LIKELY(NewPageIndex != 0))
		{
			NewEntityIndexStart = NewPageIndex << MaxEntitiesPerPageShift;
		}
		else
		{
			NewEntityIndexStart = 1;
			// Allocate the 0th entity. It will always be the sentinel entity that InvalidEntityIndex points to.
			FEntityData* SentinelEntity = new (Page + UE::Mass::Private::InvalidEntityIndex) FEntityData();
			SentinelEntity->bIsAllocated = 1;
			++SentinelEntity->GenerationId;
		}
		
		const int32 NewEntityIndexEnd = (NewPageIndex + 1) << MaxEntitiesPerPageShift;

		EntityFreeIndexList.Reserve(NewEntityIndexEnd - NewEntityIndexStart);

		// Push free entities indices onto the stack backwards so new entities pop off in order
		for (int32 NewEntityIndex = NewEntityIndexEnd - 1; NewEntityIndex >= NewEntityIndexStart; --NewEntityIndex)
		{
			// Setup the free list
			EntityFreeIndexList.Push(NewEntityIndex);
		}

		return true;
	}

	FMassEntityHandle FConcurrentEntityStorage::AcquireOne()
	{
		int32 EntityIndex;
		{
			UE::TUniqueLock FreeListLock(FreeListMutex);
		
			if (UNLIKELY(EntityFreeIndexList.IsEmpty()))
			{
				AddPage();
			}

			EntityIndex = EntityFreeIndexList.Pop(EAllowShrinking::No);

			++EntityCount;
		}

		FEntityData& EntityData = LookupEntity(EntityIndex);
		// NOTE: Technically should not be necessary, however FEntityHandle::IsValid() makes the assumption
		// that SerialNum == 0 means an invalid Entity.  FMassArchetypeEntityCollection uses this assumption
		// and will fail IsValid() checks otherwise.
		++EntityData.GenerationId;
		EntityData.bIsAllocated = 1;
		int32 SerialNumber = EntityData.GetSerialNumber();

		FMassEntityHandle Handle;
		Handle.SerialNumber = SerialNumber;
		Handle.Index = EntityIndex;
		return Handle;
	}

	int32 FConcurrentEntityStorage::Acquire(TArrayView<FMassEntityHandle> OutEntityHandles)
	{
		const int32 NumberToAdd = OutEntityHandles.Num();

		int32 CountAdded = 0;
		int32 CountLeft = NumberToAdd;
		int32 CurrentEntityHandleIndex = 0;

		while (CountLeft > 0)
		{
			UE::TUniqueLock FreeListLock(FreeListMutex);

			if (UNLIKELY(EntityFreeIndexList.IsEmpty()))
			{
				if (AddPage() == false)
				{
					break;
				}
			}

			const int32 CountToProcess = FMath::Min(CountLeft, EntityFreeIndexList.Num());

			for (int32 Iteration = 0; Iteration < CountToProcess; ++Iteration)
			{
				const int32 EntityIndex = EntityFreeIndexList.Pop(EAllowShrinking::No);

				FEntityData& EntityData = LookupEntity(EntityIndex);
				// NOTE: Technically should not be necessary, however FEntityHandle::IsValid() makes the assumption
				// that SerialNum == 0 means an invalid Entity.  FMassArchetypeEntityCollection uses this assumption
				// and will fail IsValid() checks otherwise.
				++EntityData.GenerationId;
				EntityData.bIsAllocated = 1;
				const int32 SerialNumber = EntityData.GetSerialNumber();

				OutEntityHandles[CurrentEntityHandleIndex++] = { EntityIndex, SerialNumber };
			}
			
			CountAdded += CountToProcess;
			EntityCount += CountToProcess;
			CountLeft -= CountToProcess;
		}

		return CountAdded;
	}

	int32 FConcurrentEntityStorage::Release(TConstArrayView<FMassEntityHandle> Handles)
	{
		LLM_SCOPE_BYNAME(TEXT("Mass/ConcurrentStorage"));
		int32 DeallocateCount = 0;
	
		int32 BeginHandlesIndexToFree = 0;
		int32 AllocatedRunLength = 0;

		// Helper to add a range of handles to the EntityFreeIndexList
		auto FreeRunOfHandles = [this, &BeginHandlesIndexToFree, &AllocatedRunLength, Handles]()
		{
			if (AllocatedRunLength > 0) // Cheaper than taking the lock for each in case of runs of unallocated handles
			{
				UE::TUniqueLock FreeListLock(FreeListMutex);
				EntityFreeIndexList.Reserve(EntityFreeIndexList.Num() + AllocatedRunLength);
				for (int32 IndexToFree = BeginHandlesIndexToFree; IndexToFree < BeginHandlesIndexToFree + AllocatedRunLength; ++IndexToFree)
				{
					const FMassEntityHandle& HandleToFree = Handles[IndexToFree];
					EntityFreeIndexList.Add(HandleToFree.Index);
				}
			}
			BeginHandlesIndexToFree += (AllocatedRunLength + 1); // +1 to skip to next iteration
			AllocatedRunLength = 0;
		};
	
		for (int32 Index = 0, End = Handles.Num(); Index < End; ++Index)
		{
			const FMassEntityHandle& Handle = Handles[Index];
			FEntityData& EntityData = LookupEntity(Handle.Index);
			if (EntityData.GetSerialNumber() == Handle.SerialNumber)
			{
				++AllocatedRunLength;
			
				++EntityData.GenerationId;
				EntityData.bIsAllocated = 0;
				EntityData.CurrentArchetype.Reset();
			
				++DeallocateCount;
			}
			else
			{
				// Skip, this one isn't allocated
				// Return the last run to the free list
				// Ideally this code never runs but we cannot control what is passed into the Release() function
				FreeRunOfHandles();
			}
		}

		// Free any remaining handles
		FreeRunOfHandles();

		{
			UE::TUniqueLock FreeListLock(FreeListMutex);
			EntityCount -= DeallocateCount;
		}
	
		return DeallocateCount;
	}

	int32 FConcurrentEntityStorage::ReleaseOne(FMassEntityHandle Handle)
	{
		return Release(MakeArrayView(&Handle, 1));
	}

	int32 FConcurrentEntityStorage::ForceRelease(TConstArrayView<FMassEntityHandle> Handles)
	{
		LLM_SCOPE_BYNAME(TEXT("Mass/ConcurrentStorage"));
		// ForceRelease assumes the caller knows all handles are allocated
		// no need to have complexity of tracking "runs" of handles 
		for (const FMassEntityHandle& Handle : Handles)
		{
			FEntityData& EntityData = LookupEntity(Handle.Index);

			++EntityData.GenerationId;
			EntityData.bIsAllocated = 0;
			EntityData.CurrentArchetype.Reset();
		}

		{
			UE::TUniqueLock FreeListLock(FreeListMutex);
			EntityFreeIndexList.Reserve(EntityFreeIndexList.Num() + Handles.Num());
			for (const FMassEntityHandle& Handle : Handles)
			{
				EntityFreeIndexList.Add(Handle.Index);
			}

			EntityCount -= Handles.Num();
		}

		return Handles.Num();
	}

	int32 FConcurrentEntityStorage::ForceReleaseOne(FMassEntityHandle Handle)
	{
		return ForceRelease(MakeArrayView(&Handle, 1));
	}

	int32 FConcurrentEntityStorage::Num() const
	{
		return MaxEntitiesPerPage * PageCount;
	}

	int32 FConcurrentEntityStorage::ComputeFreeSize() const
	{
		return EntityFreeIndexList.Num();
	}

	FConcurrentEntityStorage::FEntityData& FConcurrentEntityStorage::LookupEntity(int32 Index)
	{
		// PageIndex is which Page in the array of pages we need to access
		const uint32 PageIndex = static_cast<uint32>(Index) >> MaxEntitiesPerPageShift;

		// Convert the entity index into the index with respect to the page
		const uint32 EntityOffset = (PageIndex << MaxEntitiesPerPageShift);
		const uint32 InternalPageIndex = static_cast<uint32>(Index) - EntityOffset;

		check((Index >= 0) && (Index >= static_cast<int32>(EntityOffset))); // Check against negative values;

		// Pointer to start of page
		FEntityData* PageStart = EntityPages[PageIndex];
		FEntityData& EntityData = PageStart[InternalPageIndex];
		return EntityData;
	}

	const FConcurrentEntityStorage::FEntityData& FConcurrentEntityStorage::LookupEntity(int32 Index) const
	{
		return const_cast<FConcurrentEntityStorage*>(this)->LookupEntity(Index);
	}

	uint64 FConcurrentEntityStorage::ComputePageSize() const
	{
		return sizeof(FEntityData) << MaxEntitiesPerPageShift;
	}

#if WITH_MASSENTITY_DEBUG
	bool FConcurrentEntityStorage::DebugAssumptionsSelfTest()
	{
		// future proofing in case FEntityData's or TSharedPtr's internals change and make MemZero-ing not produce 
		// the same results as default FEntityData's constructor
		FEntityData DefaultData;
		FEntityData ZeroedData;
		FMemory::Memzero(&ZeroedData, sizeof(FEntityData));

		if (DefaultData != ZeroedData)
		{
			UE_LOGF(LogMass, Error, "%s assumption about default FEntityData values is no longer true.", __FUNCTION__);
			return false;
		}

		return true;
	}
#endif // WITH_MASSENTITY_DEBUG

	//-----------------------------------------------------------------------------
	// FConcurrentEntityStorage::FEntityData
	//-----------------------------------------------------------------------------
	FConcurrentEntityStorage::FEntityData::~FEntityData() = default;

	int32 FConcurrentEntityStorage::FEntityData::GetSerialNumber() const
	{
		return static_cast<int32>(GenerationId);
	}

	bool FConcurrentEntityStorage::FEntityData::operator==(const FEntityData& Other) const
	{
		return CurrentArchetype == Other.CurrentArchetype
			&& GenerationId == Other.GenerationId
			&& bIsAllocated == Other.bIsAllocated;
	}
} // namespace UE::Mass
