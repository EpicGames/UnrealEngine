// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "MassEntityTypes.h"
#include "MassDebugger.h"
#include "MassProcessingTypes.h"
#include "Modules/ModuleManager.h"
#include "Relations/MassChildOf.h"
#include "MassElement.h"

#define LOCTEXT_NAMESPACE "Mass"

class FMassEntityModule : public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FMassEntityModule, MassEntity)

void FMassEntityModule::StartupModule()
{
#if WITH_UNREAL_DEVELOPER_TOOLS
	FModuleManager::Get().LoadModule("MassEntityTestSuite");
#endif // WITH_UNREAL_DEVELOPER_TOOLS

#if MASS_DO_PARALLEL
	UE_LOGF(LogMass, Log, "MassEntity running with MULTITHREADING support.");
#else
	UE_LOGF(LogMass, Log, "MassEntity running in game thread.");
#endif // MASS_DO_PARALLEL

	UE::Mass::Relations::RegisterChildOfRelation();
	UE::Mass::FElementBitSet::OnModuleInitialized();
}

void FMassEntityModule::ShutdownModule()
{
#if WITH_MASSENTITY_DEBUG
	FMassDebugger::ShutdownDebugger();
#endif
	UE::Mass::FElementBitSet::OnModuleShutdown();
}

#undef LOCTEXT_NAMESPACE 
