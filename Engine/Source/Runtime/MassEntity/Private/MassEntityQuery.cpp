// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityQuery.h"
#include "MassDebugger.h"
#include "MassEntityManager.h"
#include "MassArchetypeData.h"
#include "MassCommandBuffer.h"
#include "MassExecutionContext.h"
#include "VisualLogger/VisualLogger.h"
#include "Async/ParallelFor.h"
#include "Containers/UnrealString.h"
#include "MassProcessor.h"
#include "MassEntityTrace.h"
#include "MassRequirementAccessDetector.h"
#include "Hash/xxhash.h"
#if WITH_MASSENTITY_DEBUG
#include "MassDebugLogging.h"
#endif // WITH_MASSENTITY_DEBUG

namespace UE::Mass::Tweakables
{
	/**
	 * Controls whether ParallelForEachEntityChunk actually performs ParallelFor operations.
	 * If `false` the call is passed to the regular ForEachEntityChunk call.
	 */
	bool bAllowParallelExecution = true;

	namespace
	{
		static FAutoConsoleVariableRef AnonymousCVars[] = {
			{	TEXT("mass.AllowQueryParallelFor"), bAllowParallelExecution, TEXT("Controls whether EntityQueries are allowed to utilize ParallelFor construct"), ECVF_Cheat }
		};
	}
} // namespace UE::Mass::Tweakables

//-----------------------------------------------------------------------------
// FScopedEntityQueryContext
//-----------------------------------------------------------------------------
struct FScopedEntityQueryContext
{
	FScopedEntityQueryContext(FMassEntityQuery& InQuery, FMassExecutionContext& ExecutionContext)
		: Query(InQuery), CachedExecutionContext(ExecutionContext)
		, ScopedAccessDetector(InQuery)
	{
		// callback might pause execution until resources are available, must be made first
		ExecutionContext.PreQueryCallback(&InQuery);
		
		CachedExecutionContext.PushQuery(InQuery);

		// Setup output device to collect errors that can be reported by CacheSubsystemRequirements
		FStringOutputDevice ErrorOutput;
		UE::Mass::ErrorReporting::FScopedOutputDevice ScopedOutput(ErrorOutput);
		bSubsystemRequirementsCached = CachedExecutionContext.CacheSubsystemRequirements(InQuery);
		if (!bSubsystemRequirementsCached)
		{
#if WITH_MASSENTITY_DEBUG
			UE_VLOG_UELOG(ExecutionContext.GetSharedEntityManager().Get().GetOwner(), LogMass, Log
				, TEXT("Mass query '%s' can't be successfully set up due to subsystem requirements not being met: %s")
				, *ExecutionContext.DebugGetExecutionDesc()
				, *ErrorOutput);
#else
			UE_VLOG_UELOG(ExecutionContext.GetSharedEntityManager().Get().GetOwner(), LogMass, Log
				, TEXT("Mass query can't be successfully set up due to subsystem requirements not being met: %s"), *ErrorOutput);
#endif
		}
	}

	~FScopedEntityQueryContext()
	{
		// The query has discovered all the chunks for processing.
		CachedExecutionContext.PostQueryCallback(&Query);

		if (IsSuccessfullySetUp())
		{
			CachedExecutionContext.ClearExecutionData();
			CachedExecutionContext.FlushDeferred();
		}
		CachedExecutionContext.PopQuery(Query);
	}

	bool IsSuccessfullySetUp() const
	{
		return bSubsystemRequirementsCached;
	}

	FMassEntityQuery& Query;
	FMassExecutionContext& CachedExecutionContext;
	UE::Mass::Debug::FScopedRequirementAccessDetector ScopedAccessDetector;
	bool bSubsystemRequirementsCached;
};

//-----------------------------------------------------------------------------
// FMassEntityQuery
//-----------------------------------------------------------------------------
FMassEntityQuery::FMassEntityQuery(UMassProcessor& Owner)
{
	UE_TRACE_MASS_QUERY_CREATED()
	RegisterWithProcessor(Owner);
}

FMassEntityQuery::FMassEntityQuery(const TSharedPtr<FMassEntityManager>& EntityManager)
	: FMassFragmentRequirements(EntityManager)
{
	UE_TRACE_MASS_QUERY_CREATED()
}

FMassEntityQuery::FMassEntityQuery(const TSharedRef<FMassEntityManager>& EntityManager, std::initializer_list<UScriptStruct*> InitList)
	: FMassFragmentRequirements(EntityManager)
{
	UE_TRACE_MASS_QUERY_CREATED()
	for (const UScriptStruct* FragmentType : InitList)
	{
		AddRequirement(FragmentType, EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);
	}
}

FMassEntityQuery::FMassEntityQuery(const TSharedRef<FMassEntityManager>& EntityManager, TConstArrayView<const UScriptStruct*> InitList)
	: FMassFragmentRequirements(EntityManager)
{
	UE_TRACE_MASS_QUERY_CREATED()
	for (const UScriptStruct* FragmentType : InitList)
	{
		AddRequirement(FragmentType, EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);
	}
}

void FMassEntityQuery::RegisterWithProcessor(UMassProcessor& Owner)
{
	UE_TRACE_MASS_QUERY_REGISTERED_TO_PROCESSOR(&Owner)

	ExpectedContextType = EMassExecutionContextType::Processor;
	Owner.RegisterQuery(*this);
#if WITH_MASSENTITY_DEBUG
	bRegistered = true;
#endif // WITH_MASSENTITY_DEBUG
}

void FMassEntityQuery::CacheArchetypes()
{
	check(CachedEntityManager);

	const uint32 InEntityManagerHash = PointerHash(CachedEntityManager.Get());

	// Do an incremental update if the last updated archetype data version is different 
    bool bUpdateArchetypes = CachedEntityManager->GetArchetypeDataVersion() != LastUpdatedArchetypeDataVersion;

	// Force a full update if the entity system changed or if the requirements changed
	if (EntitySubsystemHash != InEntityManagerHash || HasIncrementalChanges())
	{
		bUpdateArchetypes = true;
		EntitySubsystemHash = InEntityManagerHash;
		ValidArchetypes.Reset();
		OrderedArchetypeIndices.Reset();
		CachedGroupIDs.Reset();
		LastUpdatedArchetypeDataVersion = 0;
		ArchetypeFragmentMapping.Reset();

		if (HasIncrementalChanges())
		{
			ConsumeIncrementalChangesCount();
			if (CheckValidity())
			{
				SortRequirements();
			}
			else
			{
				bUpdateArchetypes = false;
				UE_VLOG_UELOG(CachedEntityManager->GetOwner(), LogMass, Error, TEXT("FMassEntityQuery::CacheArchetypes: requirements not valid: %s"), *FMassDebugger::GetRequirementsDescription(*this));
			}
		}
	}
	
	// Process any new archetype that is newer than the LastUpdatedArchetypeDataVersion
	if (bUpdateArchetypes)
	{
		TArray<FMassArchetypeHandle> NewValidArchetypes;
#if WITH_EDITOR
		if (bHasArchetypeMatchOverride)
		{
			CachedEntityManager->ForEachArchetype(static_cast<int32>(LastUpdatedArchetypeDataVersion), TNumericLimits<int32>::Max() /*last*/, [this, &NewValidArchetypes](const FMassEntityManager& EntityManager, const FMassArchetypeHandle& Handle)
			{
				const FMassArchetypeCompositionDescriptor& Composition = EntityManager.GetArchetypeComposition(Handle);
				if (ArchetypeMatchOverride.Match(&ArchetypeMatchOverride.Data, Composition))
				{
					NewValidArchetypes.Add(Handle);
				}
			});
		}
		else
#endif
		{
			CachedEntityManager->GetMatchingArchetypes(*this, NewValidArchetypes, LastUpdatedArchetypeDataVersion);
		}
		LastUpdatedArchetypeDataVersion = CachedEntityManager->GetArchetypeDataVersion();
		if (NewValidArchetypes.Num())
		{
			const int32 FirstNewArchetype = ValidArchetypes.Num();
			ValidArchetypes.Append(NewValidArchetypes);

			TRACE_CPUPROFILER_EVENT_SCOPE_STR("Mass RequirementsBinding")
			const TConstArrayView<FMassFragmentRequirementDescription> LocalRequirements = GetFragmentRequirements();
			ArchetypeFragmentMapping.AddDefaulted(NewValidArchetypes.Num());
			for (int i = FirstNewArchetype; i < ValidArchetypes.Num(); ++i)
			{
				FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ValidArchetypes[i]);
				ArchetypeData.GetRequirementsFragmentMapping(LocalRequirements, ArchetypeFragmentMapping[i].EntityFragments);
				if (ChunkFragmentRequirements.Num())
				{
					ArchetypeData.GetRequirementsChunkFragmentMapping(ChunkFragmentRequirements, ArchetypeFragmentMapping[i].ChunkFragments);
				}
				if (ConstSharedFragmentRequirements.Num())
				{
					ArchetypeData.GetRequirementsConstSharedFragmentMapping(ConstSharedFragmentRequirements, ArchetypeFragmentMapping[i].ConstSharedFragments);
				}
				if (SharedFragmentRequirements.Num())
				{
					ArchetypeData.GetRequirementsSharedFragmentMapping(SharedFragmentRequirements, ArchetypeFragmentMapping[i].SharedFragments);
				}
			}

			if (IsGrouping())
			{
				SortArchetypes(FirstNewArchetype);
			}
			else
			{
				BuildOrderedArchetypeIndices(FirstNewArchetype);
			}
		}
	}
}

void FMassEntityQuery::ForEachEntityChunkInCollections(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
	, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction)
{
	for (const FMassArchetypeEntityCollection& EntityCollection : EntityCollections)
	{
		ForEachEntityChunk(EntityCollection, ExecutionContext, ExecuteFunction);
	}
}

void FMassEntityQuery::ForEachEntityChunk(const FMassArchetypeEntityCollection& EntityCollection, FMassExecutionContext& ExecutionContext
	, const FMassExecuteFunction& ExecuteFunction)
{
	ExecutionContext.SetEntityCollection(EntityCollection);
	ForEachEntityChunk(ExecutionContext, ExecuteFunction);
	ExecutionContext.ClearEntityCollection();
}

void FMassEntityQuery::ForEachEntityChunk(FMassArchetypeEntityCollection&& EntityCollection, FMassExecutionContext& ExecutionContext
	, const FMassExecuteFunction& ExecuteFunction)
{
	ExecutionContext.SetEntityCollection(Forward<FMassArchetypeEntityCollection>(EntityCollection));
	ForEachEntityChunk(ExecutionContext, ExecuteFunction);
	ExecutionContext.ClearEntityCollection();
}

void FMassEntityQuery::ForEachEntityChunk(FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction)
{
	UE_TRACE_SCOPED_MASS_QUERY_FOR_EACH()

	checkf(CachedEntityManager == ExecutionContext.GetSharedEntityManager() && CachedEntityManager.IsValid()
		, TEXT("FMassEntityQuery needs to be initialized with a valid EntityManager and the execution context has to come from the same manager"));

#if WITH_MASSENTITY_DEBUG
	checkf(ExecutionContext.GetExecutionType() == ExpectedContextType && (ExpectedContextType == EMassExecutionContextType::Local || bRegistered)
		, TEXT("ExecutionContextType mismatch, make sure all the queries run as part of processor execution are registered with some processor with a FMassEntityQuery::RegisterWithProcessor call"));
#endif

	FScopedEntityQueryContext ScopedQueryContext(*this, ExecutionContext);

	if (ScopedQueryContext.IsSuccessfullySetUp() == false)
	{
		// required subsystems are not available, bail out.
		return;
	}

	if (FMassFragmentRequirements::IsEmpty())
	{
		if (ensureMsgf(ExecutionContext.GetEntityCollection().IsEmpty() == false, TEXT("Using empty queries is only supported in combination with Entity Collections that explicitly indicate entities to process")))
		{
			static const FMassQueryRequirementIndicesMapping EmptyMapping;

			const FMassArchetypeHandle& ArchetypeHandle = ExecutionContext.GetEntityCollection().GetArchetype();
			FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
			
			if (ExecutionContext.HasChunkQueueFunc())
			{
				for (const FMassArchetypeEntityCollection::FArchetypeEntityRange& Range : ExecutionContext.GetEntityCollection().GetRanges())
				{
					ExecutionContext.QueueChunkChecked({
						.SourceQuery = this,
						.Archetype = &ArchetypeData,
						.EntityRange = Range,
						.ExecuteFunction = &ExecuteFunction,
						.ChunkCondition = &ChunkCondition,
						.RequirementMapping = nullptr,
						.bParallelExecution = false});
				}
			}
			else
			{
				ArchetypeData.ExecuteFunction(ExecutionContext, ExecuteFunction
					, EmptyMapping
					, ExecutionContext.GetEntityCollection().GetRanges()
					, ChunkCondition);
			}
		}
	}
	else
	{
		// note that the following function will usually only resort to verifying that the data is up to date by
			// checking the version number. In rare cases when it would result in non trivial cost we actually
			// do need those calculations.
		CacheArchetypes();

		// if there's a chunk collection set by the external code - use that
		if (ExecutionContext.GetEntityCollection().IsEmpty() == false)
		{
			const FMassArchetypeHandle& ArchetypeHandle = ExecutionContext.GetEntityCollection().GetArchetype();
			const int32 ArchetypeIndex = ValidArchetypes.Find(ArchetypeHandle);

			// if given ArchetypeHandle cannot be found in ValidArchetypes then it doesn't match the query's requirements
			if (ArchetypeIndex == INDEX_NONE)
			{
				UE_VLOG_UELOG(CachedEntityManager->GetOwner(), LogMass, Verbose, TEXT("Attempted to execute FMassEntityQuery with an incompatible Archetype: %s. Note that this is fine for observers.")
					, *FMassDebugger::GetArchetypeRequirementCompatibilityDescription(*this, ArchetypeHandle));

				return;
			}

			ExecutionContext.ApplyFragmentRequirements(*this);

			FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
			
			UE_TRACE_SCOPED_MASS_QUERY_FOR_EACH_REPORT_ARCHETYPE(ArchetypeData)
			
			if (ExecutionContext.HasChunkQueueFunc())
			{
				for (const FMassArchetypeEntityCollection::FArchetypeEntityRange& Range : ExecutionContext.GetEntityCollection().GetRanges())
				{
					ExecutionContext.QueueChunkChecked({
						.SourceQuery = this,
						.Archetype = &ArchetypeData,
						.EntityRange = Range,
						.ExecuteFunction = &ExecuteFunction,
						.ChunkCondition = &ChunkCondition,
						.RequirementMapping = &GetRequirementsMappingForArchetype(ArchetypeHandle),
						.bParallelExecution = false });
				}
			}
			else
			{
				ArchetypeData.ExecuteFunction(ExecutionContext, ExecuteFunction
					, GetRequirementsMappingForArchetype(ArchetypeHandle)
					, ExecutionContext.GetEntityCollection().GetRanges()
					, ChunkCondition);
			}
			
		}
		else
		{
			// it's important to set requirements after caching archetypes due to that call potentially sorting the requirements and the order is relevant here.
			ExecutionContext.ApplyFragmentRequirements(*this);

			// note that this checkSlow is here on purpose, for debugging purposes, not data verification purposes.
			checkSlow(OrderedArchetypeIndices.Num() == ValidArchetypes.Num());
			for (const int32 ArchetypeIndex : OrderedArchetypeIndices)
			{
				const FMassArchetypeHandle& ArchetypeHandle = ValidArchetypes[ArchetypeIndex];
				FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
				
				if (ExecutionContext.HasChunkQueueFunc())
				{
					for (int32 ChunkIndex = 0; ChunkIndex < ArchetypeData.GetChunkCount(); ChunkIndex++)
					{
						ExecutionContext.QueueChunkChecked({
							.SourceQuery = this,
							.Archetype = &ArchetypeData,
							.EntityRange = FMassArchetypeEntityCollection::FArchetypeEntityRange(ChunkIndex),
							.ExecuteFunction = &ExecuteFunction,
							.ChunkCondition = &ChunkCondition,
							.RequirementMapping = &ArchetypeFragmentMapping[ArchetypeIndex] });
					}
				}
				else
				{
					UE_TRACE_SCOPED_MASS_QUERY_FOR_EACH_REPORT_ARCHETYPE(ArchetypeData)

					ArchetypeData.ExecuteFunction(ExecutionContext, ExecuteFunction, ArchetypeFragmentMapping[ArchetypeIndex], ChunkCondition);
					ExecutionContext.ClearFragmentViews(*this);
				}
			}
		}
	}
}

void FMassEntityQuery::ForEachEntityChunk(FMassExecutionContext& ExecutionContext, UE::Mass::FExecutionLimiter& Limiter, const FMassExecuteFunction& ExecuteFunction)
{
	UE_TRACE_SCOPED_MASS_QUERY_FOR_EACH()

	checkf(CachedEntityManager == ExecutionContext.GetSharedEntityManager() && CachedEntityManager.IsValid()
		, TEXT("FMassEntityQuery needs to be initialized with a valid EntityManager and the execution context has to come from the same manager"));

#if WITH_MASSENTITY_DEBUG
	checkf(ExecutionContext.GetExecutionType() == ExpectedContextType && (ExpectedContextType == EMassExecutionContextType::Local || bRegistered)
		, TEXT("ExecutionContextType mismatch, make sure all the queries run as part of processor execution are registered with some processor with a FMassEntityQuery::RegisterWithProcessor call"));
#endif

	FScopedEntityQueryContext ScopedQueryContext(*this, ExecutionContext);

	if (ScopedQueryContext.IsSuccessfullySetUp() == false)
	{
		// required subsystems are not available, bail out.
		return;
	}

	// FExecutionLimiter is not supported for use with manually set entity collections.
	checkf(FMassFragmentRequirements::IsEmpty() == false, TEXT("FExecutionLimiter is not supported for use with manually set entity collections."));
	checkf(ExecutionContext.GetEntityCollection().IsEmpty(), TEXT("FExecutionLimiter is not supported for use with manually set entity collections."));

	// note that the following function will usually only resort to verifying that the data is up to date by
	// checking the version number. In rare cases when it would result in non trivial cost we actually
	// do need those calculations.
	CacheArchetypes();

	// it's important to set requirements after caching archetypes due to that call potentially sorting the requirements and the order is relevant here.
	ExecutionContext.ApplyFragmentRequirements(*this);

	// note that this checkSlow is here on purpose, for debugging purposes, not data verification purposes.
	const int32 NumArchetypes = ValidArchetypes.Num();
	checkSlow(OrderedArchetypeIndices.Num() == NumArchetypes);

	if (NumArchetypes == 0)
	{
		return;
	}

	if (Limiter.ChunkIndex < 0 || Limiter.ArchetypeIndex < 0 || Limiter.ArchetypeIndex >= NumArchetypes)
	{
		Limiter.ArchetypeIndex = 0;
		Limiter.ChunkIndex = 0;
	}
	const int32 StartingChunkIndex = Limiter.ChunkIndex;
	const int32 StartingArchetypeIndex = Limiter.ArchetypeIndex;
	Limiter.EntityCountRemaining = Limiter.EntityLimit;
	Limiter.MaxChunkIndex = TNumericLimits<int32>::Max();
	bool bLooped = false;
	bool bDone = false;

	do
	{
		const int32 ArchetypeIndex = OrderedArchetypeIndices[Limiter.ArchetypeIndex];
		const FMassArchetypeHandle& ArchetypeHandle = ValidArchetypes[ArchetypeIndex];
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);

		if (bLooped && StartingChunkIndex != 0)
		{
			// archetype was started past the first chunk so if we loop we need to process it again
			// but stop before the starting chunk is reprocessed
			Limiter.MaxChunkIndex = StartingChunkIndex;
			bDone = true;
		}

		if (ExecutionContext.HasChunkQueueFunc())
		{
			const int32 MaxChunkIndex = FMath::Min(ArchetypeData.GetChunkCount(), Limiter.MaxChunkIndex);
			Limiter.ChunkIndex = Limiter.ChunkIndex >= MaxChunkIndex ? 0 : FMath::Max(0, Limiter.ChunkIndex);
			for (; Limiter.ChunkIndex < MaxChunkIndex && Limiter.EntityCountRemaining > 0; Limiter.ChunkIndex++)
			{
				const FMassArchetypeEntityCollection::FArchetypeEntityRange Range = FMassArchetypeEntityCollection::FArchetypeEntityRange(Limiter.ChunkIndex);
				Limiter.EntityCountRemaining -= ArchetypeData.CalculateRangeLength(Range);
				ExecutionContext.QueueChunkChecked({
					.SourceQuery = this,
					.Archetype = &ArchetypeData,
					.EntityRange = Range,
					.ExecuteFunction = &ExecuteFunction,
					.ChunkCondition = &ChunkCondition,
					.RequirementMapping = &ArchetypeFragmentMapping[ArchetypeIndex] });
			}
		}
		else
		{
			UE_TRACE_SCOPED_MASS_QUERY_FOR_EACH_REPORT_ARCHETYPE(ArchetypeData)

			ArchetypeData.ExecuteFunction(ExecutionContext, ExecuteFunction, ArchetypeFragmentMapping[ArchetypeIndex], ChunkCondition, &Limiter);
			ExecutionContext.ClearFragmentViews(*this);
		}
		
		if (Limiter.ChunkIndex >= ArchetypeData.GetChunkCount())
		{
			// all chunks in the archetype were processed, so move to the next
			Limiter.ChunkIndex = 0;
			if (++Limiter.ArchetypeIndex >= NumArchetypes)
			{
				Limiter.ArchetypeIndex = 0;
			}
			if (Limiter.ArchetypeIndex == StartingArchetypeIndex)
			{
				if (StartingChunkIndex == 0)
				{
					// no need to process the starting archetype again because the initial pass hit all the chunks
					bDone = true;
				}
				bLooped = true;
			}
		}
		
	} while (Limiter.EntityCountRemaining > 0 && !bDone);
}

void FMassEntityQuery::ParallelForEachEntityChunkInCollection(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
	, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction
	, const EParallelExecutionFlags Flags)
{
	if (UE::Mass::Tweakables::bAllowParallelExecution == false && !EnumHasAnyFlags(Flags, EParallelExecutionFlags::Force))
	{
		ForEachEntityChunkInCollections(EntityCollections, ExecutionContext, ExecuteFunction);
		return;
	}

	ParallelFor(EntityCollections.Num(), [this, &ExecutionContext, &ExecuteFunction, &EntityCollections, Flags](const int32 JobIndex)
	{
		FMassExecutionContext LocalExecutionContext = ExecutionContext; 
		LocalExecutionContext.SetEntityCollection(EntityCollections[JobIndex]);
		ParallelForEachEntityChunk(LocalExecutionContext, ExecuteFunction, Flags);
	});
}

void FMassEntityQuery::ParallelForEachEntityChunk(FMassExecutionContext& ExecutionContext
	, const FMassExecuteFunction& ExecuteFunction, const EParallelExecutionFlags Flags)
{
	if (UE::Mass::Tweakables::bAllowParallelExecution == false && !EnumHasAnyFlags(Flags, EParallelExecutionFlags::Force))
	{
		ForEachEntityChunk(ExecutionContext, ExecuteFunction);
		return;
	}
	else if (IsGrouping())
	{
		UE_VLOG_UELOG(CachedEntityManager->GetOwner(), LogMass, Warning, TEXT("Calling %hs is not supported for grouping queries yet. Calling regular ForEachEntityChunk instead."), __FUNCTION__);
		ForEachEntityChunk(ExecutionContext, ExecuteFunction);
		return;
	}

	checkf(CachedEntityManager == ExecutionContext.GetSharedEntityManager() && CachedEntityManager.IsValid()
		, TEXT("FMassEntityQuery needs to be initialized with a valid EntityManager and the execution context has to come from the same manager"));

#if WITH_MASSENTITY_DEBUG
	checkf(ExecutionContext.GetExecutionType() == ExpectedContextType && (ExpectedContextType == EMassExecutionContextType::Local || bRegistered)
		, TEXT("ExecutionContextType mismatch, make sure all the queries run as part of processor execution are registered with some processor with a FMassEntityQuery::RegisterWithProcessor call"));
#endif

	FScopedEntityQueryContext ScopedQueryContext(*this, ExecutionContext);

	if (ScopedQueryContext.IsSuccessfullySetUp() == false)
	{
		// required subsystems are not available, bail out.
		return;
	}

	struct FChunkJob
	{
		FMassArchetypeData& Archetype;
		const int32 ArchetypeIndex;
		const FMassArchetypeEntityCollection::FArchetypeEntityRange EntityRange;
	};
	TArray<FChunkJob> Jobs;

	if (FMassFragmentRequirements::IsEmpty())
	{
		if (ensureMsgf(ExecutionContext.GetEntityCollection().IsEmpty() == false, TEXT("Using empty queries is only supported in combination with Entity Collections that explicitly indicate entities to process")))
		{
			const FMassArchetypeHandle& ArchetypeHandle = ExecutionContext.GetEntityCollection().GetArchetype();
			FMassArchetypeData& ArchetypeRef = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
			for (const FMassArchetypeEntityCollection::FArchetypeEntityRange& EntityRange : ExecutionContext.GetEntityCollection().GetRanges())
			{
				Jobs.Add({ ArchetypeRef, INDEX_NONE, EntityRange });
			}
		}
	}
	else
	{

		// note that the following function will usualy only resort to verifying that the data is up to date by
		// checking the version number. In rare cases when it would result in non trivial cost we actually
		// do need those calculations.
		CacheArchetypes();

		// if there's a chunk collection set by the external code - use that
		if (ExecutionContext.GetEntityCollection().IsEmpty() == false)
		{
			const FMassArchetypeHandle& ArchetypeHandle = ExecutionContext.GetEntityCollection().GetArchetype();
			const int32 ArchetypeIndex = ValidArchetypes.Find(ArchetypeHandle);

			// if given ArchetypeHandle cannot be found in ValidArchetypes then it doesn't match the query's requirements
			if (ArchetypeIndex == INDEX_NONE)
			{
				UE_VLOG_UELOG(CachedEntityManager->GetOwner(), LogMass, Verbose, TEXT("Attempted to execute FMassEntityQuery with an incompatible Archetype: %s. Note that this is fine for observers.")
					, *FMassDebugger::GetArchetypeRequirementCompatibilityDescription(*this, ExecutionContext.GetEntityCollection().GetArchetype()));

				return;
			}

			ExecutionContext.ApplyFragmentRequirements(*this);

			FMassArchetypeData& ArchetypeRef = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
			for (const FMassArchetypeEntityCollection::FArchetypeEntityRange& EntityRange : ExecutionContext.GetEntityCollection().GetRanges())
			{
				Jobs.Add({ ArchetypeRef, ArchetypeIndex, EntityRange });
			}
		}
		else
		{
			ExecutionContext.ApplyFragmentRequirements(*this);
			for (int ArchetypeIndex = 0; ArchetypeIndex < ValidArchetypes.Num(); ++ArchetypeIndex)
			{
				FMassArchetypeHandle& ArchetypeHandle = ValidArchetypes[ArchetypeIndex];
				FMassArchetypeData& ArchetypeRef = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
				const FMassArchetypeEntityCollection AsEntityCollection(ArchetypeHandle);
				for (const FMassArchetypeEntityCollection::FArchetypeEntityRange& EntityRange : AsEntityCollection.GetRanges())
				{
					Jobs.Add({ ArchetypeRef, ArchetypeIndex, EntityRange });
				}
			}
		}
	}

	if (Jobs.Num())
	{
		const EParallelForFlags ParallelForFlags = 
			EnumHasAnyFlags(Flags, EParallelExecutionFlags::AutoBalance) ? EParallelForFlags::Unbalanced : EParallelForFlags::None;
		
		if (ExecutionContext.HasChunkQueueFunc())
		{
			for (FChunkJob& Job : Jobs)
			{
				const FMassQueryRequirementIndicesMapping* RequirementMapping = Job.ArchetypeIndex != INDEX_NONE ? &ArchetypeFragmentMapping[Job.ArchetypeIndex] : nullptr;
				ExecutionContext.QueueChunkChecked({
					.SourceQuery = this,
					.Archetype = &Job.Archetype,
					.EntityRange = Job.EntityRange,
					.ExecuteFunction = &ExecuteFunction,
					.ChunkCondition = &ChunkCondition,
					.RequirementMapping = RequirementMapping,
					.bParallelExecution = true});
			}
		}
		else if (bAllowParallelCommands)
		{
			struct FTaskContext
			{
				FTaskContext() = default;

				TSharedPtr<FMassCommandBuffer> GetCommandBuffer()
				{
					if (!CommandBuffer)
					{
						// lazily creating the command buffer to ensure we create it in the same thread it's going to be used in
						CommandBuffer = MakeShared<FMassCommandBuffer>();
					}
					else
					{
						// force-updating the thread ID because ParallelFor can move workers between threads.
						CommandBuffer->ForceUpdateCurrentThreadID();
					}
					return CommandBuffer;
				}
			private:
				TSharedPtr<FMassCommandBuffer> CommandBuffer;
			};

			TArray<FTaskContext> TaskContext;

			ParallelForWithTaskContext(TaskContext, Jobs.Num(), [this, &ExecutionContext, &ExecuteFunction, &Jobs](FTaskContext& TaskContext, const int32 JobIndex)
				{
					FMassExecutionContext LocalExecutionContext(ExecutionContext, *this, TaskContext.GetCommandBuffer());
					Jobs[JobIndex].Archetype.ExecutionFunctionForChunk(LocalExecutionContext, ExecuteFunction
						, Jobs[JobIndex].ArchetypeIndex != INDEX_NONE ? ArchetypeFragmentMapping[Jobs[JobIndex].ArchetypeIndex] : FMassQueryRequirementIndicesMapping()
						, Jobs[JobIndex].EntityRange
						, ChunkCondition);
					LocalExecutionContext.PopQuery(*this);
				}, ParallelForFlags);

			// merge all command buffers
			for (FTaskContext& CommandContext : TaskContext)
			{
				ExecutionContext.Defer().MoveAppend(*CommandContext.GetCommandBuffer());
			}
		}
		else
		{
			ParallelFor(Jobs.Num(), [this, &ExecutionContext, &ExecuteFunction, &Jobs](const int32 JobIndex)
				{
					FMassExecutionContext LocalExecutionContext(ExecutionContext, *this);
					Jobs[JobIndex].Archetype.ExecutionFunctionForChunk(LocalExecutionContext, ExecuteFunction
						, Jobs[JobIndex].ArchetypeIndex != INDEX_NONE ? ArchetypeFragmentMapping[Jobs[JobIndex].ArchetypeIndex] : FMassQueryRequirementIndicesMapping()
						, Jobs[JobIndex].EntityRange
						, ChunkCondition);
					LocalExecutionContext.PopQuery(*this);
				}, ParallelForFlags);
		}
	}
}

uint64 FMassEntityQuery::CalculateArchetypeEntityOrderHash()
{
	return CalculateArchetypeHash_Internal([](const FMassArchetypeData* ArchetypeData)
		{
			return ArchetypeData->GetEntityOrderVersion();
		});
}

uint64 FMassEntityQuery::CalculateArchetypeVersionHash()
{
	return CalculateArchetypeHash_Internal([](const FMassArchetypeData* ArchetypeData)
		{
			return ArchetypeData->GetArchetypeVersion();
		});
}

int32 FMassEntityQuery::GetNumMatchingEntities()
{
	CacheArchetypes();
	int32 TotalEntities = 0;
	for (FMassArchetypeHandle& ArchetypeHandle : ValidArchetypes)
	{
		if (const FMassArchetypeData* Archetype = FMassArchetypeHelper::ArchetypeDataFromHandle(ArchetypeHandle))
		{
			TotalEntities += Archetype->GetNumEntities();
		}
	}
	return TotalEntities;
}

int32 FMassEntityQuery::GetNumMatchingEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections)
{
	int32 TotalEntities = 0;
	for (const FMassArchetypeEntityCollection& EntityCollection : EntityCollections)
	{
		if (DoesArchetypeMatchRequirements(EntityCollection.GetArchetype()))
		{
			for (const FMassArchetypeEntityCollection::FArchetypeEntityRange& EntityRange : EntityCollection.GetRanges())
			{
				TotalEntities += EntityRange.Length;
			}
		}
	}
	return TotalEntities;
}

bool FMassEntityQuery::HasMatchingEntities()
{
	CacheArchetypes();

	for (FMassArchetypeHandle& ArchetypeHandle : ValidArchetypes)
	{
		const FMassArchetypeData* Archetype = FMassArchetypeHelper::ArchetypeDataFromHandle(ArchetypeHandle);
		if (Archetype && Archetype->GetNumEntities() > 0)
		{
			return true;
		}
	}
	return false;
}

TArray<FMassArchetypeEntityCollection> FMassEntityQuery::CreateMatchingEntitiesCollection()
{
	TArray<FMassArchetypeEntityCollection> Collections;
	CacheArchetypes();

	Collections.Reserve(ValidArchetypes.Num());
	for (FMassArchetypeHandle& ArchetypeHandle : ValidArchetypes)
	{
		Collections.Emplace(ArchetypeHandle);
	}

	return Collections;
}

TArray<FMassEntityHandle> FMassEntityQuery::GetMatchingEntityHandles()
{
	TArray<FMassEntityHandle> Handles;
	CacheArchetypes();

	for (FMassArchetypeHandle& ArchetypeHandle : ValidArchetypes)
	{
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
		ArchetypeData.ExportEntityHandles(Handles);
	}

	return Handles;
}

void FMassEntityQuery::GroupBy(UE::Mass::FArchetypeGroupType GroupType)
{
	GroupBy(GroupType, [](const UE::Mass::FArchetypeGroupID A, const UE::Mass::FArchetypeGroupID B)
	{
		return A < B;
	});
}

void FMassEntityQuery::GroupBy(UE::Mass::FArchetypeGroupType GroupType, const TFunction<bool(const UE::Mass::FArchetypeGroupID, const UE::Mass::FArchetypeGroupID)>& Predicate)
{
	GroupSortingSteps.Emplace(GroupType, Predicate);
	IncrementChangeCounter();
}

void FMassEntityQuery::ResetGrouping()
{
	GroupSortingSteps.Reset();
	IncrementChangeCounter();
}

void FMassEntityQuery::BuildOrderedArchetypeIndices(const int32 FirstNewArchetypeIndex)
{
	OrderedArchetypeIndices.SetNumUninitialized(ValidArchetypes.Num());
	for (int32 ArchetypeIndex = FirstNewArchetypeIndex; ArchetypeIndex < ValidArchetypes.Num(); ++ArchetypeIndex)
	{
		OrderedArchetypeIndices[ArchetypeIndex] = ArchetypeIndex;
	}
}

uint64 FMassEntityQuery::CalculateArchetypeHash_Internal(TFunctionRef<uint32(const FMassArchetypeData*)> HashValueGetter)
{
	CacheArchetypes();

	// The hash is created as a cumulative hash to make sure that the order of archetypes doesn't matter.
	uint64 Hash = 0;
	for (const FMassArchetypeHandle& ArchetypeHandle : ValidArchetypes)
	{
		if (const FMassArchetypeData* Archetype = FMassArchetypeHelper::ArchetypeDataFromHandle(ArchetypeHandle))
		{
			int32 HashValue = HashValueGetter(Archetype);
			// Accept the possible overflow as a desired side effect.
			Hash += FXxHash64::HashBuffer(&HashValue, sizeof(int32)).Hash;
		}
	}
	// Further reduce the risk of a hash collision between two different sized archetype lists.
	int32 ArchetypeCount = ValidArchetypes.Num();
	Hash += FXxHash64::HashBuffer(&ArchetypeCount, sizeof(int32)).Hash;
	return Hash;
}

void FMassEntityQuery::SortArchetypes(const int32 FirstNewArchetypeIndex)
{
	if (GroupSortingSteps.Num() == 0)
	{
		BuildOrderedArchetypeIndices(FirstNewArchetypeIndex);
		return;
	}

	check(CachedEntityManager);

	// first we need to cache the required group IDs from the new archetypes
	CachedGroupIDs.SetNum(ValidArchetypes.Num());
	OrderedArchetypeIndices.SetNumUninitialized(ValidArchetypes.Num());

	for (int32 NewArchetypeIndex = FirstNewArchetypeIndex; NewArchetypeIndex < ValidArchetypes.Num(); ++NewArchetypeIndex)
	{
		OrderedArchetypeIndices[NewArchetypeIndex] = NewArchetypeIndex;

		TArray<UE::Mass::FArchetypeGroupID>& ArchetypeGroupIDs = CachedGroupIDs[NewArchetypeIndex];
		ArchetypeGroupIDs.Reserve(GroupSortingSteps.Num());

		const UE::Mass::FArchetypeGroups& ArchetypeGroups = CachedEntityManager->GetGroupsForArchetype(ValidArchetypes[NewArchetypeIndex]);

		for (const FArchetypeGroupingStep& Step : GroupSortingSteps)
		{
			// ArchetypeGroups.GetID will return InvalidArchetypeGroupID if the given group type is not in ArchetypeGroups.
			// This is what we want.
			ArchetypeGroupIDs.Add(ArchetypeGroups.GetID(Step.GroupType));
		}
	}

	TConstArrayView<TArray<UE::Mass::FArchetypeGroupID>> GroupIDs = MakeConstArrayView(CachedGroupIDs);
	TArray<TTuple<int32, int32>> Ranges;
	Ranges.Add({0, OrderedArchetypeIndices.Num()});
	int32 MaxRangeSize = OrderedArchetypeIndices.Num();
	int32 Step = 0;
	int32 RangesProcessed = 0;

	while (Step < GroupSortingSteps.Num() && MaxRangeSize > 1)
	{
		const bool bLastIteration = (Step == GroupSortingSteps.Num() - 1);
		const int32 RangesThisIteration = Ranges.Num();
		MaxRangeSize = 0;
		for (; RangesProcessed < RangesThisIteration; ++RangesProcessed)
		{
			const TTuple<int32, int32> Range = Ranges[RangesProcessed];
			const int32 RangeLength = Range.Get<1>() - Range.Get<0>();
			MakeArrayView(&OrderedArchetypeIndices[Range.Get<0>()], RangeLength)
				.Sort([Step, &GroupIDs, Sorter = GroupSortingSteps[Step].Predicate](const int32 A, const int32 B)
				{
					return Sorter(GroupIDs[A][Step], GroupIDs[B][Step]);
				});

			// figure out new ranges
			if (bLastIteration == false)
			{
				int32 SubRangeStart = Range.Get<0>();
				UE::Mass::FArchetypeGroupID PrevValue = GroupIDs[OrderedArchetypeIndices[SubRangeStart]][Step];
				for (int32 Index = SubRangeStart + 1; Index < Range.Get<1>(); ++Index)
				{
					const UE::Mass::FArchetypeGroupID NewValue = GroupIDs[OrderedArchetypeIndices[Index]][Step];
					if (NewValue != PrevValue)
					{
						PrevValue = NewValue;
						Ranges.Push({SubRangeStart, Index});
						MaxRangeSize = FMath::Max(MaxRangeSize, Index - SubRangeStart);
						SubRangeStart = Index;
					}
				}

				// the loop above doesn't create any ranges when there's no change in GroupID among processed archetypes.
				// Similarly, it doesn't store the "last" range. We're addressing this now.
				Ranges.Push({SubRangeStart, Range.Get<1>()});
			}
		}

		check(MaxRangeSize >= 1 || bLastIteration);
		++Step;
	};
}

const FMassQueryRequirementIndicesMapping& FMassEntityQuery::GetRequirementsMappingForArchetype(const FMassArchetypeHandle ArchetypeHandle) const
{
	static const FMassQueryRequirementIndicesMapping FallbackEmptyMapping;
	checkf(HasIncrementalChanges() == false, TEXT("Fetching cached fragments mapping while the query's cached data is out of sync!"));
	const int32 ArchetypeIndex = ValidArchetypes.Find(ArchetypeHandle);
	return ArchetypeIndex != INDEX_NONE ? ArchetypeFragmentMapping[ArchetypeIndex] : FallbackEmptyMapping;
}

void FMassEntityQuery::ExportRequirements(FMassExecutionRequirements& OutRequirements) const
{
	FMassSubsystemRequirements::ExportRequirements(OutRequirements);
	FMassFragmentRequirements::ExportRequirements(OutRequirements);
	EntityCreationRequirements.ExportRequirements(OutRequirements);
}

void FMassEntityQuery::DebugEnableEntityOwnerLogging()
{
#if WITH_MASSENTITY_DEBUG
	if (RequiredOptionalFragments.Contains<FMassDebugLogFragment>() == false)
	{
		AddRequirement<FMassDebugLogFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	}
#endif // WITH_MASSENTITY_DEBUG
}

//-----------------------------------------------------------------------------
// DEPRECATED
//-----------------------------------------------------------------------------
// deprecated
FMassEntityQuery::FMassEntityQuery(std::initializer_list<UScriptStruct*>)
	: FMassEntityQuery()
{
}

// deprecated
FMassEntityQuery::FMassEntityQuery(TConstArrayView<const UScriptStruct*>)
	: FMassEntityQuery()
{
}

// deprecated
void FMassEntityQuery::ForEachEntityChunk(FMassEntityManager&, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction)
{
	ForEachEntityChunk(ExecutionContext, ExecuteFunction);
}

// deprecated
void FMassEntityQuery::ForEachEntityChunk(const FMassArchetypeEntityCollection& EntityCollection, FMassEntityManager&
		, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction)
{
	ForEachEntityChunk(EntityCollection, ExecutionContext, ExecuteFunction);
}

// deprecated
void FMassEntityQuery::ParallelForEachEntityChunk(FMassEntityManager&, FMassExecutionContext& ExecutionContext
		, const FMassExecuteFunction& ExecuteFunction, const EParallelForMode ParallelMode)
{
	const EParallelExecutionFlags Flags = EnumHasAnyFlags(ParallelMode, ForceParallelExecution)
		? EParallelExecutionFlags::Force
		: EParallelExecutionFlags::Default;
	ParallelForEachEntityChunk(ExecutionContext, ExecuteFunction, Flags);
}

// deprecated
void FMassEntityQuery::ForEachEntityChunkInCollections(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, FMassEntityManager&
		, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction)
{
	ForEachEntityChunkInCollections(EntityCollections, ExecutionContext, ExecuteFunction);
}

// deprecated
void FMassEntityQuery::ParallelForEachEntityChunkInCollection(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
		, FMassEntityManager&, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction
		, const EParallelForMode ParallelMode)
{
	const EParallelExecutionFlags Flags = EnumHasAnyFlags(ParallelMode, ForceParallelExecution)
		? EParallelExecutionFlags::Force
		: EParallelExecutionFlags::Default;
	ParallelForEachEntityChunkInCollection(EntityCollections, ExecutionContext, ExecuteFunction, Flags);
}

// deprecated
void FMassEntityQuery::CacheArchetypes(const FMassEntityManager&)
{
	CacheArchetypes();
}

// deprecated
int32 FMassEntityQuery::GetNumMatchingEntities(FMassEntityManager&)
{
	return GetNumMatchingEntities();
}

// deprecated
bool FMassEntityQuery::HasMatchingEntities(FMassEntityManager&)
{
	return HasMatchingEntities();
}
