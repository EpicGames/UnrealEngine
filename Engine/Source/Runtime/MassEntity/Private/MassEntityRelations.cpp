// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityRelations.h"
#include "MassEntityManager.h"
#include "MassRelationObservers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityRelations)

namespace UE::Mass
{
	//-----------------------------------------------------------------------------
	// FRelationTypeTraits
	//-----------------------------------------------------------------------------
	FRelationTypeTraits::FRelationTypeTraits(const TNotNull<const UScriptStruct*> InRelationTagType)
		: RelationTagType(InRelationTagType)
		, RelationFragmentType(FMassRelationFragment::StaticStruct())
	{
		// at the moment UMassRelationEntityCreation doesn't do anything, so we don't plug it in
		// RelationEntityCreationObserverClass = UMassRelationEntityCreation::StaticClass();
		RelationEntityDestructionObserverClass = UMassRelationEntityDestruction::StaticClass();

		SubjectEntityDestructionObserverClass
			= ObjectEntityDestructionObserverClass = UMassRelationRoleDestruction::StaticClass();
	}

	FRelationTypeTraits::FRelationTypeTraits(const FRelationTypeTraits& Other, const TNotNull<const UScriptStruct*> NewRelationTagType)
		: FRelationTypeTraits(Other)
	{
		RelationTagType = NewRelationTagType;
	}

	void FRelationTypeTraits::SetDebugInFix(FString&& InFix)
	{
#if WITH_MASSENTITY_DEBUG
		DebugInFix = MoveTemp(InFix);
#endif // WITH_MASSENTITY_DEBUG
	}

#if WITH_MASSENTITY_DEBUG
	FString FRelationTypeTraits::DebugDescribeRelation(FMassEntityHandle A, FMassEntityHandle B) const
	{
		return FString::Printf(TEXT("[%s] %s [%s]"), *A.DebugGetDescription(), *DebugInFix, *B.DebugGetDescription());
	}
#endif // WITH_MASSENTITY_DEBUG
} // namespace UE::Mass

//-----------------------------------------------------------------------------
// FMassRelationRoleInstanceHandle
//-----------------------------------------------------------------------------
FMassEntityHandle FMassRelationRoleInstanceHandle::GetRoleEntityHandle(const FMassEntityManager& EntityManager) const
{
	const int32 EntityIndex = GetRoleEntityIndex();
	return EntityManager.CreateEntityIndexHandle(EntityIndex);
}

FMassEntityHandle FMassRelationRoleInstanceHandle::GetRelationEntityHandle(const FMassEntityManager& EntityManager) const
{
	const int32 EntityIndex = GetRelationEntityIndex();
	return EntityManager.CreateEntityIndexHandle(EntityIndex);
}
