// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntitySettings.h"
#include "MassProcessingPhaseManager.h"
#include "VisualLogger/VisualLogger.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/CoreDelegates.h"
#include "MassArchetypeData.h"
#include "Algo/RemoveIf.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntitySettings)

#if WITH_EDITOR
#include "ObjectEditorUtils.h"
#include "CoreGlobals.h"
#endif // WITH_EDITOR

//----------------------------------------------------------------------//
//  UMassEntitySettings
//----------------------------------------------------------------------//
UMassEntitySettings::UMassEntitySettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	for (int32 i = 0; i < static_cast<int32>(EMassProcessingPhase::MAX); ++i)
	{
		ProcessingPhasesConfig[i].PhaseName = *UEnum::GetDisplayValueAsText(static_cast<EMassProcessingPhase>(i)).ToString();
	}

	FCoreDelegates::GetOnPostEngineInit().AddUObject(this, &UMassEntitySettings::OnPostEngineInit);

	// we need to get notified about modules being unloaded (like Game Feature Plugins) so that we can remove
	// stored CDOs originating from the modules being removed.
	FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.AddUObject(this, &UMassEntitySettings::OnModulePackagesUnloaded);
}

void UMassEntitySettings::PostInitProperties()
{
	Super::PostInitProperties();
	ChunkMemorySize = UE::Mass::SanitizeChunkMemorySize(ChunkMemorySize);
}

void UMassEntitySettings::BeginDestroy()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.RemoveAll(this);
	Super::BeginDestroy();
}

void UMassEntitySettings::OnPostEngineInit()
{
	bEngineInitialized = true;
	BuildProcessorListAndPhases();
}

void UMassEntitySettings::OnModulePackagesUnloaded(TConstArrayView<UPackage*> Packages)
{
	const int32 InitialNum = ProcessorCDOs.Num();

	ProcessorCDOs.SetNum(Algo::RemoveIf(ProcessorCDOs,
		[&Packages](const TObjectPtr<UMassProcessor>& CDO)
		{
			return !CDO || Packages.Contains(CDO->GetPackage());
		})
	);

	if (ProcessorCDOs.Num() != InitialNum)
	{
		// rebuild the phase configs
		for (FMassProcessingPhaseConfig& PhaseConfig : ProcessingPhasesConfig)
		{
			PhaseConfig.ProcessorCDOs.Reset();
		}

		for (TObjectPtr<UMassProcessor> ProcessorCDO : ProcessorCDOs)
		{
			if (ProcessorCDO->ShouldAutoAddToGlobalList())
			{
				ProcessingPhasesConfig[static_cast<int32>(ProcessorCDO->GetProcessingPhase())].ProcessorCDOs.Add(ProcessorCDO);
			}
		}

		BuildPhases();
	}
}

void UMassEntitySettings::BuildProcessorListAndPhases()
{
	if (bInitialized == true || bEngineInitialized == false)
	{
		return;
	}

	BuildProcessorList();
	BuildPhases();
	bInitialized = true;

	OnInitializedEvent.Broadcast();
}

void UMassEntitySettings::BuildPhases()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		for (int32 i = 0; i < static_cast<int32>(EMassProcessingPhase::MAX); ++i)
		{
			FMassProcessingPhaseConfig& PhaseConfig = ProcessingPhasesConfig[i];
			PhaseConfig.PhaseProcessor = NewObject<UMassCompositeProcessor>(this, PhaseConfig.PhaseGroupClass
				, *FString::Printf(TEXT("ProcessingPhase_%s"), *PhaseConfig.PhaseName.ToString()));
			PhaseConfig.PhaseProcessor->SetGroupName(PhaseConfig.PhaseName);
			PhaseConfig.PhaseProcessor->SetProcessingPhase(static_cast<EMassProcessingPhase>(i));
			const FString PhaseDumpDependencyGraphFileName = !DumpDependencyGraphFileName.IsEmpty() ? DumpDependencyGraphFileName + TEXT("_") + PhaseConfig.PhaseName.ToString() : FString();

			FMassRuntimePipeline TmpPipeline(EProcessorExecutionFlags::All);
			TmpPipeline.CreateFromArray(PhaseConfig.ProcessorCDOs, *PhaseConfig.PhaseProcessor);
			PhaseConfig.PhaseProcessor->SetChildProcessors(TmpPipeline.MoveProcessorsArray());

			FStringOutputDevice Ar;
			PhaseConfig.PhaseProcessor->DebugOutputDescription(Ar);
			PhaseConfig.Description = FText::FromString(Ar);
		}
	}
#endif // WITH_EDITOR
}

void UMassEntitySettings::BuildProcessorList()
{
	ProcessorCDOs.Reset();
	for (FMassProcessingPhaseConfig& PhaseConfig : ProcessingPhasesConfig)
	{
		PhaseConfig.ProcessorCDOs.Reset();
	}

	TArray<UClass*> SubClasses;
	GetDerivedClasses(UMassProcessor::StaticClass(), SubClasses);

	for (int i = SubClasses.Num() - 1; i >= 0; --i)
	{
		if (SubClasses[i]->HasAnyClassFlags(CLASS_Abstract))
		{
			continue;
		}

		UMassProcessor* ProcessorCDO = GetMutableDefault<UMassProcessor>(SubClasses[i]);
		// we explicitly restrict adding UMassCompositeProcessor. If needed by specific project a derived class can be added
		if (ProcessorCDO && SubClasses[i] != UMassCompositeProcessor::StaticClass()
#if WITH_EDITOR
			&& ProcessorCDO->ShouldShowUpInSettings()
#endif // WITH_EDITOR
		)
		{
			// Observers might register later than the GC disregard window, causing a GC mismatch between this early-initialized class and them.
			const bool bIsDisregardForGC = GUObjectArray.IsDisregardForGC(this);
			if (bIsDisregardForGC && !(GUObjectArray.IsDisregardForGC(ProcessorCDO) || ProcessorCDO->HasAnyFlags(RF_MarkAsRootSet)))
			{
				ProcessorCDO->AddToRoot();
			}

			ProcessorCDOs.Add(ProcessorCDO);
			if (ProcessorCDO->ShouldAutoAddToGlobalList())
			{
				ProcessingPhasesConfig[static_cast<int32>(ProcessorCDO->GetProcessingPhase())].ProcessorCDOs.Add(ProcessorCDO);
			}
		}
	}

	ProcessorCDOs.Sort([](UMassProcessor& LHS, UMassProcessor& RHS) {
		return LHS.GetName().Compare(RHS.GetName()) < 0;
	});
}

void UMassEntitySettings::AddToActiveProcessorsList(TSubclassOf<UMassProcessor> ProcessorClass)
{
	if (UMassProcessor* ProcessorCDO = GetMutableDefault<UMassProcessor>(ProcessorClass))
	{
		if (ProcessorClass == UMassCompositeProcessor::StaticClass())
		{
			UE_VLOG_UELOG(this, LogMass, Log, TEXT("%s adding MassCompositeProcessor to the global processor list is unsupported"), ANSI_TO_TCHAR(__FUNCTION__));
		}
		else if (ProcessorClass->HasAnyClassFlags(CLASS_Abstract))
		{
			UE_VLOG_UELOG(this, LogMass, Log, TEXT("%s unable to add %s due to it being an abstract class"), ANSI_TO_TCHAR(__FUNCTION__), *ProcessorClass->GetName());
		}
		else if (ProcessorCDOs.Find(ProcessorCDO) != INDEX_NONE)
		{
			UE_VLOG_UELOG(this, LogMass, Log, TEXT("%s already in global processor list"), *ProcessorCDO->GetName());
		}
		else 
		{
			ensureMsgf(ProcessorCDO->ShouldAutoAddToGlobalList() == false, TEXT("%s missing from the global list while it's already marked to be auto-added"), *ProcessorCDO->GetName());
			ProcessorCDOs.Add(ProcessorCDO);
			ProcessorCDO->SetShouldAutoRegisterWithGlobalList(true);
		}
	}
}

TConstArrayView<FMassProcessingPhaseConfig> UMassEntitySettings::GetProcessingPhasesConfig()
{
	BuildProcessorListAndPhases();
	return MakeArrayView(ProcessingPhasesConfig, static_cast<int32>(EMassProcessingPhase::MAX));
}

#if WITH_EDITOR
void UMassEntitySettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName ProcessorCDOsName = GET_MEMBER_NAME_CHECKED(UMassEntitySettings, ProcessorCDOs);
	static const FName ChunkMemorySizeName = GET_MEMBER_NAME_CHECKED(UMassEntitySettings, ChunkMemorySize);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
	{
		// ignore adding elements to arrays since it would be 'None' at first
		return;
	}

	if (PropertyChangedEvent.Property)
	{
		const FName PropName = PropertyChangedEvent.Property->GetFName();
		if (PropName == ProcessorCDOsName)
		{
			BuildProcessorList();
		}
		else if (PropName == ChunkMemorySizeName)
		{
			ChunkMemorySize = UE::Mass::SanitizeChunkMemorySize(ChunkMemorySize);
		}

		BuildPhases();
		OnSettingsChange.Broadcast(PropertyChangedEvent);
	}
}

void UMassEntitySettings::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	static const FName AutoRegisterName = "bAutoRegisterWithProcessingPhases";

	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;
	FProperty* MemberProperty = nullptr;
	FEditPropertyChain::TDoubleLinkedListNode* LastPropertyNode = PropertyChangedEvent.PropertyChain.GetActiveMemberNode();
	while (LastPropertyNode && LastPropertyNode->GetNextNode())
	{
		LastPropertyNode = LastPropertyNode->GetNextNode();
	}

	if (LastPropertyNode)
	{
		MemberProperty = LastPropertyNode->GetValue();
	}

	if (MemberProperty && MemberProperty->GetFName() == AutoRegisterName)
	{
		BuildProcessorList();
	}
}
#endif // WITH_EDITOR

