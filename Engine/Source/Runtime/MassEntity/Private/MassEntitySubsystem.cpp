// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntitySubsystem.h"
#include "HAL/IConsoleManager.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntitySubsystem)

namespace UE::Mass::Private
{
	// UE_DEPRECATED 5.8: bEnableMassConcurrentReserveRuntime and Mass.ConcurrentReserve.Enable are deprecated.
	// Single threaded entity storage is being removed. Concurrent storage will always be used.
	static bool bEnableMassConcurrentReserveRuntime = true;
	static int32 ConcurrentReserveMaxEntityCount = 1 << 27;
	static int32 ConcurrentReserveMaxEntitiesPerPage = 1 << 16;

	namespace
	{
		FAutoConsoleVariableRef CVars[] = {
			{
				TEXT("Mass.ConcurrentReserve.Enable"),
				bEnableMassConcurrentReserveRuntime,
				TEXT("DEPRECATED 5.8: Single threaded entity storage is being removed. This CVar will be ignored in a future release."),
				ECVF_Default
			},
			{
				TEXT("Mass.ConcurrentReserve.MaxEntityCount"),
				ConcurrentReserveMaxEntityCount,
				TEXT("Set maximum number of permissible entities.  Must be power of 2."),
				ECVF_Default
			},
			{
				TEXT("Mass.ConcurrentReserve.EntitiesPerPage"),
				ConcurrentReserveMaxEntitiesPerPage,
				TEXT("Set number of entities per page. Must be power of 2. Larger reduces fixed memory overhead of FEntityData page lookup but requires bigger contiguous memory blocks per page"),
				ECVF_Default
			}
		};
	}
} // namespace UE::Mass::Private

//-----------------------------------------------------------------------------
// UMassEntitySubsystem
//-----------------------------------------------------------------------------
UMassEntitySubsystem::UMassEntitySubsystem()
	: EntityManager(MakeShareable(new FMassEntityManager(this)))
{
	
}

void UMassEntitySubsystem::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	EntityManager->GetResourceSizeEx(CumulativeResourceSize);
}

void UMassEntitySubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	FMassEntityManagerStorageInitParams InitializationParams;
#if WITH_MASS_CONCURRENT_RESERVE
	if (UE::Mass::Private::bEnableMassConcurrentReserveRuntime)
	{
		InitializationParams.Emplace<FMassEntityManager_InitParams_Concurrent>(
			FMassEntityManager_InitParams_Concurrent
			{
				.MaxEntityCount = static_cast<uint32>(UE::Mass::Private::ConcurrentReserveMaxEntityCount),
				.MaxEntitiesPerPage = static_cast<uint32>(UE::Mass::Private::ConcurrentReserveMaxEntitiesPerPage)
			});
	}
	else
#endif // WITH_MASS_CONCURRENT_RESERVE
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InitializationParams.Emplace<FMassEntityManager_InitParams_SingleThreaded>();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	EntityManager->Initialize(InitializationParams);
	HandleLateCreation();

	UE::Mass::Subsystems::RegisterSubsystemType(EntityManager.ToSharedRef(), GetClass(), UE::Mass::FSubsystemTypeTraits::Make<UMassEntitySubsystem>());
}

void UMassEntitySubsystem::PostInitialize()
{
	Super::PostInitialize();
	// this needs to be done after all the subsystems have been initialized since some processors might want to access
	// them during processors' initialization
	EntityManager->PostInitialize();
}

void UMassEntitySubsystem::Deinitialize()
{
	// Frameworks like SceneGraph use Mass entities and dedicated fragments to register in multiple
	// other frameworks for the lifetime of their world objects (e.g., Render proxies, physics objects, navigation elements, etc.).
	// Deferred commands to destroy those Mass entities will get pushed when removing all these objects
	// from the scene/world (i.e., UWorld::CleanupWorldInternal), and we need these commands to be flushed to perform proper cleanup.
	EntityManager->FlushCommands();

	// @todo: Similar scenario exists for entities with a lifetime tied to the world simulation and gated by BeginPlay/EndPlay.
	// We should consider flushing commands there too (i.e, WorldEndPlay) before all world subsystems and other frameworks handle their endplay callbacks.

	EntityManager->Deinitialize();
	EntityManager.Reset();
	Super::Deinitialize();
}

#if WITH_MASSENTITY_DEBUG
//-----------------------------------------------------------------------------
// Debug commands
//-----------------------------------------------------------------------------
FAutoConsoleCommandWithWorldArgsAndOutputDevice GPrintArchetypesCmd(
	TEXT("EntityManager.PrintArchetypes"),
	TEXT("Prints information about all archetypes in the current world"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Params, UWorld* World, FOutputDevice& Ar)
		{
			if (const UMassEntitySubsystem* EntitySubsystem = World ? World->GetSubsystem<UMassEntitySubsystem>() : nullptr)
			{
				EntitySubsystem->GetEntityManager().DebugPrintArchetypes(Ar);
			}
			else
			{
				Ar.Logf(ELogVerbosity::Error, TEXT("Failed to find Entity Subsystem for world %s"), *GetPathNameSafe(World));
			}
		}));
#endif // WITH_MASSENTITY_DEBUG