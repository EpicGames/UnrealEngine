// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityTrace.h"

#if UE_MASS_TRACE_ENABLED

#include "MassArchetypeData.h"
#include "MassEntityTypes.h"
#include "MassEntityQuery.h"
#include "MassEntityUtils.h"
#include "MassProcessor.h"
#include "MassDebugger.h"

#include "Trace/Trace.inl"
#include "TraceFilter.h"

namespace UE::Mass::Trace
{
static double GMassRecordingTime = -1.0;
}

UE_TRACE_CHANNEL_DEFINE(MassChannel, "Mass Entity system archetypes, queries, and chunk execution.");

UE_TRACE_EVENT_BEGIN(MassTrace, MassPhaseBegin)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, PhaseName)
	UE_TRACE_EVENT_FIELD(uint64, PhaseId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, MassPhaseEnd)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, PhaseName)
	UE_TRACE_EVENT_FIELD(uint64, PhaseId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, RegisterMassArchetype)
	UE_TRACE_EVENT_FIELD(uint64, ArchetypeID)
	UE_TRACE_EVENT_FIELD(uint64[], Fragments)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, RegisterMassFragment)
	UE_TRACE_EVENT_FIELD(uint64, FragmentId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, FragmentName)
	UE_TRACE_EVENT_FIELD(uint32, FragmentSize)
	UE_TRACE_EVENT_FIELD(uint8, FragmentType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, MassPhaseExecutionBegin)
	UE_TRACE_EVENT_FIELD(uint64, PhaseId)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, MassPhaseExecutionEnd)
	UE_TRACE_EVENT_FIELD(uint64, PhaseId)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, MassExecuteChunk)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, ChunkId)
	UE_TRACE_EVENT_FIELD(uint64, QueryID)
	UE_TRACE_EVENT_FIELD(int32, EntityCount)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, MassExecuteChunkEnd)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, ChunkId)
	UE_TRACE_EVENT_FIELD(uint64, QueryID)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, MassBulkAddEntity)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64[], Entities)
	UE_TRACE_EVENT_FIELD(uint64[], ArchetypeIDs)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, MassBulkEntityDestroyed)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64[], Entities)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, MassEntityMoved)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, Entity)
	UE_TRACE_EVENT_FIELD(uint64, NewArchetypeID)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, QueryCreated)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, QueryID)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, QueryDestroyed)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, QueryID)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, QueryRegisteredToProcessor)
	UE_TRACE_EVENT_FIELD(uint64, QueryID)
	UE_TRACE_EVENT_FIELD(uint64, ProcessorID)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ProcessorName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, QueryArchetypeAdded)
	UE_TRACE_EVENT_FIELD(uint64, QueryID)
	UE_TRACE_EVENT_FIELD(uint64, ArchetypeID)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, QueryForEachStarted)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, QueryID)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(MassTrace, QueryForEachComplete)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, QueryID)
	UE_TRACE_EVENT_FIELD(int32, ArchetypeCount)
	UE_TRACE_EVENT_FIELD(int32, ChunkCount)
	UE_TRACE_EVENT_FIELD(int32, EntityCount)
UE_TRACE_EVENT_END()

enum class EFragmentType : uint8
{
	Unknown = 0,
	Fragment,
	Tag,
	SharedFragment,
	Sparse,
	ConstSharedFragment,
	SparseTag
};

//----------------------------------------------------------------//
//  FMassTrace
//----------------------------------------------------------------//
void FMassTrace::QueryCreated(const FMassEntityQuery* Query)
{
	UE_TRACE_LOG(MassTrace, QueryCreated, MassChannel)
		<< QueryCreated.Cycle(FPlatformTime::Cycles64())
		<< QueryCreated.QueryID(reinterpret_cast<uint64>(Query));
}

void FMassTrace::QueryDestroyed(const FMassEntityQuery* Query)
{
	UE_TRACE_LOG(MassTrace, QueryDestroyed, MassChannel)
		<< QueryDestroyed.Cycle(FPlatformTime::Cycles64())
		<< QueryDestroyed.QueryID(reinterpret_cast<uint64>(Query));
}

void FMassTrace::QueryRegisteredToProcessor(const FMassEntityQuery* Query, TNotNull<const UMassProcessor*> Processor)
{
	UE_TRACE_LOG(MassTrace, QueryRegisteredToProcessor, MassChannel)
		<< QueryRegisteredToProcessor.QueryID(reinterpret_cast<uint64>(Query))
		<< QueryRegisteredToProcessor.ProcessorID(reinterpret_cast<uint64>(static_cast<const UMassProcessor*>(Processor)))
		<< QueryRegisteredToProcessor.ProcessorName(*Processor->GetProcessorName());
}

void FMassTrace::QueryArchetypeAdded(const FMassEntityQuery* Query, const FMassArchetypeHandle& Archetype)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		uint64 ArchetypeID = reinterpret_cast<uint64>(FMassArchetypeHelper::ArchetypeDataFromHandle(Archetype));

		UE_TRACE_LOG(MassTrace, QueryArchetypeAdded, MassChannel)
			<< QueryArchetypeAdded.QueryID(reinterpret_cast<uint64>(Query))
			<< QueryArchetypeAdded.ArchetypeID(ArchetypeID);
	}
}

//----------------------------------------------------------------//
//  FMassTrace::FScopedQueryForEachTrace
//----------------------------------------------------------------//
FMassTrace::FScopedQueryForEachTrace::FScopedQueryForEachTrace(const FMassEntityQuery* InQuery)
	: Query(InQuery)
{
	UE_TRACE_LOG(MassTrace, QueryForEachStarted, MassChannel)
		<< QueryForEachStarted.Cycle(FPlatformTime::Cycles64())
		<< QueryForEachStarted.QueryID(reinterpret_cast<uint64>(Query));
}

FMassTrace::FScopedQueryForEachTrace::~FScopedQueryForEachTrace()
{
	UE_TRACE_LOG(MassTrace, QueryForEachComplete, MassChannel)
		<< QueryForEachComplete.Cycle(FPlatformTime::Cycles64())
		<< QueryForEachComplete.QueryID(reinterpret_cast<uint64>(Query))
		<< QueryForEachComplete.ArchetypeCount(ArchetypeCount)
		<< QueryForEachComplete.ChunkCount(ChunkCount)
		<< QueryForEachComplete.EntityCount(EntityCount);
}

void FMassTrace::FScopedQueryForEachTrace::ReportArchetype(const FMassArchetypeData& Archetype)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		++ArchetypeCount;
		ChunkCount += Archetype.GetChunkCount();
		EntityCount += Archetype.GetNumEntities();
	}
}

void FMassTrace::OutputRegisterFragment(const UScriptStruct* Struct)
{
	// Determine the element type for tracing. Checks derived types before base types
	// since FMassSparseFragment/FMassSparseTag inherit from FMassFragment/FMassTag.
	const EFragmentType FragmentType = [](const UScriptStruct* Struct) -> EFragmentType
		{
			if (Struct->IsChildOf<FMassSparseFragment>())
			{
				return EFragmentType::Sparse;
			}
			if (Struct->IsChildOf<FMassConstSharedFragment>())
			{
				return EFragmentType::ConstSharedFragment;
			}
			else if (Struct->IsChildOf<FMassSharedFragment>())
			{
				return EFragmentType::SharedFragment;
			}
			else if (Struct->IsChildOf<FMassFragment>())
			{
				return EFragmentType::Fragment;
			}
			else if (Struct->IsChildOf<FMassSparseTag>())
			{
				return EFragmentType::SparseTag;
			}
			else if (Struct->IsChildOf<FMassTag>())
			{
				return EFragmentType::Tag;
			}
			else
			{
				return EFragmentType::Unknown;
			}
		}(Struct);

	UE_TRACE_LOG(MassTrace, RegisterMassFragment, MassChannel)
		<< RegisterMassFragment.FragmentId(reinterpret_cast<uint64>(Struct))
		<< RegisterMassFragment.FragmentName(*Struct->GetName())
		<< RegisterMassFragment.FragmentSize(Struct->GetStructureSize())
		<< RegisterMassFragment.FragmentType(static_cast<uint8>(FragmentType));
}

void FMassTrace::OnPhaseBegin(uint64 PhaseId)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		const FString EnumName = StaticEnum<EMassProcessingPhase>()->GetNameStringByValue(PhaseId);
		OutputBeginPhaseRegion(*EnumName);
	}
}

void FMassTrace::OnPhaseEnd(uint64 PhaseId)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		const FString EnumName = StaticEnum<EMassProcessingPhase>()->GetNameStringByValue(PhaseId);
		OutputEndPhaseRegion(*EnumName);
	}
}

uint64 FMassTrace::OutputRegisterArchetype(uint64 ArchetypeID, const FMassArchetypeCompositionDescriptor& CompositionDescriptor)
{
	TArray<uint64> FragmentsScratch;
	const FMassElementBitSet& AllElements = CompositionDescriptor.GetElementsBitSet();

	// Include fragments, tags, shared fragments, AND sparse elements
	const FMassElementBitSet FragmentsAndTags = AllElements.GetFragmentsAndTags();
	const FMassElementBitSet SharedFragments = AllElements.GetSharedFragments();
	const FMassElementBitSet SparseElements = AllElements.GetSparseElements();

	FragmentsScratch.Reserve(FragmentsAndTags.CountStoredTypes() + SharedFragments.CountStoredTypes() + SparseElements.CountStoredTypes());

	auto RegisterElements = [&FragmentsScratch](const FMassElementBitSet& Elements)
	{
		for (auto It = Elements.GetIndexIterator(); It; ++It)
		{
			const UScriptStruct* FragmentStruct = Elements.GetTypeAtIndex(*It);
			OutputRegisterFragment(FragmentStruct);
			FragmentsScratch.Add(reinterpret_cast<uint64>(FragmentStruct));
		}
	};

	RegisterElements(FragmentsAndTags);
	RegisterElements(SharedFragments);
	RegisterElements(SparseElements);

	UE_TRACE_LOG(MassTrace, RegisterMassArchetype, MassChannel)
		<< RegisterMassArchetype.ArchetypeID(ArchetypeID)
		<< RegisterMassArchetype.Fragments(FragmentsScratch.GetData(), FragmentsScratch.Num());

	return ArchetypeID;
}

uint64 FMassTrace::RegisterArchetype(const FMassArchetypeHandle& ArchetypeHandle)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		const FMassArchetypeCompositionDescriptor& CompositionDescriptor = FMassDebugger::GetArchetypeComposition(ArchetypeHandle);
		const uint64 ArchetypeID = FMassDebugger::GetArchetypeTraceID(ArchetypeHandle);

		return OutputRegisterArchetype(ArchetypeID, CompositionDescriptor);
	}
	return 0;
}

uint64 FMassTrace::RegisterArchetype(const FMassArchetypeData& Data)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		const FMassArchetypeCompositionDescriptor& CompositionDescriptor = Data.GetCompositionDescriptor();
		const uint64 ArchetypeID = FMassDebugger::GetArchetypeTraceID(Data);

		return OutputRegisterArchetype(ArchetypeID, CompositionDescriptor);
	}
	return 0;
}

void FMassTrace::RegisterFragment(const UScriptStruct* Struct)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		OutputRegisterFragment(Struct);
	}
}

void FMassTrace::EntityCreated(FMassEntityHandle Entity, const FMassArchetypeData& Archetype)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		const uint64 Cycle = FPlatformTime::Cycles64();
		const uint64 EntityAsU64 = Entity.AsNumber();
		const uint64 ArchetypeID = FMassDebugger::GetArchetypeTraceID(Archetype);
		UE_TRACE_LOG(MassTrace, MassBulkAddEntity, MassChannel)
			<< MassBulkAddEntity.Cycle(Cycle)
			<< MassBulkAddEntity.RecordingTime(UE::Mass::Trace::GMassRecordingTime)
			<< MassBulkAddEntity.Entities(&EntityAsU64, 1)
			<< MassBulkAddEntity.ArchetypeIDs(&ArchetypeID, 1);
	}
}

void FMassTrace::EntityMoved(FMassEntityHandle Entity, const FMassArchetypeData& NewArchetype)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		const uint64 Cycle = FPlatformTime::Cycles64();
		const uint64 EntityAsU64 = Entity.AsNumber();
		UE_TRACE_LOG(MassTrace, MassEntityMoved, MassChannel)
			<< MassEntityMoved.Cycle(Cycle)
			<< MassEntityMoved.RecordingTime(UE::Mass::Trace::GMassRecordingTime)
			<< MassEntityMoved.Entity(EntityAsU64)
			<< MassEntityMoved.NewArchetypeID(FMassDebugger::GetArchetypeTraceID(NewArchetype));
	}
}

void FMassTrace::EntityDestroyed(FMassEntityHandle Entity)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		const uint64 Cycle = FPlatformTime::Cycles64();
		const uint64 EntityAsU64 = Entity.AsNumber();
		UE_TRACE_LOG(MassTrace, MassBulkEntityDestroyed, MassChannel)
			<< MassBulkEntityDestroyed.Cycle(Cycle)
			<< MassBulkEntityDestroyed.RecordingTime(UE::Mass::Trace::GMassRecordingTime)
			<< MassBulkEntityDestroyed.Entities(&EntityAsU64, 1);
	}
}

void FMassTrace::EntitiesDestroyed(TConstArrayView<FMassEntityHandle> Entities)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		const uint64 Cycle = FPlatformTime::Cycles64();

		TConstArrayView<uint64> EntitiesAsU64(
			reinterpret_cast<const uint64*>(Entities.GetData()),
			Entities.Num());
		UE_TRACE_LOG(MassTrace, MassBulkEntityDestroyed, MassChannel)
			<< MassBulkEntityDestroyed.Cycle(Cycle)
			<< MassBulkEntityDestroyed.RecordingTime(UE::Mass::Trace::GMassRecordingTime)
			<< MassBulkEntityDestroyed.Entities(EntitiesAsU64.GetData(), EntitiesAsU64.Num());
	}
}

void FMassTrace::SetRecordingTime(const double InRecordingTime)
{
	UE::Mass::Trace::GMassRecordingTime = InRecordingTime;
}

uint64 FMassTrace::OutputBeginPhaseWithID(const TCHAR* PhaseName)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MassChannel))
	{
		const uint64 PhaseId = FPlatformTime::Cycles64();
		UE_TRACE_LOG(MassTrace, MassPhaseBegin, MassChannel)
			<< MassPhaseBegin.Cycle(FPlatformTime::Cycles64())
			<< MassPhaseBegin.PhaseName(PhaseName)
			<< MassPhaseBegin.PhaseId(PhaseId);
		return PhaseId;
	}
	return 0;
}

void FMassTrace::OutputBeginPhaseRegion(const TCHAR* PhaseName)
{
	UE_TRACE_LOG(MassTrace, MassPhaseBegin, MassChannel)
		<< MassPhaseBegin.Cycle(FPlatformTime::Cycles64())
		<< MassPhaseBegin.PhaseName(PhaseName)
		<< MassPhaseBegin.PhaseId(0);
}

void FMassTrace::OutputEndPhaseRegion(const TCHAR* PhaseName)
{
	UE_TRACE_LOG(MassTrace, MassPhaseEnd, MassChannel)
		<< MassPhaseEnd.Cycle(FPlatformTime::Cycles64())
		<< MassPhaseEnd.PhaseName(PhaseName)
		<< MassPhaseEnd.PhaseId(0);
}

void FMassTrace::OutputEndPhaseRegion(uint64 PhaseId)
{
	UE_TRACE_LOG(MassTrace, MassPhaseEnd, MassChannel)
		<< MassPhaseEnd.Cycle(FPlatformTime::Cycles64())
		<< MassPhaseEnd.PhaseId(PhaseId);
}

#endif //UE_MASS_TRACE_ENABLED