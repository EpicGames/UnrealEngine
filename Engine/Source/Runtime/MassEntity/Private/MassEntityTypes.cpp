// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityTypes.h"

#include "MassArchetypeData.h"
#include "MassArchetypeTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityTypes)

DEFINE_STAT(STAT_Mass_Total);

FString LexToString(const EMassObservedOperationFlags Value)
{
	switch (Value)
	{
	case EMassObservedOperationFlags::None:
		return ("None");
	case EMassObservedOperationFlags::AddElement:
		return ("AddElement");
	case EMassObservedOperationFlags::RemoveElement:
		return ("RemoveElement");
	case EMassObservedOperationFlags::CreateEntity:
		return ("CreateEntity");
	case EMassObservedOperationFlags::DestroyEntity:
		return ("DestroyEntity");
	case EMassObservedOperationFlags::Add:
		return ("AddElement | CreateEntity");
	case EMassObservedOperationFlags::Remove:
		return ("RemoveElement | DestroyEntity");
	case EMassObservedOperationFlags::All:
		return ("Add | Remove");
	default:
		return "UNDEFINED";
	}
}

//-----------------------------------------------------------------------------
// FMassArchetypeCompositionDescriptor
//-----------------------------------------------------------------------------
FMassArchetypeCompositionDescriptor::FMassArchetypeCompositionDescriptor(const FMassArchetypeHandle& ArchetypeHandle)
{
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	*this = ArchetypeData.GetCompositionDescriptor();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int32 FMassArchetypeCompositionDescriptor::CountStoredTypes() const
{
	return ElementsBitSet.CountStoredTypes();
}

void FMassArchetypeCompositionDescriptor::DebugOutputDescription(FOutputDevice& Ar) const
{
#if WITH_MASSENTITY_DEBUG 
	if (ElementsBitSet.IsEmpty())
	{
		Ar.Logf(TEXT("Empty"));
		return;
	}

	const bool bAutoLineEnd = Ar.GetAutoEmitLineTerminator();
	Ar.SetAutoEmitLineTerminator(false);

	Ar.Logf(TEXT("Elements:\n"));
	ElementsBitSet.DebugGetStringDesc(Ar);

	Ar.SetAutoEmitLineTerminator(bAutoLineEnd);
#endif // WITH_MASSENTITY_DEBUG
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//-----------------------------------------------------------------------------
// FMassArchetypeSharedFragmentValues
//-----------------------------------------------------------------------------
FMassArchetypeSharedFragmentValues FMassArchetypeSharedFragmentValues::CreateCombined(const FMassArchetypeSharedFragmentValues& Original
	, const FMassElementBitSet& NewSharedElementComposition, const FMassArchetypeSharedFragmentValues* SharedFragmentValuesModification)
{
	FMassArchetypeSharedFragmentValues NewSharedValues = Original;
	if (SharedFragmentValuesModification)
	{
		NewSharedValues.Append(*SharedFragmentValuesModification);
	}

	// remove everything that's in ConstSharedFragments or SharedFragments that's not in NewSharedElementComposition
	auto RemovePredicate = [&NewSharedElementComposition](const FConstSharedStruct& SharedStruct)
		{
			return SharedStruct.GetScriptStruct() == nullptr
				|| NewSharedElementComposition.Contains(SharedStruct.GetScriptStruct()) == false;
		};

	NewSharedValues.SharedFragments.RemoveAllSwap(RemovePredicate);
	NewSharedValues.ConstSharedFragments.RemoveAllSwap(RemovePredicate);
	NewSharedValues.StoredElementsBitSet = NewSharedElementComposition;

	NewSharedValues.Sort();
	return NewSharedValues;
}

FConstSharedStruct FMassArchetypeSharedFragmentValues::Add_GetRef(const FConstSharedStruct& Fragment)
{
	check(Fragment.IsValid());
	const UScriptStruct* StructType = Fragment.GetScriptStruct();
	if (ContainsType(StructType))
	{
		FConstSharedStruct ExistingConstSharedStruct = GetConstSharedFragmentStruct(StructType);
		ensureMsgf(false, TEXT("Shared Fragment of type %s already added to FMassArchetypeSharedFragmentValues%s")
			, *GetNameSafe(StructType)
			, ExistingConstSharedStruct.IsValid() ? TEXT("") : TEXT(" as NON-CONST shared struct"));
		return ExistingConstSharedStruct;
	}

	check(StructType);
	StoredElementsBitSet.Add(StructType);
	FConstSharedStruct& StructInstance = ConstSharedFragments.Add_GetRef(Fragment);
	DirtyHashCache();
	return StructInstance;
}

FSharedStruct FMassArchetypeSharedFragmentValues::Add_GetRef(const FSharedStruct& Fragment)
{
	check(Fragment.IsValid());
	const UScriptStruct* StructType = Fragment.GetScriptStruct();
	if (ContainsType(StructType))
	{
		FSharedStruct ExistingSharedStruct = GetSharedFragmentStruct(StructType);
		ensureMsgf(false, TEXT("Shared Fragment of type %s already added to FMassArchetypeSharedFragmentValues%s")
			, *GetNameSafe(StructType)
			, ExistingSharedStruct.IsValid() ? TEXT("") : TEXT(" as CONST shared struct"));
		return ExistingSharedStruct;
	}

	check(StructType);
	StoredElementsBitSet.Add(StructType);
	FSharedStruct& StructInstance = SharedFragments.Add_GetRef(Fragment);
	DirtyHashCache();
	return StructInstance;
}

uint32 FMassArchetypeSharedFragmentValues::CalculateHash() const
{
	if (!testableEnsureMsgf(bSorted, TEXT("Expecting the containers to be sorted for the hash calculation to be consistent")))
	{
		return 0;
	}

	// Fragments are not part of the uniqueness 
	uint32 Hash = 0;
	for (const FConstSharedStruct& Fragment : ConstSharedFragments)
	{
		Hash = PointerHash(Fragment.GetMemory(), Hash);
	}

	for (const FSharedStruct& Fragment : SharedFragments)
	{
		Hash = PointerHash(Fragment.GetMemory(), Hash);
	}

	return Hash;
}

namespace UE::Mass::Private
{
	template<typename TSharedStruct>
	int32 CountInvalid(const TArray<TSharedStruct>& View)
	{
		int32 Count = 0;
		for (const TSharedStruct& SharedStruct : View)
		{
			const UScriptStruct* StructType = SharedStruct.GetScriptStruct();
			Count += StructType ? 0 : 1;
		}
		return Count;
	}

	/** Note that this function assumes that both ViewA and ViewB do not contain duplicates */
	template<typename TSharedStruct, bool bSkipNulls=true>
	bool ArraysHaveSameContents(const TArray<TSharedStruct>& ViewA, const TArray<TSharedStruct>& ViewB)
	{
		if constexpr (bSkipNulls)
		{
			const int32 NullstCountA = CountInvalid(ViewA);
			const int32 NullstCountB = CountInvalid(ViewB);
			if (ViewA.Num() - NullstCountA != ViewB.Num() - NullstCountB)
			{
				return false;
			}
		}
		else if (ViewA.Num() != ViewB.Num())
		{
			return false;
		}

		for (const TSharedStruct& SharedStruct : ViewA)
		{
			const UScriptStruct* StructType = SharedStruct.GetScriptStruct();
			if constexpr (bSkipNulls)
			{
				if (StructType == nullptr)
				{
					continue;
				}
			}
			const int32 FragmentIndex = ViewB.IndexOfByPredicate(FStructTypeEqualOperator(StructType));
			if (FragmentIndex == INDEX_NONE)
			{
				return false;
			}
			if (ViewB[FragmentIndex].CompareStructValues(SharedStruct) == false)
			{
				return false;
			}
		}

		return true;
	}
} // namespace UE::Mass::Private

bool FMassArchetypeSharedFragmentValues::HasSameValues(const FMassArchetypeSharedFragmentValues& Other) const
{
	if (StoredElementsBitSet.IsEquivalent(Other.StoredElementsBitSet) == false)
	{
		return false;
	}

	return UE::Mass::Private::ArraysHaveSameContents(SharedFragments, Other.GetSharedFragments())
		&& UE::Mass::Private::ArraysHaveSameContents(ConstSharedFragments, Other.GetConstSharedFragments());
}

int32 FMassArchetypeSharedFragmentValues::Append(const FMassArchetypeSharedFragmentValues& Other)
{
	int32 AddedOrModifiedCount = 0;

	for (const FSharedStruct& SharedStruct : Other.GetSharedFragments())
	{
		const UScriptStruct* StructType = SharedStruct.GetScriptStruct();
		check(StructType);
		if (StoredElementsBitSet.Contains(StructType))
		{
			const int32 FragmentIndex = SharedFragments.IndexOfByPredicate(FStructTypeEqualOperator(StructType));
			checkf(FragmentIndex != INDEX_NONE, TEXT("Mismatch between shared fragment bitset and stored values"));
			SharedFragments[FragmentIndex] = SharedStruct;
			++AddedOrModifiedCount;
		}
		else
		{
			SharedFragments.Add(SharedStruct);
			++AddedOrModifiedCount;
		}
	}

	for (const FConstSharedStruct& SharedStruct : Other.GetConstSharedFragments())
	{
		const UScriptStruct* StructType = SharedStruct.GetScriptStruct();
		check(StructType);
		if (StoredElementsBitSet.Contains(StructType))
		{
			const int32 FragmentIndex = ConstSharedFragments.IndexOfByPredicate(FStructTypeEqualOperator(StructType));
			checkf(FragmentIndex != INDEX_NONE, TEXT("Mismatch between const shared fragment bitset and stored values"));
			ConstSharedFragments[FragmentIndex] = SharedStruct;
			++AddedOrModifiedCount;
		}
		else
		{
			ConstSharedFragments.Add(SharedStruct);
			++AddedOrModifiedCount;
		}
	}

	StoredElementsBitSet += Other.StoredElementsBitSet;

	DirtyHashCache();

	return AddedOrModifiedCount;
}

int32 FMassArchetypeSharedFragmentValues::Remove(const FMassElementBitSet& ElementBitSet)
{
	int32 RemovedCount = 0;
	FMassElementBitSet CommonFragments = (StoredElementsBitSet & ElementBitSet);
	FMassConstSharedFragmentBitSet::FIndexIterator It = CommonFragments.GetIndexIterator();
	while(It)
	{
		const UScriptStruct* StructType = CommonFragments.GetTypeAtIndex(*It);
		check(StructType);

		const int32 RegularFragmentIndex = SharedFragments.IndexOfByPredicate(FStructTypeEqualOperator(StructType));
		if (RegularFragmentIndex != INDEX_NONE)
		{
			SharedFragments[RegularFragmentIndex].Reset();
			++RemovedCount;
		}
		else
		{
			const int32 ConstRegularFragmentIndex = ConstSharedFragments.IndexOfByPredicate(FStructTypeEqualOperator(StructType));
			if (ConstRegularFragmentIndex != INDEX_NONE)
			{
				ConstSharedFragments[ConstRegularFragmentIndex].Reset();
				++RemovedCount;
			}
		}

		++It;
	}

	if (RemovedCount)
	{
		ConstSharedFragments.RemoveAllSwap([](const FConstSharedStruct& SharedStruct) { return !SharedStruct.IsValid(); });
		SharedFragments.RemoveAllSwap([](const FConstSharedStruct& SharedStruct) { return !SharedStruct.IsValid(); });

		StoredElementsBitSet -= CommonFragments;
		DirtyHashCache();
	}
	return RemovedCount;
}

//-----------------------------------------------------------------------------
// FMassGenericPayloadView
//-----------------------------------------------------------------------------
void FMassGenericPayloadView::SwapElementsToEnd(const int32 StartIndex, int32 NumToMove)
{
	check(StartIndex >= 0 && NumToMove >= 0);

	if (UNLIKELY(NumToMove <= 0 || StartIndex < 0))
	{
		return;
	}

	TArray<uint8, TInlineAllocator<16>> MovedElements;

	for (FStructArrayView& StructArrayView : Content)
	{
		check((StartIndex + NumToMove) <= StructArrayView.Num());
		if (StartIndex + NumToMove >= StructArrayView.Num() - 1)
		{
			// nothing to do here, the elements are already at the back
			continue;
		}

		uint8* ViewData = static_cast<uint8*>(StructArrayView.GetData());
		const uint32 ElementSize = StructArrayView.GetTypeSize();
		const uint32 MovedStartOffset = StartIndex * ElementSize;
		const uint32 MovedSize = NumToMove * ElementSize;
		const uint32 MoveOffset = (StructArrayView.Num() - (StartIndex + NumToMove)) * ElementSize;

		MovedElements.Reset();
		MovedElements.Append(ViewData + MovedStartOffset, MovedSize);
		FMemory::Memmove(ViewData + MovedStartOffset, ViewData + MovedStartOffset + MovedSize, MoveOffset);
		FMemory::Memcpy(ViewData + MovedStartOffset + MoveOffset, MovedElements.GetData(), MovedSize);
	}
}

//-----------------------------------------------------------------------------
// FMassArchetypeCreationParams
//-----------------------------------------------------------------------------
FMassArchetypeCreationParams::FMassArchetypeCreationParams(const FMassArchetypeData& Archetype)
	: ChunkMemorySize(static_cast<int32>(Archetype.GetChunkAllocSize()))
{
}