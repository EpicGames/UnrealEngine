// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityUtils.h"
#include "MassEntityTypes.h"
#include "MassArchetypeTypes.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/World.h"
#if WITH_EDITOR
#include "Editor.h"
#endif // WITH_EDITOR

namespace UE::Mass::Utils
{
EProcessorExecutionFlags GetProcessorExecutionFlagsForWorld(const UWorld& World)
{
#if WITH_EDITOR
	if (World.IsEditorWorld() && !World.IsGameWorld())
	{
		return EProcessorExecutionFlags::EditorWorld;
	}
#endif // WITH_EDITOR

	switch (const ENetMode NetMode = World.GetNetMode())
	{
	case NM_ListenServer:
		return EProcessorExecutionFlags::Client | EProcessorExecutionFlags::Server;
	case NM_DedicatedServer:
		return EProcessorExecutionFlags::Server;
	case NM_Client:
		return EProcessorExecutionFlags::Client;
	case NM_Standalone:
		return EProcessorExecutionFlags::Standalone;
	default:
		checkf(false, TEXT("Unsupported ENetMode type (%i) found while determining MASS processor execution flags."), NetMode);
		return EProcessorExecutionFlags::None;
	}
}

EProcessorExecutionFlags DetermineProcessorExecutionFlags(const UWorld* World, EProcessorExecutionFlags ExecutionFlagsOverride)
{
	if (ExecutionFlagsOverride != EProcessorExecutionFlags::None)
	{
		return ExecutionFlagsOverride;
	}
	if (World)
	{
		return GetProcessorExecutionFlagsForWorld(*World);
	}

#if WITH_EDITOR
	if (GEditor)
	{
		return EProcessorExecutionFlags::Editor;
	}
#endif // WITH_EDITOR
	return EProcessorExecutionFlags::All;
}

uint8 DetermineProcessorSupportedTickTypes(const UWorld* World)
{
#if WITH_EDITOR
	if (World != nullptr && GetProcessorExecutionFlagsForWorld(*World) == EProcessorExecutionFlags::EditorWorld)
	{
		return MAX_uint8;
	}
#endif // WITH_EDITOR
	return (1 << LEVELTICK_All) | (1 << LEVELTICK_TimeOnly);
}

void CreateEntityCollections(const FMassEntityManager& EntityManager, const TConstArrayView<FMassEntityHandle> Entities
	, const FMassArchetypeEntityCollection::EDuplicatesHandling DuplicatesHandling, TArray<FMassArchetypeEntityCollection>& OutEntityCollections)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("Mass_CreateSparseChunks");

	// find first valid entity
	int32 EntityIndex = 0;
	for (; EntityIndex < Entities.Num(); ++EntityIndex)
	{
		if (EntityManager.IsEntityValid(Entities[EntityIndex]))
		{
			break;
		}
	}

	if (EntityIndex >= Entities.Num())
	{
		// no valid entities
		return;
	}

	FMassArchetypeHandle Archetype = EntityManager.GetArchetypeForEntityUnsafe(Entities[EntityIndex]);
	if (Entities.Num() - EntityIndex == 1)
	{
		// common, trivial case, so we can handle it directly
		OutEntityCollections.Add(FMassArchetypeEntityCollection(Archetype, Entities[EntityIndex]));
		return;
	}

	TMap<const FMassArchetypeHandle, TArray<FMassEntityHandle>> ArchetypeToEntities;

	TArray<FMassEntityHandle>* PerArchetypeEntities = &ArchetypeToEntities.Add(Archetype);
	PerArchetypeEntities->Add(Entities[EntityIndex++]);

	for (; EntityIndex < Entities.Num(); ++EntityIndex)
	{
		const FMassEntityHandle& Entity = Entities[EntityIndex];
		if (EntityManager.IsEntityValid(Entity))
		{
			FMassArchetypeHandle LocalArchetype = EntityManager.GetArchetypeForEntityUnsafe(Entity);
			if (Archetype != LocalArchetype)
			{
				Archetype = LocalArchetype;
				PerArchetypeEntities = &ArchetypeToEntities.FindOrAdd(Archetype);
			}
			
			PerArchetypeEntities->Add(Entity);
		}
	}

	for (auto& Pair : ArchetypeToEntities)
	{
		OutEntityCollections.Add(FMassArchetypeEntityCollection(Pair.Key, Pair.Value, DuplicatesHandling));
	}
}

FMassEntityManager* GetEntityManager(const UObject* WorldContextObject)
{
	return WorldContextObject
		? GetEntityManager(WorldContextObject->GetWorld())
		: nullptr;
}

FMassEntityManager* GetEntityManager(const UWorld* World)
{
	UMassEntitySubsystem* EntityManager = UWorld::GetSubsystem<UMassEntitySubsystem>(World);
	return EntityManager
		? &EntityManager->GetMutableEntityManager()
		: nullptr;
}

FMassEntityManager& GetEntityManagerChecked(const UWorld& World)
{
	UMassEntitySubsystem* EntityManager = UWorld::GetSubsystem<UMassEntitySubsystem>(&World);
	check(EntityManager);
	return EntityManager->GetMutableEntityManager();
}

} // namespace UE::Mass::Utils