// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityView.h"
#include "MassEntityManager.h"
#include "MassArchetypeData.h"
#include "Mass/TestableEnsures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityView)


//-----------------------------------------------------------------------------
// FMassEntityView
//-----------------------------------------------------------------------------
FMassEntityView::FMassEntityView(const FMassArchetypeHandle& ArchetypeHandle, FMassEntityHandle InEntity)
{
	Entity = InEntity;
	Archetype = &FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	EntityDataHandle = Archetype->MakeEntityHandle(Entity);
}

FMassEntityView::FMassEntityView(const FMassEntityManager& EntityManager, FMassEntityHandle InEntity)
{
	Entity = InEntity;
	const FMassArchetypeHandle ArchetypeHandle = EntityManager.GetArchetypeForEntity(Entity);
	Archetype = &FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	EntityDataHandle = Archetype->MakeEntityHandle(Entity);
}

FMassEntityView FMassEntityView::TryMakeView(const FMassEntityManager& EntityManager, FMassEntityHandle InEntity)
{
	const FMassArchetypeHandle ArchetypeHandle = EntityManager.GetArchetypeForEntity(InEntity);
	return ArchetypeHandle.IsValid() ? FMassEntityView(ArchetypeHandle, InEntity) : FMassEntityView();
}

void* FMassEntityView::GetFragmentPtr(const UScriptStruct& FragmentType) const
{
	checkSlow(EntityDataHandle.IsValid(Archetype));

	if (testableEnsureMsgf(Archetype, TEXT("%hs: Trying to access fragments while no archetype set"), __FUNCTION__))
	{
		CA_ASSUME(Archetype);
		if (const int32* FragmentIndex = Archetype->GetFragmentIndex(&FragmentType))
		{
			// failing the below Find means given entity's archetype is missing given FragmentType
			return Archetype->GetFragmentData(*FragmentIndex, EntityDataHandle);
		}
	}
	return nullptr;
}

void* FMassEntityView::GetFragmentPtrChecked(const UScriptStruct& FragmentType) const
{
	checkSlow(EntityDataHandle.IsValid(Archetype));

	if (testableEnsureMsgf(Archetype, TEXT("%hs: Trying to access fragments while no archetype set"), __FUNCTION__))
	{
		CA_ASSUME(Archetype);
		const int32 FragmentIndex = Archetype->GetFragmentIndexChecked(&FragmentType);
		return Archetype->GetFragmentData(FragmentIndex, EntityDataHandle);
	}
	return nullptr;
}

FStructView FMassEntityView::GetSparseFragmentPtr(const TNotNull<const UScriptStruct*> FragmentType)
{
	checkSlow(EntityDataHandle.IsValid(Archetype));

	if (testableEnsureMsgf(Archetype, TEXT("%hs: Trying to access sparse fragments while no archetype set"), __FUNCTION__))
	{
		CA_ASSUME(Archetype);
		return Archetype->GetMutableSparseElementDataForEntity(FragmentType, Entity);
	}
	return {};
}

FConstStructView FMassEntityView::GetSparseFragmentPtr(const TNotNull<const UScriptStruct*> FragmentType) const
{
	checkSlow(EntityDataHandle.IsValid(Archetype));

	if (testableEnsureMsgf(Archetype, TEXT("%hs: Trying to access sparse fragments while no archetype set"), __FUNCTION__))
	{
		CA_ASSUME(Archetype);
		return Archetype->GetSparseElementDataForEntity(FragmentType, Entity);
	}
	return {};
}

const void* FMassEntityView::GetConstSharedFragmentPtr(const UScriptStruct& FragmentType) const
{
	const FConstSharedStruct* SharedFragment = Archetype->GetSharedFragmentValues(Entity).GetConstSharedFragments().FindByPredicate(FStructTypeEqualOperator(&FragmentType));
	return (SharedFragment != nullptr) ? SharedFragment->GetMemory() : nullptr;
}

const void* FMassEntityView::GetConstSharedFragmentPtrChecked(const UScriptStruct& FragmentType) const
{
	const FConstSharedStruct* SharedFragment = Archetype->GetSharedFragmentValues(Entity).GetConstSharedFragments().FindByPredicate(FStructTypeEqualOperator(&FragmentType));
	check(SharedFragment != nullptr);
	return SharedFragment->GetMemory();
}

void* FMassEntityView::GetSharedFragmentPtr(const UScriptStruct& FragmentType) const
{
	const FSharedStruct* SharedFragment = Archetype->GetSharedFragmentValues(Entity).GetSharedFragments().FindByPredicate(FStructTypeEqualOperator(&FragmentType));
	return (SharedFragment != nullptr) ? SharedFragment->GetMemory() : nullptr;
}

void* FMassEntityView::GetSharedFragmentPtrChecked(const UScriptStruct& FragmentType) const
{
	const FSharedStruct* SharedFragment = Archetype->GetSharedFragmentValues(Entity).GetSharedFragments().FindByPredicate(FStructTypeEqualOperator(&FragmentType));
	check(SharedFragment != nullptr);
	return SharedFragment->GetMemory();
}

bool FMassEntityView::HasTag(TNotNull<const UScriptStruct*> TagType) const
{
	checkSlow(EntityDataHandle.IsValid(Archetype));
	return Archetype && (UE::Mass::IsSparse(TagType) ? Archetype->HasSparseElementForEntity(TagType, EntityDataHandle) : Archetype->HasTagType(TagType));
}

bool FMassEntityView::HasFragment(TNotNull<const UScriptStruct*> FragmentType) const
{
	checkSlow(EntityDataHandle.IsValid(Archetype));
	return Archetype && (UE::Mass::IsSparse(FragmentType) ? Archetype->HasSparseElementForEntity(FragmentType, EntityDataHandle) : Archetype->HasFragmentType(FragmentType));
}

bool FMassEntityView::HasElement(TNotNull<const UScriptStruct*> ElementType, const UE::Mass::EIncludeSparseElements IncludeSparseElements) const
{
	checkSlow(EntityDataHandle.IsValid(Archetype));
	return Archetype
		&& (Archetype->HasElement(ElementType) 
			|| (IncludeSparseElements == UE::Mass::EIncludeSparseElements::Yes && Archetype->HasSparseElementForEntity(ElementType, EntityDataHandle)));
}
