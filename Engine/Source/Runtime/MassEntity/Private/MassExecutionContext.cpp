// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassExecutionContext.h"
#include "MassArchetypeData.h"
#include "MassEntityManager.h"
#include "MassEntityLinkFragments.h"
#include "MassSubsystemAccess.h"
#include "MassProcessor.h"
#if WITH_MASSENTITY_DEBUG
#include "MassDebugger.h"
#endif // WITH_MASSENTITY_DEBUG
#include "Mass/TestableEnsures.h"

//-----------------------------------------------------------------------------
// FMassExecutionContext
//-----------------------------------------------------------------------------
FMassExecutionContext::FMassExecutionContext(FMassEntityManager& InEntityManager, const float InDeltaTimeSeconds, const bool bInFlushDeferredCommands)
	: SubsystemAccessPtr(MakeUnique<FMassSubsystemAccess>(&InEntityManager))
	, DeltaTimeSeconds(InDeltaTimeSeconds)
	, EntityManager(InEntityManager.AsShared())
	, bFlushDeferredCommands(bInFlushDeferredCommands)
{
}

FMassExecutionContext::FMassExecutionContext(const FMassExecutionContext& Other)
	: EntityManager(Other.EntityManager)
{
	// Delegate to operator= so copy logic lives in one place; operator= intentionally
	// excludes QueriesStack and processing callbacks (see comment in operator= body).
	*this = Other;
}

FMassExecutionContext::FMassExecutionContext(const FMassExecutionContext& Other, FMassEntityQuery& Query, const TSharedPtr<FMassCommandBuffer>& InCommandBuffer)
	: FMassExecutionContext(Other)
{
	ensureMsgf(Other.QueriesStack.Last().Query == &Query, TEXT("Creating a single-query execution context but the Query doesn't match the source context."));
	QueriesStack.Add(Other.QueriesStack.Last());
	SetDeferredCommandBuffer(InCommandBuffer);
}

// Hand-written because TUniquePtr<FMassSubsystemAccess> is not copyable (need deep copy via
// MakeUnique), and because we intentionally skip QueriesStack and processing callbacks — callers
// manage those explicitly after copy. Every data member of FMassExecutionContext must appear here;
// if you add a member to the struct, add it to this operator too.
// @todo MassCore migration: once FMassSubsystemAccess no longer needs to be pimpl'd behind
// TUniquePtr (i.e. once the MassSubsystemAccess.h include can be removed from this header),
// revert to an inline member and = default this operator.
FMassExecutionContext& FMassExecutionContext::operator=(const FMassExecutionContext& Other)
{
	if (this != &Other)
	{
		LinkedEntityView = Other.LinkedEntityView;
		CachedIndirectEntityView = Other.CachedIndirectEntityView;
		SubsystemAccessPtr = MakeUnique<FMassSubsystemAccess>(*Other.SubsystemAccessPtr);
		DeferredCommandBuffer = Other.DeferredCommandBuffer;
		EntityListView = Other.EntityListView;
		EntityCollection = Other.EntityCollection;
		AuxData = Other.AuxData;
		DeltaTimeSeconds = Other.DeltaTimeSeconds;
		CurrentArchetypeCompositionBitSet = Other.CurrentArchetypeCompositionBitSet;
#if WITH_MASSENTITY_DEBUG
		DebugColor = Other.DebugColor;
#endif
		EntityManager = Other.EntityManager;
		// Note: QueriesStack is intentionally not copied — callers manage it explicitly.
		// Note: PreQueryCallbackFunc, PostQueryCallbackFunc, and ChunkQueueFunc are intentionally
		// not copied — they are set by the processing queue on the copy after construction
		// (see FProcessorWorker::ExecuteProcessor in MassProcessingQueueTypes.cpp).
		IteratorSerialNumberGenerator = Other.IteratorSerialNumberGenerator;
		ExecutionType = Other.ExecutionType;
		bFlushDeferredCommands = Other.bFlushDeferredCommands;
		bHasSparseRequirements = Other.bHasSparseRequirements;
		ArchetypeDataPtr = Other.ArchetypeDataPtr;
		ChunkEntityIndex = Other.ChunkEntityIndex;
#if WITH_MASSENTITY_DEBUG
		DebugExecutionDescription = Other.DebugExecutionDescription;
		DebugProcessor = Other.DebugProcessor;
#endif
		ActiveEntityCreationRequirements = Other.ActiveEntityCreationRequirements;
		FragmentViews = Other.FragmentViews;
		ChunkFragmentViews = Other.ChunkFragmentViews;
		ConstSharedFragmentViews = Other.ConstSharedFragmentViews;
		SharedFragmentViews = Other.SharedFragmentViews;
		RequestedSparseElementsAccess = Other.RequestedSparseElementsAccess;
	}
	return *this;
}

FMassExecutionContext::FMassExecutionContext(FMassExecutionContext&& Other) = default;

FMassExecutionContext::~FMassExecutionContext()
{
	ensureMsgf(QueriesStack.Num() == 0, TEXT("Destroying a FMassExecutionContext instance while not all queries have been popped is unexpected."));
}

void FMassExecutionContext::FlushDeferred()
{
	if (bFlushDeferredCommands && DeferredCommandBuffer)
	{
		EntityManager->FlushCommands(DeferredCommandBuffer);
	}
}

void FMassExecutionContext::ClearExecutionData()
{
	FragmentViews.Reset();
	ChunkFragmentViews.Reset();
	ConstSharedFragmentViews.Reset();
	SharedFragmentViews.Reset();
	RequestedSparseElementsAccess.Reset();
	CurrentArchetypeCompositionBitSet.Reset();
	EntityListView = {};
	ChunkEntityIndex = {};
	ArchetypeDataPtr = nullptr;
	bHasSparseRequirements = false;
#if WITH_MASSENTITY_DEBUG
	DebugColor = FColor();
#endif // WITH_MASSENTITY_DEBUG
}

void FMassExecutionContext::SetCurrentArchetype(FMassArchetypeData& Archetype)
{
	CurrentArchetypeCompositionBitSet = Archetype.GetCompositionBitSet();
	ArchetypeDataPtr = &Archetype;
}

bool FMassExecutionContext::CacheSubsystemRequirements(const FMassSubsystemRequirements& SubsystemRequirements)
{
	return SubsystemAccessPtr->CacheSubsystemRequirements(SubsystemRequirements);
}

void FMassExecutionContext::SetEntityCollection(const FMassArchetypeEntityCollection& InEntityCollection)
{
	check(EntityCollection.IsEmpty());
	EntityCollection = InEntityCollection;
}

void FMassExecutionContext::SetEntityCollection(FMassArchetypeEntityCollection&& InEntityCollection)
{
	check(EntityCollection.IsEmpty());
	check(InEntityCollection.IsUpToDate());
	EntityCollection = MoveTemp(InEntityCollection);
}

void FMassExecutionContext::SetFragmentRequirements(const FMassFragmentRequirements& FragmentRequirements)
{
	FragmentViews.Reset();
	for (const FMassFragmentRequirementDescription& Requirement : FragmentRequirements.GetFragmentRequirements())
	{
		if (Requirement.RequiresBinding())
		{
			FragmentViews.Emplace(Requirement);
		}
	}

	ChunkFragmentViews.Reset();
	for (const FMassFragmentRequirementDescription& Requirement : FragmentRequirements.GetChunkFragmentRequirements())
	{
		if (Requirement.RequiresBinding())
		{
			ChunkFragmentViews.Emplace(Requirement);
		}
	}

	ConstSharedFragmentViews.Reset();
	for (const FMassFragmentRequirementDescription& Requirement : FragmentRequirements.GetConstSharedFragmentRequirements())
	{
		if (Requirement.RequiresBinding())
		{
			ConstSharedFragmentViews.Emplace(Requirement);
		}
	}

	SharedFragmentViews.Reset();
	for (const FMassFragmentRequirementDescription& Requirement : FragmentRequirements.GetSharedFragmentRequirements())
	{
		if (Requirement.RequiresBinding())
		{
			SharedFragmentViews.Emplace(Requirement);
		}
	}

	RequestedSparseElementsAccess.Reset();
	bHasSparseRequirements = FragmentRequirements.HasSparseRequirements();
	if (bHasSparseRequirements)
	{
		RequestedSparseElementsAccess = FragmentRequirements.GetRequiredAllSparseElements()
			+ FragmentRequirements.GetRequiredAnySparseElements()
			+ FragmentRequirements.GetRequiredOptionalSparseElements();
	}
}

UWorld* FMassExecutionContext::GetWorld() const
{ 
	return EntityManager->GetWorld(); 
}

void FMassExecutionContext::BindLinkedEntity()
{
	check(QueriesStack.Num());
	if (QueriesStack.Last().Query->HasLinkedEntityRequirements())
	{
		// find an indirect entity handle to bind to the query
		for (FConstSharedFragmentView& SharedFragmentView : ConstSharedFragmentViews)
		{
			const UScriptStruct* ScriptStruct = SharedFragmentView.FragmentView.GetScriptStruct();
			if (ScriptStruct && ScriptStruct->IsChildOf(FMassEntityLinkFragment::StaticStruct()))
			{
				const FMassEntityLinkFragment& IndirectEntityFragment =
					UE::StructUtils::GetStructRef<FMassEntityLinkFragment>(ScriptStruct, SharedFragmentView.FragmentView.GetMemory());

				BindLinkedEntity(IndirectEntityFragment);
				break;
			}
		}
		checkf(LinkedEntityView.IsSet(), TEXT("Linked Entity access requirement specified but no const shared LinkedEntityFragment required"));
	}
}

void FMassExecutionContext::BindLinkedEntity(const FMassEntityLinkFragment& EntityLinkFragment)
{
	check(QueriesStack.Num());
	checkfSlow(
		ConstSharedFragmentViews.ContainsByPredicate(
			[&EntityLinkFragment](const FConstSharedFragmentView& Element)
			{
				const void* ElemMem = Element.FragmentView.GetMemory();
				return ElemMem == static_cast<const void*>(&EntityLinkFragment);
			})
		, TEXT("EntityLinkFragment must be included in the const shared fragments of the current query"));

	LinkedEntityView = FMassEntityView(EntityManager.Get(), EntityLinkFragment.LinkedEntityHandle);
}

void FMassExecutionContext::PushQuery(FMassEntityQuery& InQuery)
{
	FQueryTransientRuntime& RuntimeData = QueriesStack.Add_GetRef({&InQuery});
	GetSubsystemRequirementBits(RuntimeData.ConstSubsystemsBitSet, RuntimeData.MutableSubsystemsBitSet);

#if WITH_MASSENTITY_DEBUG
	// check if this could possibly trigger a break before iterating to avoid extraneous breakpoint checks
	FMassEntityManager& EntityManagerRef = GetEntityManagerChecked();

	RuntimeData.bCheckProcessorBreaks = FMassDebugger::HasAnyProcessorBreakpoints(EntityManagerRef, DebugGetProcessor());
	
	if (UNLIKELY(FMassDebugger::HasAnyFragmentWriteBreakpoints(EntityManagerRef)))
	{
		auto CheckFragmentRequirement = [&RuntimeData, &EntityManagerRef](TConstArrayView<FMassFragmentRequirementDescription> Requirements) -> void
			{
				for (const FMassFragmentRequirementDescription& Req : Requirements)
				{
					if (Req.AccessMode == EMassFragmentAccess::ReadWrite &&
						FMassDebugger::HasAnyFragmentWriteBreakpoints(EntityManagerRef, Req.StructType))
					{
						if (ensureMsgf(RuntimeData.BreakFragmentsCount < FQueryTransientRuntime::MaxFragmentBreakpointCount, 
							TEXT("Fragment write breakpoint count limit exceeded for this query.")))
						{
							RuntimeData.FragmentTypesToBreakOn[RuntimeData.BreakFragmentsCount++] = Req.StructType;
						}
					}
				}
			};

		// don't need to check ConstSharedFragmentRequirements because those can't write
		CheckFragmentRequirement(InQuery.GetFragmentRequirements());
		CheckFragmentRequirement(InQuery.GetChunkFragmentRequirements());
		CheckFragmentRequirement(InQuery.GetSharedFragmentRequirements());
	}
#endif // WITH_MASSENTITY_DEBUG

	RuntimeData.PreviousEntityCreationRequirements = ActiveEntityCreationRequirements;
	ActiveEntityCreationRequirements = &InQuery.GetEntityCreationRequirements();

	RuntimeData.IteratorSerialNumber = ++IteratorSerialNumberGenerator;
}

void FMassExecutionContext::PopQuery(const FMassEntityQuery& InQuery)
{
	const FQueryTransientRuntime& RuntimeData = QueriesStack.Last();
	checkf(&InQuery == RuntimeData.Query, TEXT("Queries are stored in a stack and as such it requires elements to be added in LIFO order"));

	SetSubsystemRequirementBits(RuntimeData.ConstSubsystemsBitSet, RuntimeData.MutableSubsystemsBitSet);

	ActiveEntityCreationRequirements = RuntimeData.PreviousEntityCreationRequirements;

	QueriesStack.RemoveAt(QueriesStack.Num() - 1, EAllowShrinking::No);
}

FMassExecutionContext::FEntityIterator FMassExecutionContext::CreateEntityIterator()
{
	if (!testableEnsureMsgf(QueriesStack.Num(), TEXT("Attempting to create an Entity Iterator when no entity query is being executed.")))
	{
		return FEntityIterator(*this);
	}

	return FEntityIterator(*this, QueriesStack.Last());
}

FMassExecutionContext::FSparseEntityIterator FMassExecutionContext::CreateSparseEntityIterator()
{
	testableCheckfReturn(QueriesStack.Num()
		, return {}
		, TEXT("Attempting to create a Sparse Entity Iterator when no entity query is being executed."));
	return FSparseEntityIterator(FEntityIterator(*this, QueriesStack.Last()));
}

FMassEntityHandle FMassExecutionContext::CreateEntity(const FMassArchetypeHandle& ArchetypeHandle,
	const FMassArchetypeSharedFragmentValues& SharedValues)
{
	checkf(ActiveEntityCreationRequirements != nullptr, TEXT("Attempting to create an entity without declaring FMassEntityCreationRequirements"));
#if WITH_MASSENTITY_DEBUG
	// this check is a bit slow and very unlikely to fail for anyone other than the programmer implementing the processors calling this
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	checkf(ActiveEntityCreationRequirements->Contains(ArchetypeData.GetCompositionBitSet()), TEXT("Declared FMassEntityCreationRequirements do not allow creating entities of the requested archetype"));
#endif
	FMassEntityHandle EntityHandle = EntityManager->CreateEntityAsyncUnsafe(ArchetypeHandle, SharedValues);

	return EntityHandle;
}

void FMassExecutionContext::BuildEntity(const FMassArchetypeHandle& ArchetypeHandle, const FMassEntityHandle& EntityHandle, const FMassArchetypeSharedFragmentValues& SharedValues)
{
	checkf(ActiveEntityCreationRequirements != nullptr, TEXT("Attempting to create an entity without declaring EntityCreationRequirements"));
#if WITH_MASSENTITY_DEBUG
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	checkf(ActiveEntityCreationRequirements->Contains(ArchetypeData.GetCompositionBitSet()), TEXT("Declared FMassEntityCreationRequirements do not allow creating entities of the requested archetype"));
#endif
	EntityManager->BuildEntityAsyncUnsafe(ArchetypeHandle, EntityHandle, SharedValues);
}

void FMassExecutionContext::BatchCreateEntities(const FMassArchetypeHandle& ArchetypeHandle
	, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, const int32 Count, TArray<FMassEntityHandle>& InOutEntities)
{
	checkf(ActiveEntityCreationRequirements != nullptr, TEXT("Attempting to create an entity without declaring FMassEntityCreationRequirements"));
#if WITH_MASSENTITY_DEBUG
	const FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle);
	checkf(ActiveEntityCreationRequirements->Contains(ArchetypeData.GetCompositionBitSet()), TEXT("Declared FMassEntityCreationRequirements do not allow creating entities of the requested archetype"));
#endif

	EntityManager->BatchCreateEntitiesAsyncUnsafe(ArchetypeHandle, SharedFragmentValues, Count, InOutEntities);
}

FMassExecutionContext& FMassExecutionContext::GetDummyInstance()
{
	static FMassExecutionContext DummyContext(*TSharedRef<FMassEntityManager>(MakeShareable(new FMassEntityManager())));
	return DummyContext;
}

bool FMassExecutionContext::SetCurrentChunk(const int32 ChunkIndex, const FMassArchetypeChunk& Chunk)
{
	if (bHasSparseRequirements == false || ExecuteSparseChunkFilterImpl(Chunk))
	{
		ChunkEntityIndex = FMassEntityInChunkDataHandle(Chunk.GetRawMemory(), 0, ChunkIndex, Chunk.GetSerialModificationNumber());
		return true;
	}
	ChunkEntityIndex = {};
	return false;
}

FStructView FMassExecutionContext::GetSparseElementInternal(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle) const
{
	return EntityManager->GetMutableSparseElementDataForEntity(ElementType, EntityHandle);
}

bool FMassExecutionContext::HasSparseElement(TNotNull<const UScriptStruct*> ElementType, const FEntityIterator& EntityIterator) const
{
	return ArchetypeDataPtr && ArchetypeDataPtr->HasSparseElementForEntity(ElementType, EntityListView[EntityIterator]);
}

bool FMassExecutionContext::ExecuteSparseElementsFilterImpl(const FSparseEntityIterator& EntityIt) const
{
	// @todo we could gain some perf if we cached required sparse elements bitsets as member variables. Needs perf measuring if worth it.
	checkSlow(ArchetypeDataPtr);
	const FMassEntityInChunkDataHandle EntityInChunkHandle(ChunkEntityIndex, EntityIt);
	return ArchetypeDataPtr->DoesEntityMatchSparseComposition(EntityInChunkHandle
		, QueriesStack.Last().Query->GetRequiredAllSparseElements()
		, QueriesStack.Last().Query->GetRequiredAnySparseElements()
		, QueriesStack.Last().Query->GetRequiredNoneSparseElements());
}

inline bool FMassExecutionContext::ExecuteSparseChunkFilterImpl(const FMassArchetypeChunk& Chunk) const
{
	return Chunk.DoesMatchComposition(QueriesStack.Last().Query->GetRequiredAllSparseElements(), QueriesStack.Last().Query->GetRequiredAnySparseElements());
}

//-----------------------------------------------------------------------------
// FMassExecutionContext::FQueryTransientRuntime
//-----------------------------------------------------------------------------
FMassExecutionContext::FQueryTransientRuntime& FMassExecutionContext::FQueryTransientRuntime::GetDummyInstance()
{
	static FMassEntityQuery DummyQuery;
	static FQueryTransientRuntime DummyInstance = { &DummyQuery };
	return DummyInstance;
}

//-----------------------------------------------------------------------------
// FMassExecutionContext::FEntityIterator
//-----------------------------------------------------------------------------
FMassExecutionContext::FEntityIterator::FEntityIterator()
	: ExecutionContext(FMassExecutionContext::GetDummyInstance())
	, QueryRuntime(FQueryTransientRuntime::GetDummyInstance())
{
	
}

FMassExecutionContext::FEntityIterator::FEntityIterator(FMassExecutionContext& InExecutionContext)
	: ExecutionContext(InExecutionContext)
	, QueryRuntime(FQueryTransientRuntime::GetDummyInstance())
{
	
}

FMassExecutionContext::FEntityIterator::FEntityIterator(FMassExecutionContext& InExecutionContext, FQueryTransientRuntime& InQueryRuntime)
	: ExecutionContext(InExecutionContext)
	, QueryRuntime(InQueryRuntime)
	, NumEntities(InExecutionContext.GetNumEntities())
	, SerialNumber(InQueryRuntime.IteratorSerialNumber)
{
	this->operator++();
}

#if WITH_MASSENTITY_DEBUG

UE_DISABLE_OPTIMIZATION_SHIP
void FMassExecutionContext::FEntityIterator::TestBreakpoints()
{
	FMassEntityManager& EntityManagerRef = ExecutionContext.GetEntityManagerChecked();
	FMassEntityHandle Entity = ExecutionContext.GetEntity(EntityIndex);
	if (QueryRuntime.bCheckProcessorBreaks)
	{
		if (UE::Mass::Debug::FBreakpointHandle BreakHandle = FMassDebugger::ShouldProcessorBreak(EntityManagerRef, ExecutionContext.DebugGetProcessor(), Entity))
		{
			bool bDisableThisBreakpoint = false;
			//====================================================================
			//= A breakpoint for this entity set in the MassDebugger has triggered
			//= Step out of this function to debug the actual code being run for the entity
			//=
			//= To disable this specific breakpoint use the Watch window to set
			//= bDisableThisBreakpoint to `true` or 1
			//====================================================================
			UE_DEBUG_BREAK();

			if (bDisableThisBreakpoint)
			{
				FMassDebugger::SetBreakpointEnabled(BreakHandle, false);
			}

			// bailing out, no point to hit multiple breakpoints for the given entity/processor pair
			return;
		}
	}

	for (const UScriptStruct* Fragment : QueryRuntime.FragmentTypesToBreakOn)
	{
		if (UE::Mass::Debug::FBreakpointHandle BreakHandle = FMassDebugger::ShouldBreakOnFragmentWrite(EntityManagerRef, Fragment, Entity))
		{
			bool bDisableThisBreakpoint = false;
			//====================================================================
			//= A breakpoint for this entity set in the MassDebugger has triggered
			//= Step out of this function to debug the actual code being run for the entity
			// 
			//= To disable this specific breakpoint use the Watch window to set
			//= bDisableThisBreakpoint to `true` or 1
			//====================================================================
			UE_DEBUG_BREAK();

			if (bDisableThisBreakpoint)
			{
				FMassDebugger::SetBreakpointEnabled(BreakHandle, false);
			}

			// bailing out, no point to hit multiple breakpoints for the given entity/processor pair
			return;
		}
	}
}
UE_ENABLE_OPTIMIZATION_SHIP

#endif // WITH_MASSENTITY_DEBUG