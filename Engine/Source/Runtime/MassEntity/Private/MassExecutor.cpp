// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassExecutor.h"
#include "MassProcessingTypes.h"
#include "MassProcessor.h"
#include "MassCommandBuffer.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

namespace UE::Mass::Executor
{

FORCEINLINE void ExecuteProcessors(FMassEntityManager& EntityManager, TArrayView<UMassProcessor* const> Processors, FMassExecutionContext& ExecutionContext)
{
	for (UMassProcessor* Proc : Processors)
	{
		if (LIKELY(Proc->IsActive()))
		{
			Proc->CallExecute(EntityManager, ExecutionContext);
		}
	}
}

void Run(FMassRuntimePipeline& RuntimePipeline, FProcessingContext& ProcessingContext)
{
	if (!ensure(ProcessingContext.GetDeltaSeconds() >= 0.f) 
		|| !ensure(RuntimePipeline.GetProcessors().Find(nullptr) == INDEX_NONE))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE_STR("MassExecutor Run Pipeline")
	RunProcessorsView(RuntimePipeline.GetMutableProcessors(), ProcessingContext);
}

void RunSparseEntities(FMassRuntimePipeline& RuntimePipeline, FProcessingContext& ProcessingContext, FMassArchetypeHandle Archetype, TConstArrayView<FMassEntityHandle> Entities)
{
	if (!ensure(RuntimePipeline.GetProcessors().Find(nullptr) == INDEX_NONE)
		|| RuntimePipeline.Num() == 0
		|| !ensureMsgf(Archetype.IsValid(), TEXT("The Archetype passed in to UE::Mass::Executor::RunSparseEntities is invalid")))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE_STR("MassExecutor RunSparseEntities");

	const FMassArchetypeEntityCollection EntityCollection(Archetype, Entities, FMassArchetypeEntityCollection::NoDuplicates);
	RunProcessorsView(RuntimePipeline.GetMutableProcessors(), ProcessingContext, MakeArrayView(&EntityCollection, 1));
}

void RunWithCollection(FMassRuntimePipeline& RuntimePipeline, FProcessingContext& ProcessingContext, const FMassArchetypeEntityCollection& EntityCollection)
{
	if (!ensure(RuntimePipeline.GetProcessors().Find(nullptr) == INDEX_NONE)
		|| RuntimePipeline.Num() == 0
		|| !ensureMsgf(EntityCollection.GetArchetype().IsValid(), TEXT("The Archetype of EntityCollection passed in to UE::Mass::Executor::RunWithCollection is invalid")))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE_STR("MassExecutor RunWithCollection");

	RunProcessorsView(RuntimePipeline.GetMutableProcessors(), ProcessingContext, MakeArrayView(&EntityCollection, 1));
}

void Run(UMassProcessor& Processor, FProcessingContext& ProcessingContext)
{
	if (!ensure(ProcessingContext.GetDeltaSeconds() >= 0.f))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE_STR("MassExecutor Run")

	UMassProcessor* ProcPtr = &Processor;
	RunProcessorsView(MakeArrayView(&ProcPtr, 1), ProcessingContext);
}

void RunProcessorsView(TArrayView<UMassProcessor* const> Processors, FProcessingContext& ProcessingContext, TConstArrayView<FMassArchetypeEntityCollection> EntityCollections)
{
#if WITH_MASSENTITY_DEBUG
	if (Processors.Find(nullptr) != INDEX_NONE)
	{
		UE_LOGF(LogMass, Error, "%ls input Processors contains nullptr. Bailing out.", ANSI_TO_TCHAR(__FUNCTION__));
		return;
	}
#endif // WITH_MASSENTITY_DEBUG

	TRACE_CPUPROFILER_EVENT_SCOPE(MassExecutor_RunProcessorsView);

	FMassExecutionContext& ExecutionContext = ProcessingContext.GetExecutionContext();
	FMassEntityManager& EntityManager = *ProcessingContext.GetEntityManager();
	FMassEntityManager::FScopedProcessing ProcessingScope = EntityManager.NewProcessingScope();

	if (EntityCollections.Num() == 0)
	{
		ExecuteProcessors(EntityManager, Processors, ExecutionContext);
	}
	else
	{
		for (const FMassArchetypeEntityCollection& Collection : EntityCollections)
		{
			ExecutionContext.SetEntityCollection(Collection);
			ExecuteProcessors(EntityManager, Processors, ExecutionContext);
			ExecutionContext.ClearEntityCollection();
		}
	}
}

struct FMassExecutorDoneTask
{
	FMassExecutorDoneTask(FMassExecutionContext&& InExecutionContext, TFunction<void()> InOnDoneNotification, TFunction<void()> InOnPreCommandFlushNotification, const FString& InDebugName, ENamedThreads::Type InDesiredThread)
		: ExecutionContext(InExecutionContext)
		, OnDoneNotification(InOnDoneNotification)
		, OnPreCommandFlushNotification(InOnPreCommandFlushNotification)
		, DebugName(InDebugName)
		, DesiredThread(InDesiredThread)
	{
	}
	static TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMassExecutorDoneTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return DesiredThread;
	}

	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Flush Deferred Commands Parallel");
		SCOPE_CYCLE_COUNTER(STAT_Mass_Total);

		FMassEntityManager& EntityManagerRef = ExecutionContext.GetEntityManagerChecked();

		if (&ExecutionContext.Defer() != &EntityManagerRef.Defer())
		{
			ExecutionContext.Defer().MoveAppend(EntityManagerRef.Defer());
		}

		UE_LOGF(LogMass, VeryVerbose, "MassExecutor %ls tasks DONE", *DebugName);

		if (OnPreCommandFlushNotification)
		{
			OnPreCommandFlushNotification();
		}
		
		ExecutionContext.SetFlushDeferredCommands(true);
		ExecutionContext.FlushDeferred();
		OnDoneNotification();
	}
private:
	FMassExecutionContext ExecutionContext;
	TFunction<void()> OnDoneNotification;
	TFunction<void()> OnPreCommandFlushNotification;
	FString DebugName;
	ENamedThreads::Type DesiredThread;
};

struct FMassExecutorDispatchTask
{
	FMassExecutorDispatchTask(const FMassExecutionContext& InExecutionContext, const TSharedPtr<FMassEntityManager>& InEntityManager, UMassProcessor& InProcessor
		, TFunction<void()> InOnDoneNotification, TFunction<void()> InOnPreCommandFlushNotification, const FString& InDebugName, ENamedThreads::Type InDesiredEndTaskThread)
		: ExecutionContext(InExecutionContext)
		, OnDoneNotification(InOnDoneNotification)
		, OnPreCommandFlushNotification(InOnPreCommandFlushNotification)
		, DebugName(InDebugName)
		, EntityManager(InEntityManager)
		, Processor(InProcessor)
		, DesiredEndTaskThread(InDesiredEndTaskThread)
	{
	}
	static TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMassExecutorDispatchTask, STATGROUP_TaskGraphTasks);
	}

	static ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::AnyHiPriThreadHiPriTask;
	}
	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}
	void DoTask(ENamedThreads::Type /*CurrentThread*/, const FGraphEventRef& MyCompletionGraphEvent)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Dispatch Processors Async")
		FGraphEventRef CompletionEvent;
		FMassExecutionContext ExecutionContextThread = ExecutionContext;
		CompletionEvent = Processor.DispatchProcessorTasks(EntityManager, ExecutionContextThread, {});
		if (CompletionEvent.IsValid())
		{
			const FGraphEventArray Prerequisites = { CompletionEvent };

			CompletionEvent = TGraphTask<FMassExecutorDoneTask>::CreateTask(&Prerequisites)
				.ConstructAndDispatchWhenReady(MoveTemp(ExecutionContextThread), OnDoneNotification, OnPreCommandFlushNotification, Processor.GetName(), DesiredEndTaskThread);
		}

		if (CompletionEvent.IsValid())
		{
			MyCompletionGraphEvent->DontCompleteUntil(CompletionEvent);
		}
	}
private:
	FMassExecutionContext ExecutionContext;
	TFunction<void()> OnDoneNotification;
	TFunction<void()> OnPreCommandFlushNotification;
	FString DebugName;
	TSharedPtr<FMassEntityManager> EntityManager;
	UMassProcessor& Processor;
	ENamedThreads::Type DesiredEndTaskThread;
};

FGraphEventRef TriggerParallelTasks(UMassProcessor& Processor, FProcessingContext&& ProcessingContext, TFunction<void()> OnDoneNotification, ENamedThreads::Type CurrentThread, TFunction<void()> OnPreCommandFlushNotification)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MassExecutor_RunParallel);

	TSharedPtr<FMassEntityManager> EntityManager = ProcessingContext.EntityManager;

	// We need to transfer ProcessingContext's ExecutionContext - otherwise ProcessingContext's destructor will attempt
	// flushing stored commands. 
	FMassExecutionContext ExecutionContext = MoveTemp(ProcessingContext).GetExecutionContext();

	FGraphEventRef DispatchEvent;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Dispatch Processors")
		FGraphEventArray Prerequisites;
		DispatchEvent  = TGraphTask<FMassExecutorDispatchTask>::CreateTask(&Prerequisites)
			.ConstructAndDispatchWhenReady(ExecutionContext, EntityManager, Processor, OnDoneNotification, OnPreCommandFlushNotification, Processor.GetName(), CurrentThread);
	}

	return DispatchEvent;
}

//-----------------------------------------------------------------------------
// DEPRECATED
//-----------------------------------------------------------------------------
inline FGraphEventRef TriggerParallelTasks(UMassProcessor& Processor, FProcessingContext& ProcessingContext, TFunction<void()> OnDoneNotification
		, ENamedThreads::Type CurrentThread)
{
	FProcessingContext LocalContext = ProcessingContext;
	return TriggerParallelTasks(Processor, MoveTemp(LocalContext), OnDoneNotification, CurrentThread);
}

} // namespace UE::Mass::Executor
