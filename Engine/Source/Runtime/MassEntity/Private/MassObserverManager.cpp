// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassObserverManager.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.h"
#include "MassExecutor.h"
#include "MassProcessingTypes.h"
#include "MassObserverRegistry.h"
#include "MassDebugger.h"
#include "MassEntityCollection.h"
#include "MassProcessingContext.h"
#include "MassObserverNotificationTypes.h"
#include "MassObserverProcessor.h"
#include "VisualLogger/VisualLogger.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MassObserverManager)

namespace UE::Mass::ObserverManager
{
	namespace Tweakables
	{
		// Used as a template parameter for TInlineAllocator that we use when gathering UScriptStruct* of the observed types to process.
		constexpr int InlineAllocatorElementsForOverlapTypes = 8;
	} // namespace Tweakables

	bool bCoalesceBufferedNotifications = false;

	namespace Private
	{
		FAutoConsoleVariableRef ConsoleVariables[] =
		{
			FAutoConsoleVariableRef(TEXT("mass.observers.CoalesceBufferedNotifications"), bCoalesceBufferedNotifications
				, TEXT("If enabled, when buffering new notification we'll check if it's the same type as the previously stored one, and if so then merge the two."), ECVF_Default)
		};

		// a helper function to reduce code duplication in FMassObserverManager::Initialize
		template<typename TBitSet>
		void AddRegisteredObserverProcessorInstances(FMassEntityManager& EntityManager, const EProcessorExecutionFlags WorldExecutionFlags, UObject& Owner
			, const UMassObserverRegistry::FObserverClassesMap& RegisteredObserverTypes, TBitSet& InOutObservedBitSet, FMassObserversMap& Observers
			, TMap<const TSubclassOf<UMassProcessor>, UMassProcessor*>& InOutExistingProcessorsMap)
		{
			for (auto It : RegisteredObserverTypes)
			{
				const UObject* Type = It.Key.ResolveObjectPtr();
				
				if (Type == nullptr || It.Value.Num() == 0)
				{
					continue;
				}

				const UScriptStruct* StructType = CastChecked<const UScriptStruct>(Type);
				TArray<TObjectPtr<UMassProcessor>> ObserverProcessors;

				for (const FSoftClassPath& SoftProcessorClass : It.Value)
				{
					if (TSubclassOf<UMassProcessor> ProcessorClass = SoftProcessorClass.ResolveClass())
					{
						if (UMassProcessor** ExistingProcessor = InOutExistingProcessorsMap.Find(ProcessorClass))
						{
							ObserverProcessors.AddUnique(*ExistingProcessor);
						}
						else if (ProcessorClass->GetDefaultObject<UMassProcessor>()->ShouldExecute(WorldExecutionFlags))
						{
							UMassProcessor* ObserverInstance = NewObject<UMassProcessor>(&Owner, ProcessorClass);
							ObserverProcessors.Add(ObserverInstance);
							InOutExistingProcessorsMap.Add(ProcessorClass, ObserverInstance);
						}
					}
				}

				if (ObserverProcessors.Num() > 0)
				{
					FMassRuntimePipeline& Pipeline = (*Observers).FindOrAdd(StructType);
					Pipeline.AppendProcessors(MoveTemp(ObserverProcessors));
					Pipeline.SortByExecutionPriority();
					Pipeline.Initialize(Owner, EntityManager.AsShared());
					
					InOutObservedBitSet.Add(StructType);
				}
			}
		};
	} // namespace Private

	struct FDeprecationHelper
	{
		/** determines which observed operation type the given HandlersContainerPtr implements */
		static EMassObservedOperation DetermineOperationType(const FMassObserversMap* HandlersContainerPtr, const FMassObserversMap* MapArray)
		{
			if (HandlersContainerPtr == &MapArray[static_cast<uint8>(EMassObservedOperation::AddElement)])
			{
				return EMassObservedOperation::AddElement;
			}
			if (HandlersContainerPtr == &MapArray[static_cast<uint8>(EMassObservedOperation::RemoveElement)])
			{
				return EMassObservedOperation::RemoveElement;
			}
			return EMassObservedOperation::MAX;
		};

		static void HandleSingleElement(TNotNull<FMassObserverManager*> ObserverManager, const UScriptStruct& ElementType, const FMassArchetypeEntityCollection& EntityCollection, const FMassObserversMap& HandlersContainer)
		{
			const EMassObservedOperation Operation = DetermineOperationType(&HandlersContainer, ObserverManager->EntityManager.GetObserverManager().ElementObservers);
			HandleSingleElement(ObserverManager, ElementType, EntityCollection, Operation);
		}

		static void HandleSingleElement(TNotNull<FMassObserverManager*> ObserverManager, const UScriptStruct& ElementType, const FMassArchetypeEntityCollection& EntityCollection, const EMassObservedOperation Operation)
		{
			checkSlow(Operation != EMassObservedOperation::MAX);

			const int32 OperationIndex = static_cast<int32>(Operation);
			const UScriptStruct* ElementTypePtr = &ElementType;

			UE::Mass::FProcessingContext ProcessingContext(ObserverManager->EntityManager);
			FMassObserverManager::HandleElementsImpl(ProcessingContext, MakeArrayView(&EntityCollection, 1)
				, {Operation, MakeArrayView(&ElementTypePtr, 1)}
				, ObserverManager->ElementObservers[OperationIndex]);
		}
	};

	struct FNotificationContext
	{
		FMassObserverManager& ObserverManager;
		FProcessingContext& ProcessingContext;
	};

	struct FBufferedCreationNotificationExecutioner
	{
		explicit FBufferedCreationNotificationExecutioner(FNotificationContext& InNotificationContext)
			: NotificationContext(InNotificationContext)
		{}

		void operator()(const FEntityCollection& Entities) const
		{
			NotificationContext.ObserverManager.OnCollectionsCreatedImpl(NotificationContext.ProcessingContext
				, Entities.GetUpToDatePerArchetypeCollections(NotificationContext.ObserverManager.EntityManager));
		}

		void operator()(const FMassEntityHandle EntityHandle) const
		{
			const FMassArchetypeHandle ArchetypeHandle = NotificationContext.ObserverManager.GetEntityManager().GetArchetypeForEntity(EntityHandle);
			const FMassArchetypeCompositionDescriptor& ArchetypeComposition = NotificationContext.ProcessingContext.GetEntityManager()->GetArchetypeComposition(ArchetypeHandle);
			NotificationContext.ObserverManager.OnCompositionChanged(EntityHandle, ArchetypeComposition, EMassObservedOperation::CreateEntity, &NotificationContext.ProcessingContext);
		}
		FNotificationContext& NotificationContext;
	};

	/** Trivial type with a sole responsibility of initializing and incrementing FMassObserverExecutionContext.CurrentTypeIndex */
	struct FObserverContextIterator
	{
		FObserverContextIterator(FMassObserverExecutionContext& InRuntimeContext)
			: RuntimeContext(InRuntimeContext)
		{
			RuntimeContext.CurrentTypeIndex = 0;
		}

		int32 operator++()
		{
			return ++RuntimeContext.CurrentTypeIndex;
		}

		operator bool() const
		{
			return RuntimeContext.IsValid();
		}

	private:
		FMassObserverExecutionContext& RuntimeContext;
	};
} // namespace UE::Mass::ObserverManager

//----------------------------------------------------------------------//
// FMassObserversMap
//----------------------------------------------------------------------//
void FMassObserversMap::DebugAddUniqueProcessors(TArray<const UMassProcessor*>& OutProcessors) const
{
#if WITH_MASSENTITY_DEBUG
	for (const auto& MapElement : Container)
	{
		for (const UMassProcessor* Processor : MapElement.Value.GetProcessorsView())
		{
			ensure(Processor);
			OutProcessors.AddUnique(Processor);
		}
	}
#endif // WITH_MASSENTITY_DEBUG
}

//----------------------------------------------------------------------//
// FMassObserverManager
//----------------------------------------------------------------------//
FMassArchetypeEntityCollection FMassObserverManager::FCollectionRefOrHandle::DummyCollection;

FMassObserverManager::FMassObserverManager()
	: EntityManager(GetMutableDefault<UMassEntitySubsystem>()->GetMutableEntityManager())
{

}

FMassObserverManager::FMassObserverManager(FMassEntityManager& Owner)
	: EntityManager(Owner)
{

}

void FMassObserverManager::DebugGatherUniqueProcessors(TArray<const UMassProcessor*>& OutProcessors) const
{
#if WITH_MASSENTITY_DEBUG
	for (int32 OperationIndex = 0; OperationIndex < static_cast<int32>(EMassObservedOperation::MAX); ++OperationIndex)
	{
		ElementObservers[OperationIndex].DebugAddUniqueProcessors(OutProcessors);
	}
#endif // WITH_MASSENTITY_DEBUG
}

void FMassObserverManager::Initialize()
{
	// instantiate initializers
	const UMassObserverRegistry& Registry = UMassObserverRegistry::Get();

	UObject* Owner = EntityManager.GetOwner();
	check(Owner);
	const UWorld* World = Owner->GetWorld();
	const EProcessorExecutionFlags WorldExecutionFlags = UE::Mass::Utils::DetermineProcessorExecutionFlags(World);

	TMap<const TSubclassOf<UMassProcessor>, UMassProcessor*> TransientProcessorsMap;
	using UE::Mass::ObserverManager::Private::AddRegisteredObserverProcessorInstances;
	for (int32 i = 0; i < static_cast<int32>(EMassObservedOperation::MAX); ++i)
	{
		AddRegisteredObserverProcessorInstances(EntityManager, WorldExecutionFlags, *Owner, Registry.ElementObserverMaps[i], ObservedElements[i], ElementObservers[i], TransientProcessorsMap);
	}

#if WITH_MASSENTITY_DEBUG
	FMassDebugger::RegisterProcessorDataProvider(TEXT("Observers"), EntityManager.AsShared()
		, [WeakManager = EntityManager.AsWeak()](TArray<const UMassProcessor*>& OutProcessors)
	{
		if (TSharedPtr<FMassEntityManager> SharedEntityManager = WeakManager.Pin())
		{
			FMassObserverManager& ObserverManager = SharedEntityManager->GetObserverManager();
			ObserverManager.DebugGatherUniqueProcessors(OutProcessors);
		}
	});
#endif // WITH_MASSENTITY_DEBUG

	if (!ModulesUnloadedHandle.IsValid())
	{
		ModulesUnloadedHandle = FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.AddRaw(this, &FMassObserverManager::OnModulePackagesUnloaded);
	}
}

void FMassObserverManager::DeInitialize()
{
	if (ModulesUnloadedHandle.IsValid())
	{
		FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.Remove(ModulesUnloadedHandle);
		ModulesUnloadedHandle.Reset();
	}

	for (int32 OperationIndex = 0; OperationIndex < static_cast<int32>(EMassObservedOperation::MAX); ++OperationIndex)
	{
		(*ElementObservers[OperationIndex]).Empty();
		ObservedElements[OperationIndex].Reset();
	}
}

bool FMassObserverManager::OnPostEntitiesCreated(const FMassArchetypeEntityCollection& EntityCollection)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MassObserver_OnPostEntitiesCreated_Collection);

	if (LocksCount > 0)
	{
		TSharedRef<FObserverLock> ObserverLock = ActiveObserverLock.Pin().ToSharedRef();
		ObserverLock->AddCreatedEntitiesCollection(EntityCollection);
		return false;
	}

	const FMassArchetypeCompositionDescriptor& ArchetypeComposition = EntityManager.GetArchetypeComposition(EntityCollection.GetArchetype());
	return OnCompositionChanged(EntityCollection, ArchetypeComposition, EMassObservedOperation::CreateEntity);
}

bool FMassObserverManager::OnPostEntityCreated(FMassEntityHandle EntityHandle, const FMassElementBitSet& InQueriedBitSet)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MassObserver_OnPostEntitiesCreated_Collection);

	if (LocksCount > 0)
	{
		TSharedRef<FObserverLock> ObserverLock = ActiveObserverLock.Pin().ToSharedRef();
		ObserverLock->AddCreatedEntity(EntityHandle);
		return false;
	}

	if (InQueriedBitSet.IsEmpty())
	{
		const FMassArchetypeHandle ArchetypeHandle = EntityManager.GetArchetypeForEntity(EntityHandle);
		const FMassArchetypeCompositionDescriptor& ArchetypeComposition = EntityManager.GetArchetypeComposition(ArchetypeHandle);
		return OnCompositionChanged(EntityHandle, ArchetypeComposition, EMassObservedOperation::CreateEntity);
	}

	return OnCompositionChanged(EntityHandle, InQueriedBitSet, EMassObservedOperation::CreateEntity);
}

bool FMassObserverManager::OnPreEntitiesDestroyed(const FMassArchetypeEntityCollection& EntityCollection)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("OnPreEntitiesDestroyed")
	const FMassArchetypeCompositionDescriptor& ArchetypeComposition = EntityManager.GetArchetypeComposition(EntityCollection.GetArchetype());
	return OnCompositionChanged(EntityCollection, ArchetypeComposition, EMassObservedOperation::DestroyEntity);
}

bool FMassObserverManager::OnPreEntitiesDestroyed(UE::Mass::FProcessingContext& ProcessingContext, const FMassArchetypeEntityCollection& EntityCollection)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("OnPreEntitiesDestroyed")
	const FMassArchetypeCompositionDescriptor& ArchetypeComposition = EntityManager.GetArchetypeComposition(EntityCollection.GetArchetype());	
	return OnCompositionChanged(EntityCollection, ArchetypeComposition, EMassObservedOperation::DestroyEntity, &ProcessingContext);
}

bool FMassObserverManager::OnPreEntityDestroyed(const FMassEntityHandle Entity, const FMassElementBitSet& Composition)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("OnPreEntityDestroyed")
	return OnCompositionChanged(Entity, Composition, EMassObservedOperation::DestroyEntity);
}

bool FMassObserverManager::OnCompositionChanged(FCollectionRefOrHandle&& EntityCollection, const FMassElementBitSet& CompositionDelta
	, const EMassObservedOperation Operation, UE::Mass::FProcessingContext* ProcessingContext)
{
	using UE::Mass::ObserverManager::Tweakables::InlineAllocatorElementsForOverlapTypes;
	ensureMsgf(EntityCollection.EntityHandle.IsValid() || EntityCollection.EntityCollection.IsUpToDate()
		, TEXT("Out-of-date FMassArchetypeEntityCollection used. Stored information is unreliable."));

	if (CompositionDelta.IsEmpty())
	{
		// nothing to do here.
		// @todo calling this function just to bail out would be a lot cheaper if we didn't have to create
		// FMassArchetypeCompositionDescriptor instances first - we usually just pass in tag or fragment bitsets.
		// like in FMassEntityManager::BatchChangeTagsForEntities
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(MassObserver_OnCompositionChanged);

	if (LocksCount > 0)
	{
		if (TSharedPtr<FCreationContext> CreationContext = GetCreationContext())
		{
			// a composition mutating operation is taking place, while creation lock is active - this operation invalidates the stored collections
			CreationContext->MarkDirty();
			return false;
		}
		UE_CVLOG_UELOG(Operation == EMassObservedOperation::RemoveElement || Operation == EMassObservedOperation::DestroyEntity
			, EntityManager.GetOwner(), LogMass, Log
			, TEXT("%hs: Remove operation while observers are locked - the remove-observer will be executed after the data has already been removed."), __FUNCTION__);
	}

	const int32 OperationIndex = static_cast<int32>(Operation);

	FMassElementBitSet OverlapBitSet = ObservedElements[OperationIndex].GetOverlap(CompositionDelta);

	if (OverlapBitSet.IsEmpty() == false)
	{
		if (LocksCount > 0)
		{
			TSharedRef<FObserverLock> ObserverLockRef = ActiveObserverLock.Pin().ToSharedRef();

			// UE::Mass::FEntityCollection(EntityCollection) OR EntityHandle
			if (UE::Mass::ObserverManager::bCoalesceBufferedNotifications)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MassObserver_Notify_Coalesced);

				if (EntityCollection.EntityHandle.IsSet())
				{
					ObserverLockRef->AddNotification(Operation, EntityCollection.EntityHandle, MoveTemp(OverlapBitSet));
				}
				else
				{
					ObserverLockRef->AddNotification(Operation, EntityCollection.EntityCollection, MoveTemp(OverlapBitSet));
				}
			}
			else
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MassObserver_Notify_Emplace);

				FBufferedNotification::FEntitiesContainer Entities;
				if (EntityCollection.EntityHandle.IsSet())
				{
					Entities.Emplace<FMassEntityHandle>(EntityCollection.EntityHandle);
				}
				else
				{
					Entities.Emplace<UE::Mass::FEntityCollection>(EntityCollection.EntityCollection);
				}

				ObserverLockRef->BufferedNotifications.Emplace(Operation, MoveTemp(OverlapBitSet), MoveTemp(Entities));
			}
		}
		else
		{
			auto HandleElements = [&](const FMassArchetypeEntityCollection& Collection)
			{
				TArray<const UScriptStruct*, TInlineAllocator<InlineAllocatorElementsForOverlapTypes>> ObservedTypesOverlap;

				alignas(UE::Mass::FProcessingContext) uint8 LocalContextBuffer[sizeof(UE::Mass::FProcessingContext)];
				UE::Mass::FProcessingContext* LocalProcessingContext = (ProcessingContext == nullptr)
					? new(&LocalContextBuffer) UE::Mass::FProcessingContext(EntityManager, /*DeltaSeconds=*/0.f, /*bFlushCommandBuffer=*/false)
					: ProcessingContext;

				OverlapBitSet.ExportTypes(ObservedTypesOverlap);
				HandleElementsImpl(*LocalProcessingContext, {Collection}, {Operation, ObservedTypesOverlap}, ElementObservers[OperationIndex]);
				
				if (ProcessingContext == nullptr)
				{
					LocalProcessingContext->~FProcessingContext();
				}
			};

			if (EntityCollection.EntityHandle.IsSet())
			{
				const FMassArchetypeHandle ArchetypeHandle = EntityManager.GetArchetypeForEntity(EntityCollection.EntityHandle);
				HandleElements(FMassArchetypeEntityCollection(ArchetypeHandle, EntityCollection.EntityHandle));
			}
			else
			{
				HandleElements(EntityCollection.EntityCollection);
			}

			return true;
		}
	}

	return false;
}

bool FMassObserverManager::OnCollectionsCreatedImpl(UE::Mass::FProcessingContext& ProcessingContext, const TConstArrayView<FMassArchetypeEntityCollection> EntityCollections)
{
	using UE::Mass::ObserverManager::Tweakables::InlineAllocatorElementsForOverlapTypes;

	TRACE_CPUPROFILER_EVENT_SCOPE(MassObserver_OnCollectionsCreatedImpl_Collection);

	check(LocksCount == 0);

	constexpr EMassObservedOperation Operation = EMassObservedOperation::CreateEntity;
	constexpr int32 OperationIndex = static_cast<int32>(Operation);

	FMassElementBitSet OverlapBitSet;

	for (FMassArchetypeEntityCollection Collection : EntityCollections)
	{
		checkfSlow(Collection.IsUpToDate(), TEXT("Out-of-date FMassArchetypeEntityCollection used. Stored information is unreliable."));

		const FMassArchetypeCompositionDescriptor& ArchetypeComposition = EntityManager.GetArchetypeComposition(Collection.GetArchetype());
		OverlapBitSet += ArchetypeComposition.GetElementsBitSet();
	}

	OverlapBitSet = ObservedElements[OperationIndex].GetOverlap(OverlapBitSet);
	const bool bHasOverlap = !OverlapBitSet.IsEmpty();

	if (bHasOverlap)
	{
		TArray<const UScriptStruct*, TInlineAllocator<InlineAllocatorElementsForOverlapTypes>> ObservedTypesOverlap;

		OverlapBitSet.ExportTypes(ObservedTypesOverlap);
		HandleElementsImpl(ProcessingContext, EntityCollections, {Operation, ObservedTypesOverlap}, ElementObservers[OperationIndex]);

		return true;
	}
	return false;
}

void FMassObserverManager::HandleElementsImpl(UE::Mass::FProcessingContext& ProcessingContext, TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
		, FMassObserverExecutionContext&& ObserverContext, FMassObserversMap& HandlersContainer)
{	
	TRACE_CPUPROFILER_EVENT_SCOPE(MassObserver_HandleElementsImpl);

	check(ObserverContext.GetTypesInOperation().Num());
	ensureMsgf(EntityCollections.Num(), TEXT("Empty collections array is unexpected at this point. Nothing bad will happen, but it's a waste of perf."));

	FMassEntityManager::FScopedProcessing ProcessingScope = ProcessingContext.EntityManager->NewProcessingScope();

	// @todo maybe we could make this type configurable, so that project-specific code can extend it? 
	FMassObserverExecutionContext& RuntimeObserverContext = ProcessingContext.AuxData.InitializeAs<FMassObserverExecutionContext>(MoveTemp(ObserverContext));

	using UE::Mass::ObserverManager::Tweakables::InlineAllocatorElementsForOverlapTypes;

	// Prevent the same instance from executing more than once per batch.
	// Multi-type observer processors appear in multiple type pipelines;
	// without this guard Execute() would fire once per matched type.
	// Linear scan is intentional: typical element count is 3-15,
	// well below the crossover point where hashing would win.
	TArray<UMassProcessor*, TInlineAllocator<InlineAllocatorElementsForOverlapTypes>> SeenProcessors;
	TArray<UMassProcessor*, TInlineAllocator<InlineAllocatorElementsForOverlapTypes>> NewProcessors;

	for (UE::Mass::ObserverManager::FObserverContextIterator ContextIterator(RuntimeObserverContext); ContextIterator; ++ContextIterator)
	{
		FMassRuntimePipeline& Pipeline = (*HandlersContainer).FindChecked(RuntimeObserverContext.GetCurrentType());

		NewProcessors.Reset();
		for (UMassProcessor* Proc : Pipeline.GetMutableProcessors())
		{
			if (!SeenProcessors.Contains(Proc))
			{
				NewProcessors.Add(Proc);
				SeenProcessors.Add(Proc);
			}
		}
		if (NewProcessors.Num() > 0)
		{
			UE::Mass::Executor::RunProcessorsView(NewProcessors, ProcessingContext, EntityCollections);
		}
	}
}

void FMassObserverManager::AddObserverInstance(TNotNull<const UScriptStruct*> ElementType, const EMassObservedOperationFlags OperationFlags, TNotNull<UMassProcessor*> ObserverProcessor)
{
	bool bInitializeCalled = false;

	for (uint8 OperationIndex = 0; OperationIndex < static_cast<uint8>(EMassObservedOperation::MAX); ++OperationIndex)
	{
		if (!!(OperationFlags & static_cast<EMassObservedOperationFlags>(1 << OperationIndex)))
		{
			ObservedElements[OperationIndex].Add(ElementType);

			FMassRuntimePipeline& Pipeline = (*ElementObservers[OperationIndex]).FindOrAdd(ElementType);
			if (Pipeline.AppendUniqueProcessor(*ObserverProcessor) && !bInitializeCalled)
			{
				if (UObject* Owner = EntityManager.GetOwner())
				{	
					ObserverProcessor->CallInitialize(Owner, EntityManager.AsShared());
					bInitializeCalled = true;
				}

				Pipeline.SortByExecutionPriority();
			}
		}
	}
}

void FMassObserverManager::AddObserverInstance(const UScriptStruct& ElementType, const EMassObservedOperation Operation, UMassProcessor& ObserverProcessor)
{
	ensureAlways(Operation < EMassObservedOperation::MAX);
	check(Operation != EMassObservedOperation::MAX);

	EMassObservedOperationFlags OperationFlag = (Operation < EMassObservedOperation::MAX)
		? static_cast<EMassObservedOperationFlags>(1 << static_cast<uint8>(Operation))
		: (Operation == EMassObservedOperation::Add
			? EMassObservedOperationFlags::Add
			: EMassObservedOperationFlags::Remove);

	AddObserverInstance(&ElementType, OperationFlag, &ObserverProcessor);
}

void FMassObserverManager::AddObserverInstance(TNotNull<UMassObserverProcessor*> ObserverProcessor)
{
	for (const UScriptStruct* Type : ObserverProcessor->GetObservedTypes())
	{
		AddObserverInstance(Type, ObserverProcessor->GetObservedOperations(), ObserverProcessor);
	}
}

void FMassObserverManager::RemoveObserverInstance(TNotNull<UMassObserverProcessor*> ObserverProcessor)
{
	for (const UScriptStruct* Type : ObserverProcessor->GetObservedTypes())
	{
		RemoveObserverInstance(Type, ObserverProcessor->GetObservedOperations(), ObserverProcessor);
	}
}

void FMassObserverManager::RemoveObserverInstance(TNotNull<const UScriptStruct*> ElementType, const EMassObservedOperationFlags OperationFlags, TNotNull<UMassProcessor*> ObserverProcessor)
{
	for (uint8 OperationIndex = 0; OperationIndex < static_cast<uint8>(EMassObservedOperation::MAX); ++OperationIndex)
	{
		if (!!(OperationFlags & static_cast<EMassObservedOperationFlags>(1 << OperationIndex)))
		{
			FMassRuntimePipeline* Pipeline = (*ElementObservers[OperationIndex]).Find(ElementType);
			if (ensureMsgf(Pipeline, TEXT("Trying to remove an observer for a fragment/tag that does not seem to be observed.")))
			{
				Pipeline->RemoveProcessor(*ObserverProcessor);
				if (Pipeline->Num() == 0)
				{
					(*ElementObservers[OperationIndex]).Remove(ElementType);
					ObservedElements[OperationIndex].Remove(ElementType);
				}
			}
		}
	}
}

void FMassObserverManager::RemoveObserverInstance(const UScriptStruct& ElementType, const EMassObservedOperation Operation, UMassProcessor& ObserverProcessor)
{
	ensureAlways(Operation < EMassObservedOperation::MAX);
	check(Operation != EMassObservedOperation::MAX);

	EMassObservedOperationFlags OperationFlag = (Operation < EMassObservedOperation::MAX)
		? static_cast<EMassObservedOperationFlags>(1 << static_cast<uint8>(Operation))
		: (Operation == EMassObservedOperation::Add
			? EMassObservedOperationFlags::Add
			: EMassObservedOperationFlags::Remove);

	RemoveObserverInstance(&ElementType, OperationFlag, &ObserverProcessor);
}

TSharedRef<FMassObserverManager::FObserverLock> FMassObserverManager::GetOrMakeObserverLock()
{
	if (TSharedPtr<FMassObserverManager::FObserverLock> SharedActiveLock = ActiveObserverLock.Pin())
	{
		return SharedActiveLock.ToSharedRef();
	}
	else
	{
		FObserverLock* ObserverLock = new FObserverLock(*this);
		TSharedRef<FObserverLock> SharedContext = MakeShareable(ObserverLock);
		ActiveObserverLock = SharedContext;
		return SharedContext;
	}
}

TSharedRef<FMassObserverManager::FCreationContext> FMassObserverManager::GetOrMakeCreationContext()
{
	if (ActiveCreationContext.IsValid())
	{
		return ActiveCreationContext.Pin().ToSharedRef();
	}
	else
	{
		FCreationContext* ObserverLock = new FCreationContext(GetOrMakeObserverLock());
#if WITH_MASSENTITY_DEBUG
		ObserverLock->CreationHandle.SerialNumber = LockedNotificationSerialNumber;
#endif // WITH_MASSENTITY_DEBUG
		ObserverLock->CreationHandle.OpIndex = ObserverLock->Lock->AddCreatedEntities({});
		TSharedRef<FCreationContext> SharedContext = MakeShareable(ObserverLock);
		ActiveCreationContext = SharedContext;
		return SharedContext;
	}
}

TSharedRef<FMassObserverManager::FCreationContext> FMassObserverManager::GetOrMakeCreationContext(TConstArrayView<FMassEntityHandle> ReservedEntities
	, FMassArchetypeEntityCollection&& EntityCollection)
{
	if (TSharedPtr<FCreationContext> CreationContext = ActiveCreationContext.Pin())
	{
		CreationContext->GetObserverLock()->AddCreatedEntities(ReservedEntities, Forward<FMassArchetypeEntityCollection>(EntityCollection));
		return CreationContext.ToSharedRef();
	}
	else
	{
		FCreationContext* ObserverLock = new FCreationContext(GetOrMakeObserverLock());
#if WITH_MASSENTITY_DEBUG
		ObserverLock->CreationHandle.SerialNumber = LockedNotificationSerialNumber;
#endif // WITH_MASSENTITY_DEBUG
		ObserverLock->CreationHandle.OpIndex = ObserverLock->Lock->AddCreatedEntities(ReservedEntities, Forward<FMassArchetypeEntityCollection>(EntityCollection));
		TSharedRef<FCreationContext> SharedContext = MakeShareable(ObserverLock);
		ActiveCreationContext = SharedContext;
		return SharedContext;
	}
}

void FMassObserverManager::OnPostFork(EForkProcessRole)
{
	if (TSharedPtr<FObserverLock> ActiveContext = ActiveObserverLock.Pin())
	{
		ActiveContext->ForceUpdateCurrentThreadID();
	}
}

void FMassObserverManager::OnModulePackagesUnloaded(TConstArrayView<UPackage*> Packages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMassObserverManager::OnModulePackagesUnloaded);

	const auto ProcessObserversMap = [&Packages](FMassObserversMap& ObserversMap)
	{
		for (auto MapIt = (*ObserversMap).CreateIterator(); MapIt; ++MapIt)
		{
			TObjectPtr<const UScriptStruct> ObserverClass = MapIt->Key;
			const UPackage* ObserverPackage = ObserverClass->GetPackage();

			if (Packages.Contains(ObserverPackage))
			{
				UE_LOGF(LogMass, Verbose, "%s: removed observer %ls (%ls)", __FUNCTION__, *ObserverClass->GetName(), *ObserverPackage->GetName());
				MapIt.RemoveCurrent();
			}
		}
	};

	for (int32 OperationIndex = 0; OperationIndex < static_cast<int32>(EMassObservedOperation::MAX); ++OperationIndex)
	{
		ProcessObserversMap(ElementObservers[OperationIndex]);
	}
}

void FMassObserverManager::ResumeExecution(FObserverLock& LockBeingReleased)
{
	using namespace UE::Mass::ObserverManager;
	using UE::Mass::ObserverManager::Tweakables::InlineAllocatorElementsForOverlapTypes;

	ensureMsgf(LocksCount == 0, TEXT("We only expect this function to be called if all locks are released."));
#if WITH_MASSENTITY_DEBUG
	ensureMsgf(LockBeingReleased.LockSerialNumber == LockedNotificationSerialNumber
		, TEXT("Lock's and ObserverManager's lock serial numbers are expected to match."));
	++LockedNotificationSerialNumber;
#endif // WITH_MASSENTITY_DEBUG

	if (LockBeingReleased.BufferedNotifications.IsEmpty() == false)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassObserver_ResumeExecution);

		UE::Mass::FProcessingContext ProcessingContext(EntityManager);

		FNotificationContext NotificationContext{*this, ProcessingContext};
		FBufferedCreationNotificationExecutioner CreationOpExecutioner(NotificationContext);
		TArray<const UScriptStruct*, TInlineAllocator<InlineAllocatorElementsForOverlapTypes>> ObservedTypesOverlap;

		for (FBufferedNotification& Op : LockBeingReleased.BufferedNotifications)
		{
			if (Op.Type == EMassObservedOperation::CreateEntity)
			{
				Visit(CreationOpExecutioner, Op.AffectedEntities);
			}
			else
			{
				if (const UE::Mass::FEntityCollection* Entities = Op.AffectedEntities.TryGet<UE::Mass::FEntityCollection>())
				{
					ObservedTypesOverlap.Reset();
					Op.CompositionChange.ExportTypes(ObservedTypesOverlap);
					FMassObserverManager::HandleElementsImpl(NotificationContext.ProcessingContext
						, Entities->GetUpToDatePerArchetypeCollections(NotificationContext.ObserverManager.EntityManager)
						, {Op.Type, ObservedTypesOverlap}
						, NotificationContext.ObserverManager.ElementObservers[static_cast<uint8>(Op.Type)]);
				}
				else 
				{
					const FMassEntityHandle EntityHandle = Op.AffectedEntities.Get<FMassEntityHandle>();
					ObservedTypesOverlap.Reset();
					Op.CompositionChange.ExportTypes(ObservedTypesOverlap);
					
					FMassArchetypeHandle ArchetypeHandle = NotificationContext.ObserverManager.GetEntityManager().GetArchetypeForEntity(EntityHandle);
					FMassArchetypeEntityCollection AsEntityCollection(MoveTemp(ArchetypeHandle), EntityHandle);

					NotificationContext.ObserverManager.HandleElementsImpl(NotificationContext.ProcessingContext
						, MakeArrayView(&AsEntityCollection, 1)
						, {Op.Type, ObservedTypesOverlap}
						, NotificationContext.ObserverManager.ElementObservers[static_cast<uint8>(Op.Type)]);
				}
			}
		}
#if WITH_MASSENTITY_DEBUG
		++DebugNonTrivialResumeExecutionCount;
#endif // WITH_MASSENTITY_DEBUG
	}
}

void FMassObserverManager::ReleaseCreationHandle(FCreationNotificationHandle InCreationNotificationHandle)
{
	ensureMsgf(InCreationNotificationHandle.IsSet(), TEXT("Invalid creation handle passed to %hs"), __FUNCTION__);
#if WITH_MASSENTITY_DEBUG
	ensureMsgf(InCreationNotificationHandle.SerialNumber == LockedNotificationSerialNumber
		, TEXT("Creation handle's serial number doesn't match the ObserverManager's data"));
#endif // WITH_MASSENTITY_DEBUG

	TSharedPtr<FObserverLock> LockInstance = ActiveObserverLock.Pin();
	if (ensure(LockInstance))
	{
		ensure(LockInstance->ReleaseCreationNotification(InCreationNotificationHandle));
		ensure(ActiveCreationContext.IsValid() == false);
	}
}

//----------------------------------------------------------------------//
// DEPRECATED
//----------------------------------------------------------------------//
bool FMassObserverManager::OnPostEntitiesCreated(UE::Mass::FProcessingContext&, TConstArrayView<FMassArchetypeEntityCollection> EntityCollections)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("OnPostEntitiesCreated")

	bool bReturnValue = false;

	for (const FMassArchetypeEntityCollection& Collection : EntityCollections)
	{
		const FMassArchetypeCompositionDescriptor& ArchetypeComposition = EntityManager.GetArchetypeComposition(Collection.GetArchetype());
		bReturnValue |= OnCompositionChanged(Collection, ArchetypeComposition, EMassObservedOperation::CreateEntity);
	}

	return bReturnValue;
}

bool FMassObserverManager::OnCompositionChanged(UE::Mass::FProcessingContext&, const FMassArchetypeEntityCollection& EntityCollection
	, const FMassArchetypeCompositionDescriptor& CompositionDelta, const EMassObservedOperation InOperation)
{
	return OnCompositionChanged(EntityCollection, CompositionDelta, InOperation);
}

void FMassObserverManager::HandleSingleEntityImpl(const UScriptStruct& FragmentType, const FMassArchetypeEntityCollection& EntityCollection, FMassObserversMap& HandlersContainer)
{
	UE::Mass::ObserverManager::FDeprecationHelper::HandleSingleElement(this, FragmentType, EntityCollection, HandlersContainer);
}

void FMassObserverManager::OnPostFragmentOrTagAdded(const UScriptStruct& FragmentOrTagType, const FMassArchetypeEntityCollection& EntityCollection)
{
	UE::Mass::ObserverManager::FDeprecationHelper::HandleSingleElement(this, FragmentOrTagType, EntityCollection, EMassObservedOperation::AddElement);
}

void FMassObserverManager::OnPreFragmentOrTagRemoved(const UScriptStruct& FragmentOrTagType, const FMassArchetypeEntityCollection& EntityCollection)
{
	UE::Mass::ObserverManager::FDeprecationHelper::HandleSingleElement(this, FragmentOrTagType, EntityCollection, EMassObservedOperation::RemoveElement);
}

void FMassObserverManager::OnFragmentOrTagOperation(const UScriptStruct& FragmentOrTagType, const FMassArchetypeEntityCollection& EntityCollection, const EMassObservedOperation Operation)
{
	UE::Mass::ObserverManager::FDeprecationHelper::HandleSingleElement(this, FragmentOrTagType, EntityCollection, Operation);
}

TConstArrayView<FMassArchetypeEntityCollection> FMassObserverManager::FCreationContext::GetEntityCollections() const
{
	return {};
}

int32 FMassObserverManager::FCreationContext::GetSpawnedNum() const
{
	return 0;
}

bool FMassObserverManager::FCreationContext::IsDirty() const
{
	return true;
}

void FMassObserverManager::FCreationContext::AppendEntities(const TConstArrayView<FMassEntityHandle>)
{
}

void FMassObserverManager::FCreationContext::AppendEntities(const TConstArrayView<FMassEntityHandle>, FMassArchetypeEntityCollection&&)
{
}

void FMassObserverManager::HandleElementsImpl(UE::Mass::FProcessingContext& ProcessingContext, TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
		, TArrayView<const UScriptStruct*> ObservedTypes, FMassObserversMap& HandlersContainer)
{
	if (ObservedTypes.Num() == 0 || ObservedTypes[0] == nullptr)
	{
		return;
	}

	// Legacy support: we need to figure out which operation this is
	// and to do that we need to know whether we're handling fragments or tags
	const bool bIsFragment = UE::Mass::IsA<FMassFragment>(ObservedTypes[0]);
	check(bIsFragment || UE::Mass::IsA<FMassTag>(ObservedTypes[0]));

	using UE::Mass::ObserverManager::FDeprecationHelper;
	const EMassObservedOperation Operation = FDeprecationHelper::DetermineOperationType(&HandlersContainer, ProcessingContext.GetEntityManager()->GetObserverManager().ElementObservers);

	checkf(Operation < EMassObservedOperation::MAX, TEXT("Unable to determine the legacy operation type"));

	HandleElementsImpl(ProcessingContext, EntityCollections, {Operation, ObservedTypes}, HandlersContainer);
}

void FMassObserverManager::HandleFragmentsImpl(UE::Mass::FProcessingContext& ProcessingContext, const FMassArchetypeEntityCollection& EntityCollection
		, TArrayView<const UScriptStruct*> ObservedTypes, FMassObserversMap& HandlersContainer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HandleElementsImpl(ProcessingContext, {EntityCollection}, ObservedTypes, HandlersContainer);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool FMassObserverManager::OnCollectionsCreatedImpl(UE::Mass::FProcessingContext& ProcessingContext, TArray<FMassArchetypeEntityCollection>&& EntityCollections)
{
	return OnCollectionsCreatedImpl(ProcessingContext, MakeArrayView(EntityCollections));
}