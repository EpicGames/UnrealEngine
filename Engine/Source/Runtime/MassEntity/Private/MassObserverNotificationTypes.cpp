// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassObserverNotificationTypes.h"
#include "MassObserverManager.h"
#include "MassEntityManager.h"
#include "MassEntityUtils.h"

namespace UE::Mass::ObserverManager
{
	namespace Private
	{
		TSharedRef<FObserverLock> GetDummyObserverLock()
		{
			static TSharedRef<FObserverLock> DummyObserverLock = MakeShareable(new FObserverLock());
			return DummyObserverLock;
		}
	} // namespace Private

	//-----------------------------------------------------------------------------
	// FObserverLock
	//-----------------------------------------------------------------------------
	FObserverLock::FObserverLock(FMassObserverManager& ObserverManager)
		: OwnerThreadId(FPlatformTLS::GetCurrentThreadId())
		, WeakEntityManager(ObserverManager.EntityManager.AsWeak())
	#if WITH_MASSENTITY_DEBUG
		, LockSerialNumber(ObserverManager.LockedNotificationSerialNumber)
	#endif // WITH_MASSENTITY_DEBUG
	{
		++ObserverManager.LocksCount;
	}

	FObserverLock::~FObserverLock()
	{
		TSharedPtr<FMassEntityManager> SharedEntityManager = WeakEntityManager.Pin();
		if (UNLIKELY(!SharedEntityManager))
		{
			return;
		}

		--SharedEntityManager->GetObserverManager().LocksCount;
		checkf(SharedEntityManager->GetObserverManager().LocksCount >= 0
			, TEXT("%hs: the lock count has become unbalanced."), __FUNCTION__);
		SharedEntityManager->GetObserverManager().ResumeExecution(*this);
	}

	void FObserverLock::ForceUpdateCurrentThreadID()
	{
		OwnerThreadId = FPlatformTLS::GetCurrentThreadId();
	}

	//-----------------------------------------------------------------------------
	// FCreationContext
	//-----------------------------------------------------------------------------
	FCreationContext::FCreationContext()
		: Lock(Private::GetDummyObserverLock())
	{	
	}

	FCreationContext::~FCreationContext()
	{
		if (CreationHandle.IsSet())
		{
			if (TSharedPtr<FMassEntityManager> SharedEntityManager = Lock->GetWeakEntityManager().Pin())
			{
				SharedEntityManager->GetObserverManager().ReleaseCreationHandle(CreationHandle);
			}
		}
	}

	TSharedRef<FCreationContext> FCreationContext::DebugCreateDummyCreationContext()
	{
		return MakeShareable(new FCreationContext());
	}

	TArray<FMassArchetypeEntityCollection> FCreationContext::GetEntityCollections(const FMassEntityManager& InEntityManager) const
	{
		TArray<FMassArchetypeEntityCollection> OutCollections;

		// if the creation handle isn't set there are no creation ops we know about
		if (CreationHandle.IsSet())
		{
			UE::TScopeLock ScopeLock(Lock->GetNotificationLock());

			const FBufferedNotification& Notification = Lock->GetCreationNotification(CreationHandle);

			if (const UE::Mass::FEntityCollection* CreatedEntities = Notification.AffectedEntities.TryGet<FEntityCollection>())
			{
				OutCollections.Append(CreatedEntities->GetUpToDatePerArchetypeCollections(InEntityManager));
			}
			else
			{
				const FMassEntityHandle EntityHandle = Notification.AffectedEntities.Get<FMassEntityHandle>();

				UE::Mass::Utils::CreateEntityCollections(InEntityManager, MakeArrayView(&EntityHandle, 1)
					, FMassArchetypeEntityCollection::NoDuplicates
					, OutCollections);
			}
		}

		return OutCollections;
	}

	bool FCreationContext::DebugAreEntityCollectionsUpToDate() const 
	{ 
		if (CreationHandle.IsSet())
		{
			const FBufferedNotification& Notification = Lock->GetCreationNotification(CreationHandle);

			// collections can be not up to date only if we're storing multiple entities (i.e. not a single handle)
			if (const UE::Mass::FEntityCollection* CreatedEntities = Notification.AffectedEntities.TryGet<FEntityCollection>())
			{
				return CreatedEntities->IsUpToDate();
			}
		}

		return true;
	}
} // namespace UE::Mass::ObserverManager
