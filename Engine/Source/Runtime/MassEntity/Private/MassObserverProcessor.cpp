// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassObserverProcessor.h"
#include "MassObserverRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassObserverProcessor)

//----------------------------------------------------------------------//
// UMassObserverProcessor
//----------------------------------------------------------------------//
UMassObserverProcessor::UMassObserverProcessor()
{
	InitObserverDefaults();
}

UMassObserverProcessor::UMassObserverProcessor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	InitObserverDefaults();
}

void UMassObserverProcessor::InitObserverDefaults()
{
	bAutoRegisterWithProcessingPhases = false;
#if WITH_EDITORONLY_DATA
	bCanShowUpInSettings = false;
#endif // WITH_EDITORONLY_DATA
}

void UMassObserverProcessor::PostInitProperties()
{
	Super::PostInitProperties();

	UClass* MyClass = GetClass();
	CA_ASSUME(MyClass);

	if (HasAnyFlags(RF_ClassDefaultObject) && MyClass->HasAnyClassFlags(CLASS_Abstract) == false)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Operation != EMassObservedOperation::MAX)
		{
			ObservedOperations |= (Operation == EMassObservedOperation::Add)
				? EMassObservedOperationFlags::Add
				: EMassObservedOperationFlags::Remove;
			Operation = EMassObservedOperation::MAX;
		}

		// Merge legacy single ObservedType into ObservedTypes array for backward compat
		if (ObservedType != nullptr)
		{
			ObservedTypes.AddUnique(ObservedType);
			ObservedType = nullptr;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (ObservedOperations != EMassObservedOperationFlags::None)
		{
			Register();
		}
		else if (bAutoRegisterWithObserverRegistry)
		{
			UE_LOGF(LogMass, Error, "%s attempting to register %ls while it\'s misconfigured, OperationFlags: %#x"
				, __FUNCTION__, *MyClass->GetName(), EnumToUnderlyingType(ObservedOperations));
		}
	}
}

void UMassObserverProcessor::Register()
{
	if (!bAutoRegisterWithObserverRegistry)
	{
		return;
	}

	// remove nulls
	const int32 NumRemoved = ObservedTypes.RemoveAll([](const TObjectPtr<const UScriptStruct>& Element)
		{
			return Element.Get() == nullptr;
		});
	ensureMsgf(NumRemoved == 0, TEXT("Null entries in ObservedTypes for %s"), *GetClass()->GetName());

	if (ObservedTypes.IsEmpty())
	{
		UE_LOGF(LogMass, Error
			, "%hs: %ls has bAutoRegisterWithObserverRegistry=true but no observed types configured"
			, __FUNCTION__, *GetClass()->GetName());
		return;
	}

	UMassObserverRegistry& Registry = UMassObserverRegistry::GetMutable();
	const uint8 OperationFlags = static_cast<uint8>(ObservedOperations);
	for (const UScriptStruct* Type : ObservedTypes)
	{
		Registry.RegisterObserver(Type, OperationFlags, GetClass());
	}
}
