// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassObserverRegistry.h"
#include "Algo/Find.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassObserverRegistry)

//----------------------------------------------------------------------//
// UMassObserverRegistry
//----------------------------------------------------------------------//
UMassObserverRegistry::UMassObserverRegistry()
{
	// there can be only one!
	check(HasAnyFlags(RF_ClassDefaultObject));

	if (!ModulesUnloadedHandle.IsValid())
	{
		ModulesUnloadedHandle = FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.AddUObject(this, &UMassObserverRegistry::OnModulePackagesUnloaded);
	}
}

void UMassObserverRegistry::RegisterObserver(TNotNull<const UScriptStruct*> ObservedType, const uint8 OperationFlags, const TSubclassOf<UMassProcessor>& ObserverClass)
{
	check(ObserverClass);
	ensureAlwaysMsgf(UE::Mass::IsSparse(ObservedType) == false, TEXT("%hs: trying to register observer for %s, but observing sparse elements is not supported at the moment.")
		, __FUNCTION__, *ObservedType->GetName());

	for (uint8 OperationIndex = 0; OperationIndex < static_cast<uint8>(EMassObservedOperation::MAX); ++OperationIndex)
	{
		if ((OperationFlags & (1 << OperationIndex)))
		{
			ElementObserverMaps[OperationIndex].FindOrAdd(ObservedType).AddUnique(FSoftClassPath(ObserverClass));
		}
	}
}

void UMassObserverRegistry::RegisterObserver(const UScriptStruct& ObservedType, const EMassObservedOperation Operation, const TSubclassOf<UMassProcessor>& ObserverClass)
{
	ensureAlwaysMsgf(UE::Mass::IsSparse(&ObservedType) == false, TEXT("%hs: trying to register observer for %s, but observing sparse elements is not supported at the moment.")
		, __FUNCTION__, *ObservedType.GetName());

	RegisterObserver(&ObservedType, static_cast<uint8>(1 << static_cast<uint8>(Operation)), ObserverClass);
}

void UMassObserverRegistry::OnModulePackagesUnloaded(TConstArrayView<UPackage*> Packages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMassObserverRegistry::OnModulePackagesUnloaded);

	const auto ProcessObserversMap = [&Packages](FObserverClassesMap& ObserversMap)
	{
		for (auto MapIt = ObserversMap.CreateIterator(); MapIt; ++MapIt)
		{
			TArray<FSoftClassPath>& ObserverClasses = MapIt->Value;

			for (auto ObserverIt = ObserverClasses.CreateIterator(); ObserverIt; ++ObserverIt)
			{
				const FSoftClassPath& ObservedClass = *ObserverIt;
				const FName PackageName = ObservedClass.GetLongPackageFName();

				bool bRemove = !!Algo::FindByPredicate(Packages, [PackageName](const UPackage* Package)
				{
					return PackageName == Package->GetFName();
				});

				if (bRemove)
				{
					UE_LOGF(LogMass, Log, "%s: removed observer %ls (%ls)", __FUNCTION__, *ObservedClass.ToString(), *PackageName.ToString());
					ObserverIt.RemoveCurrent();
				}
			}

			if (ObserverClasses.Num() == 0)
			{
				MapIt.RemoveCurrent();
			}
		}
	};

	for (uint8 OperationIndex = 0; OperationIndex < static_cast<uint8>(EMassObservedOperation::MAX); ++OperationIndex)
	{
		ProcessObserversMap(ElementObserverMaps[OperationIndex]);
	}
}
