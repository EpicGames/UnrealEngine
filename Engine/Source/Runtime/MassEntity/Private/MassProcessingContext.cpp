// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassProcessingContext.h"

namespace UE::Mass
{
	// @todo remove the deprecation disabling once CommandBuffer and EntityManager are moved to `protected` and un-deprecated (around UE5.8)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FProcessingContext::~FProcessingContext()
	{	
		if (ExecutionContextPtr)
		{
			checkf(ExecutionContextPtr->GetSharedDeferredCommandBuffer(), TEXT("A valid execution context without a valid command buffer is unexpected"));
			checkf(ExecutionContextPtr->GetSharedDeferredCommandBuffer() == CommandBuffer
			   , TEXT("Hosted Execution Context's command buffer is not the same as FProcessingContext's command buffer. "
				 "This is not supposed to happen. Make sure FProcessingContext.CommandBuffer never gets assigned after FMassExecutionContext's creation"));

			ensure(!CommandBuffer->IsFlushing());

			if (bFlushCommandBuffer)
			{
				EntityManager->FlushCommands(CommandBuffer);
			}
			else
			{
				EntityManager->AppendCommands(CommandBuffer);
			}

			ExecutionContextPtr->~FMassExecutionContext();
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
} // namespace UE::Mass
