// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassProcessingPhaseManager.h"
#include "MassProcessingTypes.h"
#include "MassDebugger.h"
#include "MassProcessor.h"
#include "MassExecutor.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.h"
#include "VisualLogger/VisualLogger.h"
#include "Engine/World.h"
#include "MassCommandBuffer.h"
#include "MassEntityTrace.h"
#include "MassProcessingContext.h"
#include "Misc/StringOutputDevice.h"
#include "MassProcessingQueue.h"
#include "ObjectTraceDefines.h"
#if OBJECT_TRACE_ENABLED
#include "ObjectTrace.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassProcessingPhaseManager)

#define LOCTEXT_NAMESPACE "Mass"

DECLARE_CYCLE_STAT(TEXT("Mass Phase Tick"), STAT_Mass_PhaseTick, STATGROUP_Mass);
DECLARE_CYCLE_STAT(TEXT("Mass Phase Configure Pipeline Creation"), STAT_Mass_PhaseConfigurePipelineCreation, STATGROUP_Mass);

namespace UE::Mass::Tweakables
{
	bool bFullyParallel = MASS_DO_PARALLEL;
	bool bMakePrePhysicsTickFunctionHighPriority = true;
	bool bUseProcessingQueue = true;

	FAutoConsoleVariableRef CVars[] = {
		{TEXT("mass.FullyParallel"), bFullyParallel, TEXT("Enables mass processing distribution to all available thread (via the task graph)")},
		{TEXT("mass.MakePrePhysicsTickFunctionHighPriority"), bMakePrePhysicsTickFunctionHighPriority, TEXT("Whether to make the PrePhysics tick function high priority - can minimise GameThread waits by starting parallel work as soon as possible")},
		{TEXT("mass.UseProcessingQueue"), bUseProcessingQueue, TEXT("Enable MassProcessingQueue for executing processing phases.")},
	};
} // namespace UE::Mass::Tweakables

namespace UE::Mass::Private
{
	ETickingGroup PhaseToTickingGroup[static_cast<int32>(EMassProcessingPhase::MAX)]
	{
		ETickingGroup::TG_PrePhysics, // EMassProcessingPhase::PrePhysics
		ETickingGroup::TG_StartPhysics, // EMassProcessingPhase::StartPhysics
		ETickingGroup::TG_DuringPhysics, // EMassProcessingPhase::DuringPhysics
		ETickingGroup::TG_EndPhysics,	// EMassProcessingPhase::EndPhysics
		ETickingGroup::TG_PostPhysics,	// EMassProcessingPhase::PostPhysics
		ETickingGroup::TG_LastDemotable, // EMassProcessingPhase::FrameEnd
	};
} // namespace UE::Mass::Private

//----------------------------------------------------------------------//
//  FMassProcessingPhase
//----------------------------------------------------------------------//
FMassProcessingPhase::FMassProcessingPhase()
{
	bCanEverTick = true;
	bStartWithTickEnabled = false;
	SupportedTickTypes = (1 << LEVELTICK_All) | (1 << LEVELTICK_TimeOnly);
}

void FMassProcessingPhase::ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	if (ShouldTick(TickType) == false)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_Mass_PhaseTick);
	SCOPE_CYCLE_COUNTER(STAT_Mass_Total);

	checkf(PhaseManager, TEXT("Manager is null which is not a supported case. Either this FMassProcessingPhase has not been initialized properly or it's been left dangling after the FMassProcessingPhase owner got destroyed."));

	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FMassProcessingPhase::ExecuteTick %s"), *UEnum::GetValueAsString(Phase)));

	PhaseManager->OnPhaseStart(*this);
	{
		LLM_SCOPE_BYNAME(TEXT("Mass/PhaseStartDelegate"));
		OnPhaseStart.Broadcast(DeltaTime);
	}

	check(PhaseProcessor);
	
	FMassEntityManager& EntityManager = PhaseManager->GetEntityManagerRef();

	bIsDuringMassProcessing = true;	

	if (bRunInParallelMode && UE::Mass::Tweakables::bUseProcessingQueue && PhaseManager->IsPaused() == false)
	{
		ObserversLockPtr = EntityManager.GetOrMakeObserversLock().ToSharedPtr();
		bool bWorkRequested = false;
		if (PhaseProcessor->IsEmpty() == false)
		{
			FMassProcessingContext Context(EntityManager, DeltaTime);
			auto OnProcessingDone = [this, DeltaTime]()
				{
					OnParallelExecutionDone(DeltaTime);
				};
			auto OnPreCommandFlush = [this]()
				{
					bIsDuringMassProcessing = false;
					ObserversLockPtr = nullptr;
				};
			const FGraphEventRef CleanupEvent = ProcessingQueue.Execute_GetEventRef(MoveTemp(Context), OnProcessingDone, CurrentThread, OnPreCommandFlush);
			if (CleanupEvent.IsValid())
			{
				MyCompletionGraphEvent->DontCompleteUntil(CleanupEvent);
				bWorkRequested = true;
			}
		}
		if (bWorkRequested == false)
		{
			ObserversLockPtr = nullptr;
			OnParallelExecutionDone(DeltaTime);
		}
	}
	else if (bRunInParallelMode && PhaseManager->IsPaused() == false)
	{
		bool bWorkRequested = false;
		if (PhaseProcessor->IsEmpty() == false)
		{
			ObserversLockPtr = EntityManager.GetOrMakeObserversLock().ToSharedPtr();
			FMassProcessingContext Context(EntityManager, DeltaTime);

			auto OnProcessingDone = [this, DeltaTime]()
				{
					OnParallelExecutionDone(DeltaTime);
				};

			auto OnPreCommandFlush = [this]()
				{
					ObserversLockPtr = nullptr;
				};

			const FGraphEventRef PipelineCompletionEvent = UE::Mass::Executor::TriggerParallelTasks(*PhaseProcessor, MoveTemp(Context), 
				OnProcessingDone, CurrentThread, OnPreCommandFlush);
			if (PipelineCompletionEvent.IsValid())
			{
				MyCompletionGraphEvent->DontCompleteUntil(PipelineCompletionEvent);
				bWorkRequested = true;
			}
		}
		if (bWorkRequested == false)
		{
			ObserversLockPtr = nullptr;
			OnParallelExecutionDone(DeltaTime);
		}
	}
	else
	{
		if (PhaseManager->IsPaused() == false)
		{
			ObserversLockPtr = EntityManager.GetOrMakeObserversLock().ToSharedPtr();
			// note that it's important to create the processing context in this scope
			// so that it wraps up its destruction before we call OnPhaseEnd, which in turn will cause
			// the main EntityManager's command buffer to flush
			FMassProcessingContext Context(EntityManager, DeltaTime);
			UE::Mass::Executor::Run(*PhaseProcessor, Context);
			
			// release the observer lock before the FMassProcessingContext is destroyed and commands are flushed
			ObserversLockPtr = nullptr;

		}

		bIsDuringMassProcessing = false;
		{
			LLM_SCOPE_BYNAME(TEXT("Mass/PhaseEndDelegate"));
			OnPhaseEnd.Broadcast(DeltaTime);
		}
		PhaseManager->OnPhaseEnd(*this);
	}
}

void FMassProcessingPhase::OnParallelExecutionDone(const float DeltaTime)
{
	bIsDuringMassProcessing = false;
	ObserversLockPtr = nullptr;
	{
		LLM_SCOPE_BYNAME(TEXT("Mass/PhaseEndDelegate"));
		OnPhaseEnd.Broadcast(DeltaTime);
	}
	check(PhaseManager);
	PhaseManager->OnPhaseEnd(*this);
}

FString FMassProcessingPhase::DiagnosticMessage()
{
	return (PhaseManager ? PhaseManager->GetName() : TEXT("NULL-MassProcessingPhaseManager")) + TEXT("[ProcessingPhaseTick]");
}

FName FMassProcessingPhase::DiagnosticContext(bool bDetailed)
{
	return TEXT("MassProcessingPhase");
}

void FMassProcessingPhase::Initialize(FMassProcessingPhaseManager& InPhaseManager, const EMassProcessingPhase InPhase, const ETickingGroup InTickGroup, UMassCompositeProcessor& InPhaseProcessor)
{
	PhaseManager = &InPhaseManager;
	Phase = InPhase;
	TickGroup = InTickGroup;
	PhaseProcessor = &InPhaseProcessor;
}

//----------------------------------------------------------------------//
// FPhaseProcessorConfigurator
//----------------------------------------------------------------------//
void FMassPhaseProcessorConfigurationHelper::Configure(TArrayView<UMassProcessor* const> DynamicProcessors, TArray<TWeakObjectPtr<UMassProcessor>>& InOutRemovedDynamicProcessors
	, EProcessorExecutionFlags InWorldExecutionFlags, const TSharedRef<FMassEntityManager>& EntityManager
	, FMassProcessorDependencySolver::FResult& InOutOptionalResult)
{
	FMassRuntimePipeline TmpPipeline(PhaseProcessor.GetChildProcessorsView(), InWorldExecutionFlags);
	{
		SCOPE_CYCLE_COUNTER(STAT_Mass_PhaseConfigurePipelineCreation);

		TmpPipeline.AppendProcessors(InOutOptionalResult.PrunedProcessors);

		if (TmpPipeline.Num())
		{
			// some previously added dynamic processors were either in the active processor group,
			// or were among the pruned processors. At this point we have both groups in TmpPipeline now
			// so we need to check if any of these processors have been removed since las processing
			// graph recreation
			for (int32 Index = InOutRemovedDynamicProcessors.Num() - 1; Index >= 0; --Index)
			{
				if (const UMassProcessor* RemovedProcessor = InOutRemovedDynamicProcessors[Index].Get())
				{
					if (TmpPipeline.RemoveProcessor(*RemovedProcessor) == false)
					{
						continue;
					}
				}
				InOutRemovedDynamicProcessors.RemoveAtSwap(Index, 1, EAllowShrinking::No);
			}
		}

		for (UMassProcessor* Processor : DynamicProcessors)
		{
			checkf(Processor != nullptr, TEXT("Dynamic processor provided to MASS is null."));
			if (Processor->GetProcessingPhase() == Phase)
			{
				TmpPipeline.AppendUniqueProcessor(*Processor);
			}
		}

		UObject* Owner = EntityManager->GetOwner();
		check(Owner);
		// @todo consider doing this only during initial config.
		TmpPipeline.AppendUniqueRuntimeProcessorCopies(PhaseConfig.ProcessorCDOs, *Owner, EntityManager);
	}

	TArray<FMassProcessorOrderInfo> SortedProcessors;
	FMassProcessorDependencySolver Solver(TmpPipeline.GetMutableProcessors(), bIsGameRuntime);

	Solver.ResolveDependencies(SortedProcessors, EntityManager, &InOutOptionalResult);

	PhaseProcessor.UpdateProcessorsCollection(SortedProcessors, InWorldExecutionFlags);

#if WITH_MASSENTITY_DEBUG
	for (const FMassProcessorOrderInfo& ProcessorOrderInfo : SortedProcessors)
	{
		TmpPipeline.RemoveProcessor(*ProcessorOrderInfo.Processor);
	}
	
	if (TmpPipeline.Num())
	{
		UE_VLOG_UELOG(&PhaseProcessor, LogMass, Verbose, TEXT("Discarding processors due to not having anything to do (no relevant Archetypes):"));
		for (UMassProcessor* Processor : TmpPipeline.GetProcessors())
		{
			UE_VLOG_UELOG(&PhaseProcessor, LogMass, Verbose, TEXT("\t%s"), *Processor->GetProcessorName());
		}
	}
#endif // WITH_MASSENTITY_DEBUG

	if (Solver.IsSolvingForSingleThread() == false)
	{
		PhaseProcessor.BuildFlatProcessingGraph(SortedProcessors);
	}

	if (bInitializeCreatedProcessors)
	{
		PhaseProcessor.InitializeInternal(ProcessorOuter, EntityManager);
	}
}

//----------------------------------------------------------------------//
// FMassProcessingPhaseManager::FPhaseGraphBuildState
//----------------------------------------------------------------------//
void FMassProcessingPhaseManager::FPhaseGraphBuildState::Reset()
{
	LastResult.Reset();
	bInitialized = false;
}

//----------------------------------------------------------------------//
// FMassProcessingPhaseManager
//----------------------------------------------------------------------//
FMassProcessingPhaseManager::FMassProcessingPhaseManager(EProcessorExecutionFlags InProcessorExecutionFlags) 
	: ProcessorExecutionFlags(InProcessorExecutionFlags)
{
#if WITH_MASSENTITY_DEBUG
	OnDebugEntityManagerInitializedHandle = FMassDebugger::OnEntityManagerInitialized.AddRaw(this, &FMassProcessingPhaseManager::OnDebugEntityManagerInitialized);
	OnDebugEntityManagerDeinitializedHandle = FMassDebugger::OnEntityManagerInitialized.AddRaw(this, &FMassProcessingPhaseManager::OnDebugEntityManagerDeinitialized);
#endif // WITH_MASSENTITY_DEBUG
}

void FMassProcessingPhaseManager::Initialize(UObject& InOwner, TConstArrayView<FMassProcessingPhaseConfig> InProcessingPhasesConfig, const FString& DependencyGraphFileName)
{
	UWorld* World = InOwner.GetWorld();

	Owner = &InOwner;
	ProcessingPhasesConfig = InProcessingPhasesConfig;

	ProcessorExecutionFlags = UE::Mass::Utils::DetermineProcessorExecutionFlags(World, ProcessorExecutionFlags);
	const uint8 SupportedTickTypes = UE::Mass::Utils::DetermineProcessorSupportedTickTypes(World);

	for (int32 PhaseAsInt = 0; PhaseAsInt < static_cast<int32>(EMassProcessingPhase::MAX); ++PhaseAsInt)
	{		
		const EMassProcessingPhase Phase = static_cast<EMassProcessingPhase>(PhaseAsInt);
		FMassProcessingPhase& ProcessingPhase = ProcessingPhases[PhaseAsInt];

		UMassCompositeProcessor* PhaseProcessor = NewObject<UMassCompositeProcessor>(&InOwner, UMassCompositeProcessor::StaticClass()
			, *FString::Printf(TEXT("ProcessingPhase_%s"), *UEnum::GetDisplayValueAsText(Phase).ToString()));
	
		check(PhaseProcessor);
		ProcessingPhase.Initialize(*this, Phase, UE::Mass::Private::PhaseToTickingGroup[PhaseAsInt], *PhaseProcessor);
		ProcessingPhase.SupportedTickTypes = SupportedTickTypes;

		REDIRECT_OBJECT_TO_VLOG(PhaseProcessor, &InOwner);
		PhaseProcessor->SetProcessingPhase(Phase);
		PhaseProcessor->SetGroupName(FName(FString::Printf(TEXT("%s Group"), *UEnum::GetDisplayValueAsText(Phase).ToString())));

#if WITH_MASSENTITY_DEBUG
		FStringOutputDevice Ar;
		PhaseProcessor->DebugOutputDescription(Ar);
		UE_VLOG(&InOwner, LogMass, Log, TEXT("Setting new group processor for phase %s:\n%s"), *UEnum::GetValueAsString(Phase), *Ar);
#endif // WITH_MASSENTITY_DEBUG
	}

	bIsAllowedToTick = true;

	UE_LOGF(LogMass, Log, "FMassProcessingPhaseManager::Initialize complete. bUseProcessingQueue=%ls", UE::Mass::Tweakables::bUseProcessingQueue ? TEXT("true") : TEXT("false"));
}

void FMassProcessingPhaseManager::Deinitialize()
{
	for (FMassProcessingPhase& Phase : ProcessingPhases)
	{
		Phase.PhaseProcessor = nullptr;
	}

	DynamicProcessors.Reset();

	for (FPhaseGraphBuildState& GraphBuildState : ProcessingGraphBuildStates)
	{
		GraphBuildState.Reset();
	}

	// manually deque all the queues, since there's no guarantee that this
	// FMassProcessingPhaseManager instance is getting destroyed right after this call
	FDynamicProcessorOperation DummyElement;
	for (TMpscQueue<FDynamicProcessorOperation>& Queue : PendingDynamicProcessors)
	{	
		while (Queue.Dequeue(DummyElement))
		{
			// empty on purpose
		}
	}
}

const FGraphEventRef& FMassProcessingPhaseManager::TriggerPhase(const EMassProcessingPhase Phase, const float DeltaTime
	, const FGraphEventRef& MyCompletionGraphEvent, ENamedThreads::Type CurrentThread)
{
	check(Phase != EMassProcessingPhase::MAX);

	if (bIsAllowedToTick)
	{
		ProcessingPhases[static_cast<int32>(Phase)].ExecuteTick(DeltaTime, LEVELTICK_All, CurrentThread, MyCompletionGraphEvent);
	}

	return MyCompletionGraphEvent;
}

void FMassProcessingPhaseManager::Start(UWorld& World)
{
	UMassEntitySubsystem* EntitySubsystem = UWorld::GetSubsystem<UMassEntitySubsystem>(&World);

	if (ensure(EntitySubsystem))
	{
		Start(EntitySubsystem->GetMutableEntityManager().AsShared());
	}
	else
	{
		UE_VLOG_UELOG(Owner.Get(), LogMass, Error, TEXT("Called %s while missing the EntitySubsystem"), ANSI_TO_TCHAR(__FUNCTION__));
	}
}

void FMassProcessingPhaseManager::Start(const TSharedRef<FMassEntityManager>& InEntityManager)
{
	EntityManager = InEntityManager;

#if WITH_MASSENTITY_DEBUG
	FMassDebugger::RegisterProcessorDataProvider(TEXT("Phase-executed processors"), InEntityManager, [WeakThis = AsWeak()](TArray<const UMassProcessor*>& OutProcessors)
	{
		if (TSharedPtr<FMassProcessingPhaseManager> SharedThis = WeakThis.Pin())
		{
			for (const FMassProcessingPhase& Phase : SharedThis->ProcessingPhases)
			{
				OutProcessors.Add(Phase.DebugGetPhaseProcessor());
				OutProcessors.Append(Phase.DebugGetPhaseProcessor()->GetChildProcessorsView());
			}
		}
	});

	FMassDebugger::RegisterProcessorDataProvider(TEXT("Pruned processors"), InEntityManager, [WeakThis = AsWeak()](TArray<const UMassProcessor*>& OutProcessors)
	{
		if (TSharedPtr<FMassProcessingPhaseManager> SharedThis = WeakThis.Pin())
		{
			TConstArrayView<FPhaseGraphBuildState> BuildStates = SharedThis->DebugGetProcessingGraphBuildStates();
			for (const FPhaseGraphBuildState& State : BuildStates)
			{
				OutProcessors.Append(ObjectPtrDecay(State.LastResult.PrunedProcessors));
			}
		}
	});
#endif // WITH_MASSENTITY_DEBUG

	OnNewArchetypeHandle = EntityManager->GetOnNewArchetypeEvent().AddRaw(this, &FMassProcessingPhaseManager::OnNewArchetype);

	if (UWorld* World = EntityManager->GetWorld())
	{
		EnableTickFunctions(*World);
	}

	bIsAllowedToTick = true;
}

void FMassProcessingPhaseManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (FMassProcessingPhase& Phase : ProcessingPhases)
	{
		if (Phase.PhaseProcessor)
		{
			Collector.AddReferencedObject(Phase.PhaseProcessor);
		}
	}

	auto NullProcessorRemover = [](const TObjectPtr<UMassProcessor>& Processor)
	{
		return !Processor;
	};

	check(DynamicProcessors.RemoveAllSwap(NullProcessorRemover) == 0);
	Collector.AddStableReferenceArray(&DynamicProcessors);

	// we also need to store our pruned processors
	for (FPhaseGraphBuildState& GraphBuildState : ProcessingGraphBuildStates)
	{
		check(GraphBuildState.LastResult.PrunedProcessors.RemoveAllSwap(NullProcessorRemover) == 0);
		Collector.AddStableReferenceArray(&GraphBuildState.LastResult.PrunedProcessors);
	}
}

void FMassProcessingPhaseManager::EnableTickFunctions(const UWorld& World)
{
	check(EntityManager);

	const bool bIsGameWorld = World.IsGameWorld();

	for (FMassProcessingPhase& Phase : ProcessingPhases)
	{
		if (UE::Mass::Tweakables::bMakePrePhysicsTickFunctionHighPriority && (Phase.Phase == EMassProcessingPhase::PrePhysics))
		{
			constexpr bool bHighPriority = true;
			Phase.SetPriorityIncludingPrerequisites(bHighPriority);
		}

		Phase.RegisterTickFunction(World.PersistentLevel);
		Phase.SetTickFunctionEnable(true);
#if WITH_MASSENTITY_DEBUG
		if (Phase.PhaseProcessor && bIsGameWorld)
		{
			// not logging this in the editor mode since it messes up the game-recorded vislog display (with its progressively larger timestamp)
			FStringOutputDevice Ar;
			Phase.PhaseProcessor->DebugOutputDescription(Ar);
			UE_VLOG_UELOG(Owner.Get(), LogMass, Log, TEXT("Enabling phase %s tick:\n%s")
				, *UEnum::GetValueAsString(Phase.Phase), *Ar);
		}
#endif // WITH_MASSENTITY_DEBUG
	}

	if (bIsGameWorld)
	{
		// not logging this in the editor mode since it messes up the game-recorded vislog display (with its progressively larger timestamp)
		UE_VLOG_UELOG(Owner.Get(), LogMass, Log, TEXT("MassProcessingPhaseManager %s.%s has been started")
			, *GetNameSafe(Owner.Get()), *GetName());
	}
}

void FMassProcessingPhaseManager::Stop()
{
	bIsAllowedToTick = false;

	if (EntityManager)
	{
		EntityManager->GetOnNewArchetypeEvent().Remove(OnNewArchetypeHandle);
		EntityManager.Reset();
	}
	
	for (FMassProcessingPhase& Phase : ProcessingPhases)
	{
		Phase.SetTickFunctionEnable(false);
	}

	if (UObject* LocalOwner = Owner.Get())
	{
		UWorld* World = LocalOwner->GetWorld();
		if (World && World->IsGameWorld())
		{
			// not logging this in editor mode since it messes up the game-recorded vislog display (with its progressively larger timestamp) 
			UE_VLOG_UELOG(LocalOwner, LogMass, Log, TEXT("MassProcessingPhaseManager %s.%s has been stopped")
				, *GetNameSafe(LocalOwner), *GetName());
		}
	}
}

void FMassProcessingPhaseManager::Pause()
{
	check(IsInGameThread());

	if (bIsPaused == false)
	{
		bIsPauseTogglePending = true;

		UE_VLOG_UELOG(Owner.Get(), LogMass, Log, TEXT("Scheduling Pause for next FrameEnd phase"));
	}
}

void FMassProcessingPhaseManager::Resume()
{
	check(IsInGameThread());

	if (bIsPaused == true)
	{
		bIsPauseTogglePending = true;

		UE_VLOG_UELOG(Owner.Get(), LogMass, Log, TEXT("Scheduling Resume for next PrePhysics phase"));
	}
}

void FMassProcessingPhaseManager::OnPhaseStart(FMassProcessingPhase& Phase)
{
	ensure(CurrentPhase == EMassProcessingPhase::MAX);
	CurrentPhase = Phase.Phase;

	const int32 PhaseAsInt = static_cast<int32>(Phase.Phase);

	// The VERY FIRST thing we do in the first phase is to change the Pause state if needed.
	// This way any code that depends on knowing the pause state (if any) gets consistent results.
	if (bIsPauseTogglePending && bIsPaused == true && PhaseAsInt == 0)
	{
		bIsPaused = false;
		bIsPauseTogglePending = false;

		UE_VLOG_UELOG(Owner.Get(), LogMass, Log, TEXT("Phase Processing is now Resumed"));
	}

	// switch between parallel and single-thread versions only after a given batch of processing has been wrapped up	
	if (Phase.IsConfiguredForParallelMode() != UE::Mass::Tweakables::bFullyParallel)
	{
		if (UE::Mass::Tweakables::bFullyParallel)
		{
			Phase.ConfigureForParallelMode();
		}
		else
		{
			Phase.ConfigureForSingleThreadMode();
		}
	}

	if (PendingDynamicProcessors[PhaseAsInt].IsEmpty() == false)
	{
		HandlePendingDynamicProcessorOperations(PhaseAsInt);
	}

#if OBJECT_TRACE_ENABLED
	UE_TRACE_MASS_SET_RECORDING_TIME(FObjectTrace::GetWorldElapsedTime(EntityManager->GetWorld()))
#endif

	UE_TRACE_MASS_PHASE_BEGIN(PhaseAsInt)

	if (Owner.IsValid()
		&& ensure(Phase.Phase != EMassProcessingPhase::MAX)
		&& (ProcessingGraphBuildStates[PhaseAsInt].bNewArchetypes || ProcessingGraphBuildStates[PhaseAsInt].bProcessorsNeedRebuild)
		// if not a valid index then we're not able to recalculate dependencies 
		&& ensure(ProcessingPhasesConfig.IsValidIndex(PhaseAsInt)))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Mass Rebuild Phase Graph");

		FPhaseGraphBuildState& GraphBuildState = ProcessingGraphBuildStates[PhaseAsInt];
		if (GraphBuildState.bInitialized == false 
			|| ProcessingGraphBuildStates[PhaseAsInt].bProcessorsNeedRebuild
			|| FMassProcessorDependencySolver::IsResultUpToDate(GraphBuildState.LastResult, EntityManager) == false)
		{
			UMassCompositeProcessor* PhaseProcessor = ProcessingPhases[PhaseAsInt].PhaseProcessor;
			check(PhaseProcessor);

			GraphBuildState.LastResult.Reset();

			FMassPhaseProcessorConfigurationHelper Configurator(*PhaseProcessor, ProcessingPhasesConfig[PhaseAsInt], *Owner.Get(), Phase.Phase);
			Configurator.Configure(DynamicProcessors, RemovedDynamicProcessors, ProcessorExecutionFlags, EntityManager.ToSharedRef(), GraphBuildState.LastResult);

			GraphBuildState.bInitialized = true;

			if (UE::Mass::Tweakables::bUseProcessingQueue)
			{
				UE::Mass::FProcessingQueue& PhaseQueue = ProcessingPhases[PhaseAsInt].ProcessingQueue;
				TConstArrayView<UMassProcessor*> Processors = PhaseProcessor->GetChildProcessorsView();
				PhaseQueue.SetProcessors(Processors);
			}

#if WITH_MASSENTITY_DEBUG
			UObject* OwnerPtr = Owner.Get();
			// print it all out to vislog
			UE_VLOG_UELOG(OwnerPtr, LogMass, Verbose, TEXT("Phases initialization done. Current composition:"));

			FStringOutputDevice OutDescription;
			PhaseProcessor->DebugOutputDescription(OutDescription);
			UE_VLOG_UELOG(OwnerPtr, LogMass, Verbose, TEXT("--- %s"), *OutDescription);
			OutDescription.Reset();
#endif // WITH_MASSENTITY_DEBUG
		}

		ProcessingGraphBuildStates[PhaseAsInt].bProcessorsNeedRebuild = false;
		ProcessingGraphBuildStates[PhaseAsInt].bNewArchetypes = false;
	}
}

void FMassProcessingPhaseManager::OnPhaseEnd(FMassProcessingPhase& Phase)
{
	ensure(CurrentPhase == Phase.Phase);
	UE_TRACE_MASS_PHASE_END(static_cast<int32>(CurrentPhase))
	CurrentPhase = EMassProcessingPhase::MAX;

	// The VERY LAST thing we do in FrameEnd is change the Pause state if needed.
	// This way any code that depends on knowing the pause state (if any) gets consistent results.
	if (bIsPauseTogglePending && bIsPaused == false
		&& Phase.Phase == EMassProcessingPhase::FrameEnd)
	{
		bIsPaused = true;
		bIsPauseTogglePending = false;

#if WITH_MASSENTITY_DEBUG
		UE_VLOG_UELOG(Owner.Get(), LogMass, Log, TEXT("Phase Processing is now Paused"));
#endif // WITH_MASSENTITY_DEBUG
	}

	if (GetEntityManagerRef().Defer().HasPendingCommands())
	{
		GetEntityManagerRef().FlushCommands();
	}
}

FString FMassProcessingPhaseManager::GetName() const
{
	return GetNameSafe(Owner.Get()) + TEXT("_MassProcessingPhaseManager");
}

void FMassProcessingPhaseManager::RegisterDynamicProcessor(UMassProcessor& Processor)
{
	if (ensureMsgf(Processor.GetProcessingPhase() != EMassProcessingPhase::MAX
		, TEXT("%hs, Misconfigured processor %s, marked as ProcessingPhase == MAX"), __FUNCTION__, *Processor.GetName()))
	{
		PendingDynamicProcessors[static_cast<int32>(Processor.GetProcessingPhase())].Enqueue(&Processor, EDynamicProcessorOperationType::Add);
	}
}

void FMassProcessingPhaseManager::RegisterDynamicProcessorInternal(TNotNull<UMassProcessor*> Processor)
{
	if (Processor->IsInitialized() == false)
	{
		check(EntityManager->GetOwner());
		Processor->CallInitialize(EntityManager->GetOwner(), EntityManager.ToSharedRef());
	}
	DynamicProcessors.Add(Processor);
	Processor->MarkAsDynamic();
}

void FMassProcessingPhaseManager::UnregisterDynamicProcessor(UMassProcessor& Processor)
{
	if (ensureMsgf(Processor.GetProcessingPhase() != EMassProcessingPhase::MAX
		, TEXT("%hs, Misconfigured processor %s, marked as ProcessingPhase == MAX"), __FUNCTION__, *Processor.GetName()))
	{
		PendingDynamicProcessors[static_cast<int32>(Processor.GetProcessingPhase())].Enqueue(&Processor, EDynamicProcessorOperationType::Remove);
	}
}

void FMassProcessingPhaseManager::UnregisterDynamicProcessorInternal(TNotNull<UMassProcessor*> Processor)
{
	int32 Index = INDEX_NONE;
	if (DynamicProcessors.Find(Processor, Index))
	{
		DynamicProcessors.RemoveAtSwap(Index, 1, EAllowShrinking::No);
		ProcessingGraphBuildStates[static_cast<int32>(Processor->GetProcessingPhase())].bProcessorsNeedRebuild = true;

		// it's possible that the given dynamic processor is a part of processing graph at the moment
		// we need to store the information about its removal and use it when rebuilding the graph next time around.
		RemovedDynamicProcessors.Add(Processor);
	}
	else
	{
		checkf(false, TEXT("Unable to remove Processor '%s', as it was never added or already removed."), *Processor->GetName());
	}
}

void FMassProcessingPhaseManager::HandlePendingDynamicProcessorOperations(const int32 PhaseIndex)
{
	bool bWorkDone = false;
	FDynamicProcessorOperation Operation;
	while (PendingDynamicProcessors[PhaseIndex].Dequeue(Operation))
	{
		if (Operation.Get<1>() == EDynamicProcessorOperationType::Add)
		{
			RegisterDynamicProcessorInternal(Operation.Get<0>().Get());
		}
		else
		{
			UnregisterDynamicProcessorInternal(Operation.Get<0>().Get());
		}
		bWorkDone = true;
	}

	if (bWorkDone)
	{
		ProcessingGraphBuildStates[PhaseIndex].bProcessorsNeedRebuild = bWorkDone;
	}
}

void FMassProcessingPhaseManager::OnNewArchetype(const FMassArchetypeHandle& NewArchetype)
{
	for (FPhaseGraphBuildState& GraphBuildState : ProcessingGraphBuildStates)
	{
		GraphBuildState.bNewArchetypes = true;
	}
}

#if WITH_MASSENTITY_DEBUG
void FMassProcessingPhaseManager::OnDebugEntityManagerInitialized(const FMassEntityManager& InEntityManager)
{
	
}

void FMassProcessingPhaseManager::OnDebugEntityManagerDeinitialized(const FMassEntityManager& InEntityManager)
{
	
}
#endif // WITH_MASSENTITY_DEBUG

//-----------------------------------------------------------------------------
// DEPRECATED
//-----------------------------------------------------------------------------
UE_DEPRECATED(5.6, "This flavor of Configure is deprecated. Please use the one using a TSharedRef<FMassEntityManager> parameter instead")
void FMassPhaseProcessorConfigurationHelper::Configure(TArrayView<UMassProcessor* const> DynamicProcessors, EProcessorExecutionFlags InWorldExecutionFlags
	, const TSharedPtr<FMassEntityManager>& EntityManager
	, FMassProcessorDependencySolver::FResult* OutOptionalResult)
{
	if (ensureMsgf(EntityManager, TEXT("Configuring processors without a valid EntityManager is no longer supported"))
		&& OutOptionalResult)
	{
		static TArray<TWeakObjectPtr<UMassProcessor>> DummyRemovedDynamicProcessors;
		Configure(DynamicProcessors, DummyRemovedDynamicProcessors, InWorldExecutionFlags, EntityManager.ToSharedRef(), *OutOptionalResult);
	}
}

void FMassProcessingPhaseManager::Start(const TSharedPtr<FMassEntityManager>& InEntityManager)
{
	if (InEntityManager)
	{
		Start(InEntityManager.ToSharedRef());
	}
}

#undef LOCTEXT_NAMESPACE
