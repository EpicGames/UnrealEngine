// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassProcessingQueue.h"
#include "Algo/Compare.h"
#include "Containers/BitArray.h"
#include "Containers/Queue.h"
#include "MassProcessingTypes.h"
#include "MassProcessor.h"
#include "MassTypeManager.h"
#include "MassCommandBuffer.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"
#include "Logging/MessageLog.h"
#include "MassArchetypeData.h"
#include "MassEntityQuery.h"

#define LOCTEXT_NAMESPACE "MassProcessingQueue"

#define MASS_PROCESSING_QUEUE_LOGGING 0
#define REUSE_CHUNK_QUEUE_ENTRIES 1

namespace UE::Mass::Tweakables
{
	// Defined in MassProcessingQueueTypes.cpp; startup-only (ECVF_ReadOnly).
	extern bool bForceInlineProcessorExecution;
}

#if MASS_CONCURRENCY_STATS
#include "ProfilingDebugging/CountersTrace.h"

static std::atomic<int32> GActiveMassProcessorCount{0};
static std::atomic<int32> GGlobalMassConcurrencyHWM{0};
static std::atomic<int32> GActiveMassChunkCount{0};
static std::atomic<int32> GGlobalMassChunkHWM{0};

TRACE_DECLARE_ATOMIC_INT_COUNTER(MassProcessing_ActiveProcessors,  TEXT("Mass Active Processors"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(MassProcessing_ExecutionHWM,      TEXT("Mass Execution Concurrency HWM"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(MassProcessing_GlobalHWM,         TEXT("Mass Global Concurrency HWM"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(MassProcessing_ActiveChunks,      TEXT("Mass Active Chunks"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(MassProcessing_ExecutionChunkHWM, TEXT("Mass Execution Chunk HWM"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(MassProcessing_GlobalChunkHWM,    TEXT("Mass Global Chunk HWM"));
#endif

namespace UE::Mass
{
	//----------------------------------------------------------------------//
	//  FProcessingQueueAsyncCleanup
	//----------------------------------------------------------------------//
	struct FProcessingQueueAsyncCleanup
	{
		FProcessingQueueAsyncCleanup(FMassExecutionContext&& InExecutionContext,
			FMassEntityManager& InEntityManager,
			TFunction<void()> InOnDoneNotification,
			TFunction<void()> InOnPreCommandFlushNotification)
			: ExecutionContext(MoveTemp(InExecutionContext))
			, OnDoneNotification(MoveTemp(InOnDoneNotification))
			, OnPreCommandFlushNotification(MoveTemp(InOnPreCommandFlushNotification))
			, ProcessingScope(InEntityManager.NewProcessingScope())
		{}

		FMassExecutionContext ExecutionContext;
		TFunction<void()> OnDoneNotification;
		TFunction<void()> OnPreCommandFlushNotification;
		FMassEntityManager::FScopedProcessing ProcessingScope;
	};

	//----------------------------------------------------------------------//
	//  FProcessingQueue
	//----------------------------------------------------------------------//
	void FProcessingQueue::AddProcessor(TNotNull<UMassProcessor*> InProcessor)
	{
		Processors.AppendProcessor(*InProcessor);
		bProcessorListDirty = true;
	}

	void FProcessingQueue::AddProcessors(FMassRuntimePipeline& InPipeline)
	{
		Processors.AppendProcessors(InPipeline.GetMutableProcessors());
		bProcessorListDirty = true;
	}

	void FProcessingQueue::AddProcessors(TConstArrayView<UMassProcessor*> InProcessors)
	{
		for (UMassProcessor* Proc : InProcessors)
		{
			Processors.AppendProcessor(*Proc);
		}
		bProcessorListDirty = true;
	}

	void FProcessingQueue::SetProcessors(TConstArrayView<UMassProcessor*> InProcessors)
	{
		if (!bProcessorListDirty && Algo::Compare(Processors.GetProcessorsView(), InProcessors))
		{
			return;
		}

		Processors.Reset();
		AddProcessors(InProcessors);
		InitQueue();
	}

	void FProcessingQueue::Execute(FProcessingContext&& ProcessingContext, TFunction<void()> OnDoneNotification, ENamedThreads::Type CurrentThread, TFunction<void()> OnPreCommandFlushNotification)
	{
		const bool bWasBlocking = bBlockMainThread;
		bBlockMainThread = true;
		Execute_GetEventRef(MoveTemp(ProcessingContext), OnDoneNotification, CurrentThread, OnPreCommandFlushNotification);
		bBlockMainThread = bWasBlocking;
	}

	FGraphEventRef FProcessingQueue::Execute_GetEventRef(FProcessingContext&& ProcessingContext, TFunction<void()> OnDoneNotification, ENamedThreads::Type CurrentThread, TFunction<void()> OnPreCommandFlushNotification)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Mass FProcessingQueue::Execute");

#if WITH_EDITOR
		// The editor Mass environment relies on processing blocking the main thread to completion.
		// The game-side default of bBlockMainThread=false is not safe here, so force it on in editor builds.
		// Note: this also covers PIE since PIE runs in editor builds; that's an acceptable tradeoff for simplicity.
		bBlockMainThread = true;
#endif // WITH_EDITOR

		if (bProcessorListDirty)
		{
			InitQueue();
		}

		FMassExecutionContext ExecutionContext = MoveTemp(ProcessingContext).GetExecutionContext();
		FMassEntityManager& EntityManager = ExecutionContext.GetEntityManagerChecked();

		MainThreadProcessorCount = 0;
		RemainingProcessorIndices.Reset();
		RemainingMainThreadProcessorIndices.Reset();

		int32 QueueIndex = 0;

		for (FQueuedProcessor& QueuedProcessor : ProcessorQueue)
		{
			QueuedProcessor.PrepareForExecute();
#if MASS_DO_PARALLEL
			if (QueuedProcessor.bRequiresGameThread)
			{
				MainThreadProcessorCount++;
				RemainingMainThreadProcessorIndices.Add(QueueIndex++);
			}
			else
			{
				RemainingProcessorIndices.Add(QueueIndex++);
			}
#else
			MainThreadProcessorCount++;
			RemainingMainThreadProcessorIndices.Add(QueueIndex++);
#endif
		}

		PendingMainThreadProcessorCount.store(MainThreadProcessorCount);
		PendingProcessorCount.store(ProcessorQueue.Num());
		CompletedProcessorCount.store(0);
#if MASS_CONCURRENCY_STATS
		ActiveProcessorCount.store(0, std::memory_order_relaxed);
		ExecutionConcurrencyHWM.store(0, std::memory_order_relaxed);
		ActiveChunkCount.store(0, std::memory_order_relaxed);
		ExecutionChunkHWM.store(0, std::memory_order_relaxed);
#endif

#if MASS_DO_PARALLEL
		if (bBlockMainThread)
		{
			{
				FMassEntityManager::FScopedProcessing ProcessingScope = EntityManager.NewProcessingScope();

				UE::Tasks::FTask QueueRunner = UE::Tasks::Launch(TEXT("MassProcessingQueue Runner Task"),
					[this, &ExecutionContext]()
					{
						TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("MassProcessingQueue Async Runner Task");
						Run(ExecutionContext);
					},
					LowLevelTasks::ETaskPriority::Normal);

				if (MainThreadProcessorCount > 0)
				{
					UE::Tasks::FTask MainThreadRunner = UE::Tasks::Launch(TEXT("Mass Main Thread Processing Task"),
						[this, &ExecutionContext]()
						{
							TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("MassProcessingQueue Main-Thread Runner Task");
							RunMainThread(ExecutionContext);
						},
						UE::Tasks::ETaskPriority::Normal,
						UE::Tasks::EExtendedTaskPriority::Inline);

					MainThreadRunner.Wait();
				}

				QueueRunner.Wait();

				check(CompletedProcessorCount.load() == ProcessorQueue.Num());
			}

			FinalizeAsyncCleanup(ExecutionContext, OnDoneNotification, OnPreCommandFlushNotification);
			FGraphEventRef DummyEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(
				[]()
				{
					// Return a dummy event that is valid so the phase manager
					// doesn't take the bWorkRequested=false path.
				});

			return DummyEvent;
		}
		else
		{
			TSharedPtr<FProcessingQueueAsyncCleanup> State = MakeShared<FProcessingQueueAsyncCleanup>(
				MoveTemp(ExecutionContext), EntityManager,
				MoveTemp(OnDoneNotification), MoveTemp(OnPreCommandFlushNotification));

			FMassExecutionContext* StateExecutionContextPtr = &State->ExecutionContext;

			FGraphEventRef RunnerEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(
				[this, StateExecutionContextPtr]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("MassProcessingQueue Async Runner Task");
					Run(*StateExecutionContextPtr);
				},
				TStatId(), nullptr, ENamedThreads::AnyThread);

			if (MainThreadProcessorCount > 0)
			{
				UE::Tasks::FTask MainThreadRunner = UE::Tasks::Launch(TEXT("Mass Main Thread Processing Task"),
					[this, StateExecutionContextPtr]()
					{
						TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("MassProcessingQueue Main-Thread Runner Task");
						RunMainThread(*StateExecutionContextPtr);
					},
					UE::Tasks::ETaskPriority::Normal,
					UE::Tasks::EExtendedTaskPriority::Inline);

				MainThreadRunner.Wait();
			}

			// GT cleanup task, prereq'd on the async runner.
			FGraphEventArray CleanupPrereqs{ RunnerEvent };
			FGraphEventRef CleanupEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(
				[this, OwnedState = MoveTemp(State)]() mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("MassProcessingQueue GT Cleanup");

					checkf(CompletedProcessorCount.load() == ProcessorQueue.Num(),
						TEXT("Mass processing queue runners returned with %d of %d processors completed - a processor was lost, never dispatched, or finished without reporting"),
						CompletedProcessorCount.load(), ProcessorQueue.Num());

					FMassExecutionContext LocalExecutionContext = MoveTemp(OwnedState->ExecutionContext);
					TFunction<void()> LocalOnDoneNotification = MoveTemp(OwnedState->OnDoneNotification);
					TFunction<void()> LocalOnPreCommandFlushNotification = MoveTemp(OwnedState->OnPreCommandFlushNotification);

					OwnedState.Reset();

					FinalizeAsyncCleanup(LocalExecutionContext, LocalOnDoneNotification, LocalOnPreCommandFlushNotification);
				},
				TStatId(), &CleanupPrereqs, ENamedThreads::GameThread);

			return CleanupEvent;
		}
#else
		{
			FMassEntityManager::FScopedProcessing ProcessingScope = EntityManager.NewProcessingScope();
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("MassProcessingQueue Inline Runner");
			RunMainThread(ExecutionContext);
			checkf(CompletedProcessorCount.load() == ProcessorQueue.Num(),
				TEXT("Mass processing queue runners returned with %d of %d processors completed - a processor was lost, never dispatched, or finished without reporting"),
				CompletedProcessorCount.load(), ProcessorQueue.Num());
		}

		FinalizeAsyncCleanup(ExecutionContext, OnDoneNotification, OnPreCommandFlushNotification);
		FGraphEventRef DummyEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(
			[]()
			{
				// Return a dummy event that is valid so the phase manager
				// doesn't take the bWorkRequested=false path.
			});

		return DummyEvent;
#endif
	}

	void FProcessingQueue::FinalizeAsyncCleanup(FMassExecutionContext& ExecutionContext,
		const TFunction<void()>& OnDoneNotification,
		const TFunction<void()>& OnPreCommandFlushNotification)
	{
		FMassEntityManager& EntityManager = ExecutionContext.GetEntityManagerChecked();

		// There is a very small chance that the entity manager bookkeeping for the ProcessingScope
		// count hasn't settled by the time we get here.
		while (EntityManager.IsProcessing())
		{
			FPlatformProcess::YieldCycles(1);
		}

		//@todo: Command buffers need to be merged to sort them, but it would probably be faster to
		//	 sort them in place and process the commands directly. This would need a multi-container
		//	 sort method.
		if (&ExecutionContext.Defer() != &EntityManager.Defer())
		{
			ExecutionContext.Defer().MoveAppend(EntityManager.Defer());
		}

		FWriteScopeLock ScopeLock(CommandBuffersLock);
		while (CommandBufferPool.Num() > 0)
		{
			ExecutionContext.Defer().MoveAppend(*CommandBufferPool.Pop());
		}

		// this needs to happen after the ProcessingScope is destroyed:
		if (OnPreCommandFlushNotification)
		{
			// this will unlock observers so they execute when commands are flushed:
			OnPreCommandFlushNotification();
		}

		ExecutionContext.SetFlushDeferredCommands(true);
		ExecutionContext.FlushDeferred();

		CommandBufferPool.Reset();
		OnDoneNotification();
	}

	void FProcessingQueue::Reserve(int32 RequestorIndex)
	{
		FChangeListener Listener(QueueChangeTracker);

		for (;;)
		{
			if (TryReserve(RequestorIndex))
			{
				return;
			}
			Listener.WaitForChange();
		}
	}

	bool FProcessingQueue::TryReserve(int32 RequestorIndex)
	{
		return ActiveContentionTracker.TryActivate(RequestorIndex);
	}

	bool FProcessingQueue::ReleaseAndTryReserve(int32 IndexToRelease, int32 IndexToReserve)
	{
		// Releasing a contention slot can unblock another worker's TryReserve. Bump the change
		// tracker so waiters in FChangeListener::WaitForChange see the state change immediately
		// rather than relying on the WaitForChange timeout to retry.
		FChangeScope ReleaseScope(QueueChangeTracker);
		ActiveContentionTracker.Deactivate(IndexToRelease);
		return TryReserve(IndexToReserve);
	}

	void FProcessingQueue::ReleaseAndReserve(int32 IndexToRelease, int32 IndexToReserve)
	{
		if (ReleaseAndTryReserve(IndexToRelease, IndexToReserve))
		{
			return;
		}
		Reserve(IndexToReserve);
	}

	void FProcessingQueue::Release(int32 RequestorIndex)
	{
		// Bump the change tracker for the same reason as ReleaseAndTryReserve.
		FChangeScope ReleaseScope(QueueChangeTracker);
		ActiveContentionTracker.Deactivate(RequestorIndex);
	}

	FQueuedProcessor* FProcessingQueue::GetNextReadyProcessor(TArray<int32>& InOutRemainingIndices)
	{
		for (int32 QueueIndex : InOutRemainingIndices)
		{
			FQueuedProcessor& QueuedProcessor = ProcessorQueue[QueueIndex];
			if (QueuedProcessor.State == FQueuedProcessor::EState::Ready)
			{
				if (!ActiveContentionTracker.TryActivate(QueuedProcessor.ContentionIndex))
				{
					continue;
				}

				FChangeScope QueueChange(QueueChangeTracker);

				QueuedProcessor.State = FQueuedProcessor::EState::Running;

#if MASS_CONCURRENCY_STATS
				const int32 QueueActive = ++ActiveProcessorCount;
				const int32 GlobalActive = ++GActiveMassProcessorCount;
				TRACE_COUNTER_SET(MassProcessing_ActiveProcessors, GlobalActive);

				{
					int32 Prev = ExecutionConcurrencyHWM.load(std::memory_order_relaxed);
					while (QueueActive > Prev && !ExecutionConcurrencyHWM.compare_exchange_weak(Prev, QueueActive, std::memory_order_relaxed)) {}
				}
				TRACE_COUNTER_SET(MassProcessing_ExecutionHWM, ExecutionConcurrencyHWM.load(std::memory_order_relaxed));

				{
					int32 Prev = GGlobalMassConcurrencyHWM.load(std::memory_order_relaxed);
					while (QueueActive > Prev && !GGlobalMassConcurrencyHWM.compare_exchange_weak(Prev, QueueActive, std::memory_order_relaxed)) {}
				}
				TRACE_COUNTER_SET(MassProcessing_GlobalHWM, GGlobalMassConcurrencyHWM.load(std::memory_order_relaxed));
#endif

				--PendingProcessorCount;
				if (QueuedProcessor.bRequiresGameThread)
				{
					--PendingMainThreadProcessorCount;
				}
				// Ordering doesn't matter as this won't affect explicit ordering dependencies
				InOutRemainingIndices.RemoveSingleSwap(QueueIndex, EAllowShrinking::No);
				return &QueuedProcessor;
			}
		}
		return nullptr;
	}

	void FActiveChunkTracker::OnSlotExhausted()
	{
		static std::atomic<bool> bHasWarned{false};
		if (!bHasWarned.exchange(true, std::memory_order_relaxed))
		{
			UE_LOG(LogMass, Warning,
				TEXT("FActiveChunkTracker capacity (%d active chunks) reached. Caller will retry. ")
				TEXT("If hit regularly, raise via *.Build.cs file."),
				MaxActiveChunks);
		}
	}

	bool FProcessingQueue::TryActivateChunk(FQueuedChunk& InChunk, int32 DirectChunkContentionIndex)
	{
		// Calculated from "ProcessChunksBefore" or "ProcessChunksAfter"
		if (InChunk.RemainingDependencyCount > 0)
		{
			return false; 
		}

		const bool bActive = ActiveChunkContentionTracker.TryAcquireAccess(InChunk, DirectChunkContentionIndex);
#if MASS_CONCURRENCY_STATS
		if (bActive)
		{
			const int32 QueueActive = ++ActiveChunkCount;
			const int32 GlobalActive = ++GActiveMassChunkCount;
			TRACE_COUNTER_SET(MassProcessing_ActiveChunks, GlobalActive);

			{
				int32 Prev = ExecutionChunkHWM.load(std::memory_order_relaxed);
				while (QueueActive > Prev && !ExecutionChunkHWM.compare_exchange_weak(Prev, QueueActive, std::memory_order_relaxed)) {}
			}
			TRACE_COUNTER_SET(MassProcessing_ExecutionChunkHWM, ExecutionChunkHWM.load(std::memory_order_relaxed));

			{
				int32 Prev = GGlobalMassChunkHWM.load(std::memory_order_relaxed);
				while (QueueActive > Prev && !GGlobalMassChunkHWM.compare_exchange_weak(Prev, QueueActive, std::memory_order_relaxed)) {}
			}
			TRACE_COUNTER_SET(MassProcessing_GlobalChunkHWM, GGlobalMassChunkHWM.load(std::memory_order_relaxed));
		}
#endif
		return bActive;
	}

	void FProcessingQueue::DeactivateChunk(FQueuedChunk& InChunk)
	{
		// Bump the change tracker so ProcessChunks waiters keyed on chunk contention see the
		// release immediately, rather than waiting for the chunk worker lambda to return and
		// bump via its outer FChangeScope.
		FChangeScope ReleaseScope(QueueChangeTracker);
		ActiveChunkContentionTracker.ReleaseAccess(InChunk, InChunk.QueryInfo->ChunkContentionIndex);
#if MASS_CONCURRENCY_STATS
		--ActiveChunkCount;
		const int32 GlobalActive = --GActiveMassChunkCount;
		TRACE_COUNTER_SET(MassProcessing_ActiveChunks, GlobalActive);
#endif
	}

	void FProcessingQueue::ProcessorFinished(FQueuedProcessor& CompletedProcessor)
	{
		FChangeScope QueueChange(QueueChangeTracker);

		++CompletedProcessorCount;
#if MASS_CONCURRENCY_STATS
		--ActiveProcessorCount;
		const int32 GlobalActive = --GActiveMassProcessorCount;
		TRACE_COUNTER_SET(MassProcessing_ActiveProcessors, GlobalActive);
#endif
		CompletedProcessor.State = FQueuedProcessor::EState::Completed;

		if (CompletedProcessor.DependentProcessorIndices.Num() > 0)
		{
			for (int32 DependentProcessorIndex : CompletedProcessor.DependentProcessorIndices)
			{
				if (ProcessorQueue[DependentProcessorIndex].RemainingDependencyCount.fetch_sub(1) == 1)
				{
					check(ProcessorQueue[DependentProcessorIndex].State == FQueuedProcessor::EState::WaitingForDependencies);
					ProcessorQueue[DependentProcessorIndex].State = FQueuedProcessor::EState::Ready;
				}
			}
		}
		ActiveContentionTracker.Deactivate(CompletedProcessor.ContentionIndex);
	}

	void FProcessingQueue::MakeProcessorWorker(FMassExecutionContext& ExecutionContext, FQueuedProcessor* AssignedProcessor, bool bInLine)
	{
		if (bInLine)
		{
			if (AssignedProcessor)
			{
				FProcessorWorker ProcessorWorker(*this, ExecutionContext, true);
				ProcessorWorker.AssignProcessor(AssignedProcessor);
				ProcessorWorker.ExecuteProcessor();
			}
		}
		else
		{
			UE::Tasks::FTask WorkerTask = UE::Tasks::Launch(TEXT("Mass Processor Worker Task"),
				[this, &ExecutionContext, AssignedProcessor]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("Mass Processor Worker Task");
					FProcessorWorker ProcessorWorker(*this, ExecutionContext);
					if (AssignedProcessor)
					{
						ProcessorWorker.AssignProcessor(AssignedProcessor);
						ProcessorWorker.ExecuteProcessor();
					}
				},
				LowLevelTasks::ETaskPriority::Normal);

			// Nest the worker under the currently-executing task
			UE::Tasks::AddNested(WorkerTask);
		}
	}

	void FProcessingQueue::MakeChunkWorker(FQueuedChunk* AssignedChunk, bool bInline)
	{
		// No outer FChangeScope on QueueChangeTracker is needed here: ChunkDone's call to
		// DeactivateChunk self-announces, and ChunksRemaining is signalled via Task.Wait, not the
		// change tracker.
		if (bInline)
		{
			FChunkWorker ChunkWorker(*this, AssignedChunk, bInline);
			if (ChunkWorker.Chunk)
			{
				ChunkWorker.ExecuteChunk(true);
			}
		}
		else
		{
			UE::Tasks::FTask WorkerTask = UE::Tasks::Launch(TEXT("Mass Chunk Worker Task"),
				[this, AssignedChunk]()
				{
					FChunkWorker ChunkWorker(*this, AssignedChunk);
					if (ChunkWorker.Chunk)
					{
						ChunkWorker.ExecuteChunk(false);
					}
				}
				, UE::Tasks::ETaskPriority::Normal);

			// Store the task on the persistent FQueuedProcessor so ProcessChunks can Wait() on it.
			if (AssignedChunk && AssignedChunk->OwningProcessorWorker && AssignedChunk->OwningProcessorWorker->RunningProcessor)
			{
				AssignedChunk->OwningProcessorWorker->RunningProcessor->DispatchedChunkTasks.Add(MoveTemp(WorkerTask));
			}
		}
	}

	TSharedPtr<FMassCommandBuffer> FProcessingQueue::GetOrCreateCommandBuffer()
	{
		if (CommandBuffersLock.TryWriteLock())
		{
			if (CommandBufferPool.IsEmpty())
			{
				CommandBuffersLock.WriteUnlock();
				return MakeShared<FMassCommandBuffer>();
			}
			TSharedPtr<FMassCommandBuffer> CommandBuffer = CommandBufferPool.Pop();
			CommandBuffersLock.WriteUnlock();
			CommandBuffer->ForceUpdateCurrentThreadID();
			return CommandBuffer;
		}
		// if the lock fails, just make a new one instead of blocking
		// @todo: consider creating a pool of chunk workers each with an owned command buffer to avoid creating them excessively
		return MakeShared<FMassCommandBuffer>();
	}

	void FProcessingQueue::ReturnCommandBufferToPool(TSharedPtr<FMassCommandBuffer>& InCommandBuffer)
	{
		FWriteScopeLock ScopeLock(CommandBuffersLock);
		check(InCommandBuffer.IsValid());
		checkSlow(CommandBufferPool.Contains(InCommandBuffer) == false);
		CommandBufferPool.Add(InCommandBuffer);
	}

	void FProcessingQueue::RunMainThread(FMassExecutionContext& ExecutionContext)
	{
		int32 CountRemaining = 0;
		FChangeListener ChangeListener(QueueChangeTracker);
		do
		{
			FQueuedProcessor* QueuedProcessor = GetNextReadyProcessor(RemainingMainThreadProcessorIndices);
			if (QueuedProcessor)
			{
				MakeProcessorWorker(ExecutionContext, QueuedProcessor, true);
			}
#if MASS_DO_PARALLEL
			CountRemaining = PendingMainThreadProcessorCount.load();

			if (QueuedProcessor == nullptr && CountRemaining > 0)
			{
				// There's still game-thread work remaining, but it's blocked on async processors
				// Wait for queue changes to avoid hogging resources
				ChangeListener.WaitForChange();
			}
#else
			// In non-parallel builds all processors run on the main thread, so count everything
			CountRemaining = PendingProcessorCount.load();
#endif
		} while (CountRemaining > 0);
	}

	void FProcessingQueue::Run(FMassExecutionContext& ExecutionContext)
	{
		FChangeListener ChangeListener(QueueChangeTracker);
		for (;;)
		{
			// Dispatch all currently ready processors
			while (PendingProcessorCount.load() > 0)
			{
				FQueuedProcessor* QueuedProcessor = GetNextReadyProcessor(RemainingProcessorIndices);
				if (!QueuedProcessor)
				{
					break;
				}
				MakeProcessorWorker(ExecutionContext, QueuedProcessor);
			}

			if (IsComplete())
			{
				break;
			}
			ChangeListener.WaitForChange();
		}
	}

	bool FProcessingQueue::IsComplete() const
	{
		return CompletedProcessorCount.load() == ProcessorQueue.Num();
	}

	int32 FProcessingQueue::ComputeGraphDepthRecursive(int32 ProcessorIndex, TArray<int32>& BestDepth)
	{
		if (BestDepth[ProcessorIndex] != -1)
		{
			return BestDepth[ProcessorIndex];
		}

		int32 MaxChildDepth = 0;

		// Find everyone who depends on THIS processor (ExecuteAfter this)
		for (int32 DependentIndex : ProcessorQueue[ProcessorIndex].DependentProcessorIndices)
		{
			int32 ChildDepth = ComputeGraphDepthRecursive(DependentIndex, BestDepth);
			if (ChildDepth > MaxChildDepth)
			{
				MaxChildDepth = ChildDepth;
			}
		}

		int32 MyDepth = MaxChildDepth + 1; // I am 1 deeper than my deepest dependent
		BestDepth[ProcessorIndex] = MyDepth;
		return MyDepth;
	}

	void FProcessingQueue::InitializeExplicitDependencies()
	{
		// Reset and build base maps
		TMap<FName, TArray<int32>> ClassToIndices; // class name -> processor indices
		TMap<FName, TArray<int32>> GroupToIndices; // group name -> processor indices

		ClassToIndices.Reserve(ProcessorQueue.Num());
		GroupToIndices.Reserve(ProcessorQueue.Num());

		for (int32 ProcessorIndex = 0; ProcessorIndex < ProcessorQueue.Num(); ++ProcessorIndex)
		{
			FQueuedProcessor& QueuedProcessor = ProcessorQueue[ProcessorIndex];
			QueuedProcessor.DependentProcessorIndices.Reset();
			QueuedProcessor.DependencyCount = 0;
			const FName ClassName = QueuedProcessor.Processor->GetClass()->GetFName();
			ClassToIndices.FindOrAdd(ClassName).Add(ProcessorIndex);

			const FName GroupName = QueuedProcessor.Processor->GetExecutionOrder().ExecuteInGroup;
			if (!GroupName.IsNone())
			{
				GroupToIndices.FindOrAdd(GroupName).Add(ProcessorIndex);
			}
		}

		// Helper: add a dependency on From to To meaning "To waits for From"
		auto AddEdge = [this](int32 FromIdx, int32 ToIdx)
			{
				if (FromIdx == INDEX_NONE || ToIdx == INDEX_NONE || FromIdx == ToIdx)
				{
					return;
				}
				FQueuedProcessor& From = ProcessorQueue[FromIdx];
				FQueuedProcessor& To = ProcessorQueue[ToIdx];

				if(!From.DependentProcessorIndices.Contains(ToIdx))
				{
					From.DependentProcessorIndices.Add(ToIdx);
					++To.DependencyCount;
				}
			};

		// Helper: resolve a name to a set of processor indices
		auto GetIndicesForName = [&GroupToIndices, &ClassToIndices](const FName& Name) -> TSet<int32>
			{
				TSet<int32> Indices;

				if (const TArray<int32>* GroupMatches = GroupToIndices.Find(Name))
				{
					for (int32 Idx : *GroupMatches)
					{
						Indices.Add(Idx);
					}
				}
				if (const TArray<int32>* ClassMatches = ClassToIndices.Find(Name))
				{
					for (int32 Idx : *ClassMatches)
					{
						Indices.Add(Idx);
					}
				}
				return Indices;
			};

		for (int32 ProcessorIndex = 0; ProcessorIndex < ProcessorQueue.Num(); ++ProcessorIndex)
		{
			UMassProcessor* const Processor = ProcessorQueue[ProcessorIndex].Processor;
			
			const FMassProcessorExecutionOrder& Order = Processor->GetExecutionOrder();
			for (const FName& BeforeName : Order.ExecuteBefore)
			{
				const TSet<int32> BeforeIndices = GetIndicesForName(BeforeName);
				for (int32 BeforeIndex : BeforeIndices)
				{
					AddEdge(ProcessorIndex, BeforeIndex);
				}
			}

			for (const FName& AfterName : Order.ExecuteAfter)
			{
				const TSet<int32> AfterIndices = GetIndicesForName(AfterName);
				for (int32 AfterIndex : AfterIndices)
				{
					AddEdge(AfterIndex, ProcessorIndex);
				}
			}
		}
		if (CheckForValidDependencies())
		{
			// @todo: distribute workers favoring higher graph depth
			// Calculate Graph Depth for each processor
			TArray<int32> BestDepth;
			BestDepth.Init(-1, ProcessorQueue.Num());

			for (int32 i = 0; i < ProcessorQueue.Num(); ++i)
			{
				ProcessorQueue[i].GraphDepth = ComputeGraphDepthRecursive(i, BestDepth);
			}
		}
	}

	bool FProcessingQueue::CheckForValidDependencies()
	{
		const int32 NumProcessors = ProcessorQueue.Num();
		if (NumProcessors == 0)
		{
			return true;
		}

		TQueue<int32> Queue;
		for (int32 ProcessorIndex = 0; ProcessorIndex < NumProcessors; ++ProcessorIndex)
		{
			ProcessorQueue[ProcessorIndex].PrepareForExecute();
			if (ProcessorQueue[ProcessorIndex].RemainingDependencyCount.load() == 0)
			{
				Queue.Enqueue(ProcessorIndex);
			}
		}

		int32 Processed = 0;
		int32 Index;
		while (Queue.Dequeue(Index))
		{
			++Processed;

			FQueuedProcessor& QueuedProcessor = ProcessorQueue[Index];
			for (int32 DepIndex : QueuedProcessor.DependentProcessorIndices)
			{
				FQueuedProcessor& Dep = ProcessorQueue[DepIndex];
				int32 NewCount = --Dep.RemainingDependencyCount;
				if (NewCount == 0)
				{
					Queue.Enqueue(DepIndex);
				}
			}
		}

		checkf(Processed == NumProcessors, TEXT("Explicit dependency cycle detected."));
		return Processed == NumProcessors;
	}

	void FProcessingQueue::InitQueue()
	{
		bProcessorListDirty = false;
		const int32 ProcessorCount = Processors.Num();
		// todo: sort processors
		int32 GlobalContendersCount = 0;
		int32 DirectContendersCount = 0;
		ProcessorQueue.Reset(ProcessorCount);
		RemainingProcessorIndices.Reserve(ProcessorCount);

		for (UMassProcessor* Processor : Processors.GetProcessorsView())
		{
			FQueuedProcessor& QueuedProcessor = ProcessorQueue.Emplace_GetRef(Processor);
			QueuedProcessor.ContentionIndex = GlobalContendersCount++;
			for (int32 QueryIndex = 0; QueryIndex < QueuedProcessor.QueryCount; ++QueryIndex)
			{
				QueuedProcessor.QueryInfos[QueryIndex].QueryContentionIndex = GlobalContendersCount++;
				QueuedProcessor.QueryInfos[QueryIndex].ChunkGlobalContentionIndex = GlobalContendersCount++;
				QueuedProcessor.QueryInfos[QueryIndex].ChunkContentionIndex = DirectContendersCount++;
			}
			QueuedProcessor.InitThreadRequirement();
		}

		InitializeExplicitDependencies();


		ContentionMatrix.Init(GlobalContendersCount);
		ActiveContentionTracker.Init();

		ChunkContentionMatrix.Init(DirectContendersCount);
		ActiveChunkContentionTracker.Init(ChunkContentionMatrix);

		for (int32 ProcessorIndexA = 0; ProcessorIndexA < ProcessorCount - 1; ++ProcessorIndexA)
		{
			FQueuedProcessor& QueuedProcessorA = ProcessorQueue[ProcessorIndexA];

			for (int32 ProcessorIndexB = ProcessorIndexA + 1; ProcessorIndexB < ProcessorCount; ++ProcessorIndexB)
			{
				FQueuedProcessor& QueuedProcessorB = ProcessorQueue[ProcessorIndexB];

				// init mutual contention for ProcessorA vs ProcessorB
				QueuedProcessorA.InitWriteContention(ContentionMatrix, QueuedProcessorB);

				for (int32 QueryIndexB = 0; QueryIndexB < QueuedProcessorB.QueryCount; ++QueryIndexB)
				{
					// init mutual contention for ProcessorA vs each query owned by ProcessorB
					QueuedProcessorB.InitWriteContention(ContentionMatrix, QueryIndexB, QueuedProcessorA);
				}

				for (int32 QueryIndexA = 0; QueryIndexA < QueuedProcessorA.QueryCount; ++QueryIndexA)
				{
					// init mutual contention for ProcessorB vs each query owned by ProcessorA
					QueuedProcessorA.InitWriteContention(ContentionMatrix, QueryIndexA, QueuedProcessorB);
					for (int32 QueryIndexB = 0; QueryIndexB < QueuedProcessorB.QueryCount; ++QueryIndexB)
					{
						// init mutual cross-chunk contention for each query owned by ProcessorB vs each query owned by ProcessorA
						// "cross-chunk contention" means there's contention even if the chunks are different (indirect access, subsytems, etc)
						QueuedProcessorA.InitCrossChunkWriteContention(ContentionMatrix, QueryIndexA, QueuedProcessorB, QueryIndexB);

						// init mutual chunk contention for each query owned by ProcessorB vs each query owned by ProcessorA
						// "chunk contention" means there's contention only within the access for a given chunk (direct access, composition changes, etc)
						QueuedProcessorA.InitChunkWriteContention(ChunkContentionMatrix, QueryIndexA, QueuedProcessorB, QueryIndexB);
					}
				}
			}
		}
	}
} // namespace UE::Mass

#undef MASS_PROCESSING_QUEUE_LOGGING
#undef REUSE_CHUNK_QUEUE_ENTRIES
#undef LOCTEXT_NAMESPACE