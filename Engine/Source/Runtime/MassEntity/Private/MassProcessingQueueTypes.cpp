// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassProcessingQueueTypes.h"
#include "MassProcessingTypes.h"
#include "MassProcessingQueue.h"
#include "MassProcessor.h"
#include "MassTypeManager.h"
#include "MassCommandBuffer.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"
#include "Logging/MessageLog.h"
#include "MassArchetypeData.h"
#include "Containers/BitArray.h"
#include "MassEntityQuery.h"

#define LOCTEXT_NAMESPACE "MassProcessingQueueTypes"

#define MASS_PROCESSING_QUEUE_LOGGING 0

namespace UE::Mass::Tweakables
{
	bool bFullLogging = false;
	bool bLockLogging = false;

	// Do not modify after FProcessingQueue::Init has been called.
	bool bForceInlineProcessorExecution = true;

	FAutoConsoleVariableRef ProcessingQueueTypesCVars[] = {
		{TEXT("mass.ProcessingQueue.FullLogging"), bFullLogging, TEXT("Log Everything...")},
		{TEXT("mass.ProcessingQueue.LockLogging"), bLockLogging, TEXT("Log All lock operations...")},
		{TEXT("mass.ForceInlineProcessorExecution"), bForceInlineProcessorExecution,
			TEXT("When true force Mass processors to process chunks inline on the processor's thread."),
			ECVF_ReadOnly},
	};
} // namespace UE::Mass::Tweakables

static inline const TCHAR* MassThreadTag(bool bMain)
{
	return bMain ? TEXT("Main") : TEXT("Async");
}

static inline const TCHAR* MassInlineTag(bool bInline)
{
	return bInline ? TEXT("Inline") : TEXT("Async");
}

#if MASS_PROCESSING_QUEUE_LOGGING
#define LOG_PROC_WORKER(Fmt, ...) \
	UE_CLOG(UE::Mass::Tweakables::bFullLogging, LogMass, VeryVerbose, TEXT("[Mass][FProcessorWorker][%s][TID=%u][%p][L%u] " Fmt), \
		MassThreadTag(bInlineExecution), FPlatformTLS::GetCurrentThreadId(), this, __LINE__, ##__VA_ARGS__) \

#define LOG_PROC_WORKER_CHUNK(ChunkPtr, Fmt, ...) \
	UE_CLOG(UE::Mass::Tweakables::bFullLogging, LogMass, VeryVerbose, TEXT("[Mass][FProcessorWorker][%s][TID=%u][%p][Chunk=%p][SourceQuery=%p][L%u] " Fmt), \
		MassThreadTag(bInlineExecution), FPlatformTLS::GetCurrentThreadId(), this, ChunkPtr, ChunkPtr->Params.SourceQuery, __LINE__, ##__VA_ARGS__)

#define LOG_CHUNK_WORKER(Fmt, ...) \
	UE_CLOG(UE::Mass::Tweakables::bFullLogging, LogMass, VeryVerbose, TEXT("[Mass][ChunkWorker][TID=%u][%p][L%u] " Fmt), \
		FPlatformTLS::GetCurrentThreadId(), this, __LINE__, ##__VA_ARGS__)

#define LOG_CHUNK_WORKER_CHUNK(ChunkPtr, Fmt, ...) \
	UE_CLOG(UE::Mass::Tweakables::bFullLogging, LogMass, VeryVerbose, TEXT("[Mass][ChunkWorker][TID=%u][%p][Chunk=%p][SourceQuery=%p][L%u] " Fmt), \
		FPlatformTLS::GetCurrentThreadId(), this, ChunkPtr, ChunkPtr->Params.SourceQuery, __LINE__, ##__VA_ARGS__)

#else //MASS_PROCESSING_QUEUE_LOGGING
#define LOG_PROC_WORKER(...) \
	do {} while (0)
#define LOG_PROC_WORKER_CHUNK(...) \
	do {} while (0)
#define LOG_CHUNK_WORKER(...) \
	do {} while (0)
#define LOG_CHUNK_WORKER_CHUNK(...) \
	do {} while (0)

#endif //MASS_PROCESSING_QUEUE_LOGGING

#define CHECK_THREAD_CONSISTENCY(bRequireGameThread) \
	checkfSlow( \
		!(bRequireGameThread && !IsInGameThread()), \
		TEXT("Thread consistency error: GameThread is required, but current thread is not the game thread!") \
	)

namespace UE::Mass
{
	// Enum -> text helpers
	static inline const TCHAR* LexToString(UE::Mass::FQueuedProcessor::EState S)
	{
		using E = UE::Mass::FQueuedProcessor::EState;
		switch (S)
		{
		case E::WaitingForDependencies:
			return TEXT("WaitingForDependencies");
		case E::Ready:
			return TEXT("Ready");
		case E::Running:
			return TEXT("Active");
		case E::Completed:
			return TEXT("Complete");
		case E::MAX:
			return TEXT("MAX");
		default:
			return TEXT("<Unknown>");
		}
	}

	static inline const TCHAR* LexToString(UE::Mass::FQueuedChunk::EState S)
	{
		using E = UE::Mass::FQueuedChunk::EState;
		switch (S)
		{
		case E::Pending:
			return TEXT("Pending");
		case E::Running:
			return TEXT("Running");
		case E::Complete:
			return TEXT("Complete");
		case E::MAX:
			return TEXT("MAX");
		default:
			return TEXT("<Unknown>");
		}
	}

	//----------------------------------------------------------------------//
	//  FQueuedProcessor
	//----------------------------------------------------------------------//
	FQueuedProcessor::FQueuedProcessor(TNotNull<UMassProcessor*> InProcessor)
		: Processor(InProcessor)
	{
		bAggregateRequirements = Processor->bAggregateRequirements;
		QueryCount = Processor->OwnedQueries.Num();

		for (const FElementBitSet& Composition : Processor->GetProcessorEntityCreationRequirements().GetCompositions())
		{
			DataAccess.IndirectWritesElements += Composition;
		}

		DataAccess.WritesSubsystems = Processor->GetProcessorRequirements().GetRequiredMutableSubsystems();
		DataAccess.ReadsSubsystems = Processor->GetProcessorRequirements().GetRequiredConstSubsystems() + DataAccess.WritesSubsystems;

		QueryInfos.SetNum(QueryCount);

		for (int32 QueryIndex = 0; QueryIndex < QueryCount; ++QueryIndex)
		{
			QueryInfos[QueryIndex].SourceQuery = Processor->OwnedQueries[QueryIndex];
			QueryInfos[QueryIndex].DataAccess.SetFragmentWriteRequirements(*Processor->OwnedQueries[QueryIndex]);
			QueryInfos[QueryIndex].DataAccess.IndirectWritesElements = Processor->OwnedQueries[QueryIndex]->GetIndirectReadWriteFragments();
			QueryInfos[QueryIndex].DataAccess.IndirectReadsElements = Processor->OwnedQueries[QueryIndex]->GetIndirectReadOnlyFragments() + QueryInfos[QueryIndex].DataAccess.IndirectWritesElements;
			QueryInfos[QueryIndex].DataAccess.WritesSubsystems = Processor->OwnedQueries[QueryIndex]->GetRequiredMutableSubsystems();
			QueryInfos[QueryIndex].DataAccess.ReadsSubsystems = Processor->OwnedQueries[QueryIndex]->GetRequiredConstSubsystems() + QueryInfos[QueryIndex].DataAccess.WritesSubsystems;

			if (bAggregateRequirements)
			{
				DataAccess.Add(QueryInfos[QueryIndex].DataAccess);
			}
		}

		if (bAggregateRequirements)
		{
			for (FQueryInfo& Info : QueryInfos)
			{
				Info.DataAccess.Add(DataAccess);
			}
		}
	}

	void FQueuedProcessor::PrepareForExecute()
	{
		RemainingDependencyCount.store(DependencyCount);
		State = DependencyCount == 0 ? EState::Ready : EState::WaitingForDependencies;
		CurrentQueryIndex = QueryCount > 0 ? 0 : INDEX_NONE;
	}

	const FQueryInfo& FQueuedProcessor::GetCurrentQueryInfo() const
	{
		return QueryInfos[CurrentQueryIndex];
	}

	void FQueuedProcessor::InitThreadRequirement()
	{
		bRequiresGameThread = Processor->DoesRequireGameThreadExecution() || Processor->ProcessorRequirements.DoesRequireGameThreadExecution();

		for (int QueryIndex = 0; QueryIndex < Processor->OwnedQueries.Num(); ++QueryIndex)
		{
			FQueryInfo& Info = QueryInfos[QueryIndex];
			FMassEntityQuery& Query = *(Processor->OwnedQueries[QueryIndex]);

			Info.bChunksRequireGameThread = Query.DoesRequireGameThreadExecution();
			if (bAggregateThreadRequirements)
			{
				bRequiresGameThread |= Info.bChunksRequireGameThread;
			}
		}
		if (bAggregateThreadRequirements)
		{
			for (FQueryInfo& Info : QueryInfos)
			{
				Info.bChunksRequireGameThread = bRequiresGameThread;
			}
		}

	}

	void FQueuedProcessor::InitWriteContention(FContentionMatrix& ContentionMatrix, FQueuedProcessor& Other)
	{
		if (Processor == Other.Processor)
		{
			return;
		}

		if (DataAccess.SubsystemsCollide(Other.DataAccess))
		{
			ContentionMatrix.SetMutual(ContentionIndex, Other.ContentionIndex);
			return;
		}

		// find created entity collisions:
		if (Processor->GetProcessorEntityCreationRequirements().HasAny(Other.Processor->GetProcessorEntityCreationRequirements()))
		{
			ContentionMatrix.SetMutual(ContentionIndex, Other.ContentionIndex);
			return;
		}
	}

	void FQueuedProcessor::InitWriteContention(FContentionMatrix& ContentionMatrix, int32 MyQueryIndex, FQueuedProcessor& Other)
	{
		FMassEntityQuery* Query = Processor->OwnedQueries[MyQueryIndex];

		// Do the processor and query have a write collision on any subsystems?
		if (QueryInfos[MyQueryIndex].DataAccess.SubsystemsCollide(Other.DataAccess))
		{
			// Subsystem access collision applies to processing all chunks, but not running the query.
			ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkGlobalContentionIndex, Other.ContentionIndex);
			return;
		}

		// Can the processor create entities the query would find?
		for (const FMassElementBitSet& Composition : Other.Processor->GetProcessorEntityCreationRequirements().GetCompositions())
		{
			if (Query->DoesArchetypeMatchRequirements(Composition))
			{
				// This means the query can not run, and its chunks can not process while that processor is active.
				ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkGlobalContentionIndex, Other.ContentionIndex);
				return;
			}
		}

		// Do the query and processor both specify an identical creation requirement?:
		if (Other.Processor->GetProcessorEntityCreationRequirements().HasAny(Query->GetEntityCreationRequirements()))
		{
			// If they both create entities of the exact same archetype then they contend on that FMassArchetypeData.
			ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkGlobalContentionIndex, Other.ContentionIndex);
			return;
		}
	}

	void FQueuedProcessor::InitCrossChunkWriteContention(FContentionMatrix& ContentionMatrix, int32 MyQueryIndex, FQueuedProcessor& Other, int32 OtherQueryIndex)
	{
		FMassEntityQuery* QueryA = Processor->OwnedQueries[MyQueryIndex];
		FMassEntityQuery* QueryB = Other.Processor->OwnedQueries[OtherQueryIndex];
		const FDataAccess& DataA = QueryInfos[MyQueryIndex].DataAccess;
		const FDataAccess& DataB = Other.QueryInfos[OtherQueryIndex].DataAccess;

		// Do the processor and query have a write collision on any subsystems?
		if (DataA.SubsystemsCollide(DataB))
		{
			// Subsystem write collisions are global (cross-chunk), but do not block running the query.
			ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkGlobalContentionIndex, Other.QueryInfos[OtherQueryIndex].ChunkGlobalContentionIndex);
			return;
		}

		// Can QueryA create entities QueryB would find?
		for (const FMassElementBitSet& Composition : QueryA->GetEntityCreationRequirements().GetCompositions())
		{
			if (QueryB->DoesArchetypeMatchRequirements(Composition))
			{
				// Entity creation and composition changes are global (cross-chunk) contention.
				ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkGlobalContentionIndex, Other.QueryInfos[OtherQueryIndex].ChunkGlobalContentionIndex);
				return;
			}
		}

		// Can QueryB create entities QueryA would find?
		for (const FMassElementBitSet& Composition : QueryB->GetEntityCreationRequirements().GetCompositions())
		{
			if (QueryA->DoesArchetypeMatchRequirements(Composition))
			{
				// Entity creation and composition changes are global (cross-chunk) contention.
				ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkGlobalContentionIndex, Other.QueryInfos[OtherQueryIndex].ChunkGlobalContentionIndex);
				return;
			}
		}

		// Do QueryA and QueryB specify an identical entity creation requirement?:
		if (QueryA->GetEntityCreationRequirements().HasAny(QueryB->GetEntityCreationRequirements()))
		{
			// They both create entities of an identical archetype which is cross-chunk contention.
			ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkGlobalContentionIndex, Other.QueryInfos[OtherQueryIndex].ChunkGlobalContentionIndex);
			return;
		}

		// indirect vs indirect fragment access collision:
		if (DataA.IndirectWritesElements.HasAny(DataB.IndirectWritesElements)
			|| DataA.IndirectWritesElements.HasAny(DataB.IndirectReadsElements)
			|| DataB.IndirectWritesElements.HasAny(DataA.IndirectReadsElements))
		{
			// Indirect contention is cross-chunk.
			ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkGlobalContentionIndex, Other.QueryInfos[OtherQueryIndex].ChunkGlobalContentionIndex);
			return;
		}

		// indirect writes vs direct reads or writes collision:
		if (DataA.IndirectWritesElements.HasAny(DataB.DirectReadsElements)
			|| DataB.IndirectWritesElements.HasAny(DataA.DirectReadsElements)
			|| DataA.IndirectWritesElements.HasAny(DataB.DirectWritesElements)
			|| DataB.IndirectWritesElements.HasAny(DataA.DirectWritesElements))
		{
			// Any indirect contention is cross-chunk.
			ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkGlobalContentionIndex, Other.QueryInfos[OtherQueryIndex].ChunkGlobalContentionIndex);
			return;
		}

		// direct writes vs indirect reads collision:
		if (DataA.DirectWritesElements.HasAny(DataB.IndirectReadsElements)
			|| DataB.DirectWritesElements.HasAny(DataA.IndirectReadsElements))
		{
			// Any indirect contention is cross-chunk.
			ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkGlobalContentionIndex, Other.QueryInfos[OtherQueryIndex].ChunkGlobalContentionIndex);
			return;
		}

		// direct vs direct fragment access is promoted to global contention if we're executing in-line
		if (UE::Mass::Tweakables::bForceInlineProcessorExecution)
		{
			if (DataA.DirectWritesElements.HasAny(DataB.DirectReadsElements)
				|| DataB.DirectWritesElements.HasAny(DataA.DirectReadsElements))
			{
				ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkGlobalContentionIndex, Other.QueryInfos[OtherQueryIndex].ChunkGlobalContentionIndex);
				return;
			}
		}

		// note: We don't otherwise test direct vs direct fragment access because those can only collide within a single chunk.
		//       See InitChunkWriteContention.
	}

	void FQueuedProcessor::InitChunkWriteContention(FContentionMatrix& ContentionMatrix, int32 MyQueryIndex, FQueuedProcessor& Other, int32 OtherQueryIndex)
	{
		// note: we don't care about archetype collisions here because we're only testing possible collisions within the same chunk so we know the archetypes match

		const FDataAccess& DataA = QueryInfos[MyQueryIndex].DataAccess;
		const FDataAccess& DataB = Other.QueryInfos[OtherQueryIndex].DataAccess;

		if (DataA.DirectWritesElements.HasAny(DataB.DirectReadsElements)
			|| DataB.DirectWritesElements.HasAny(DataA.DirectReadsElements))
		{
			ContentionMatrix.SetMutual(QueryInfos[MyQueryIndex].ChunkContentionIndex, Other.QueryInfos[OtherQueryIndex].ChunkContentionIndex);
			return;
		}

		// all other contention should be caught by the cross-chunk contention check
	}

	//----------------------------------------------------------------------//
	//   FDataAccess
	//----------------------------------------------------------------------//
	void FDataAccess::Add(const FDataAccess& Other)
	{
		DirectReadsElements += Other.DirectReadsElements;
		DirectWritesElements += Other.DirectWritesElements;
		IndirectReadsElements += Other.IndirectReadsElements;
		IndirectWritesElements += Other.IndirectWritesElements;
		ReadsSubsystems += Other.ReadsSubsystems;
		WritesSubsystems += Other.WritesSubsystems;
	}

	bool FDataAccess::SubsystemsCollide(const FDataAccess& Other) const
	{
		// find mutable subsystem collisions:
		return WritesSubsystems.HasAny(Other.WritesSubsystems)
			|| WritesSubsystems.HasAny(Other.ReadsSubsystems)
			|| Other.WritesSubsystems.HasAny(ReadsSubsystems);
	}

	bool FDataAccess::CollidesWithIndirect(const FDataAccess& Other) const
	{
		// indirect vs indirect fragment access collision:
		const bool bIvI = IndirectWritesElements.HasAny(Other.IndirectWritesElements)
			|| IndirectWritesElements.HasAny(Other.IndirectReadsElements)
			|| Other.IndirectWritesElements.HasAny(IndirectReadsElements);

		// indirect writes vs direct reads or writes collision:
		const bool bIvD = IndirectWritesElements.HasAny(Other.DirectReadsElements)
			|| Other.IndirectWritesElements.HasAny(DirectReadsElements)
			|| IndirectWritesElements.HasAny(Other.DirectWritesElements)
			|| Other.IndirectWritesElements.HasAny(DirectWritesElements);

		// direct writes vs indirect reads collision:
		const bool bDvI = DirectWritesElements.HasAny(Other.IndirectReadsElements)
			|| Other.DirectWritesElements.HasAny(IndirectReadsElements);

		return bIvI || bIvD || bDvI;
	}

	bool FDataAccess::CollidesWithDirect(const FDataAccess& Other) const
	{
		const bool bDvD = DirectWritesElements.HasAny(Other.DirectReadsElements)
			|| DirectWritesElements.HasAny(Other.DirectWritesElements)
			|| Other.DirectWritesElements.HasAny(DirectReadsElements);
		return bDvD;
	}

	void FDataAccess::SetFragmentWriteRequirements(const FMassFragmentRequirements& FragmentRequirements)
	{
		auto ProcessFragmentRequirements = [this](TConstArrayView<FMassFragmentRequirementDescription> Descriptions)
			{
				for (const FMassFragmentRequirementDescription& Description : Descriptions)
				{
					if (Description.StructType && Description.Presence != EMassFragmentPresence::None)
					{
						switch (Description.AccessMode)
						{
						case EMassFragmentAccess::ReadWrite:
							DirectWritesElements.Add(Description.StructType);
							DirectReadsElements.Add(Description.StructType);
							break;
						case EMassFragmentAccess::ReadOnly:
							DirectReadsElements.Add(Description.StructType);
							break;
						default:
							break;
						}
					}
				}
			};

		ProcessFragmentRequirements(FragmentRequirements.GetFragmentRequirements());
		ProcessFragmentRequirements(FragmentRequirements.GetChunkFragmentRequirements());
		ProcessFragmentRequirements(FragmentRequirements.GetConstSharedFragmentRequirements());
		ProcessFragmentRequirements(FragmentRequirements.GetSharedFragmentRequirements());
	}

	FProcessorWorker::FProcessorWorker(FProcessingQueue& InQueue, const FMassExecutionContext& InExecutionContext, bool bInMainThread)
		: OwningQueue(InQueue)
		, GlobalExecutionContext(InExecutionContext)
		, bInlineExecution(bInMainThread)
		, CommandBuffer(OwningQueue.GetOrCreateCommandBuffer())
	{
		CHECK_THREAD_CONSISTENCY(bInlineExecution);
		LOG_PROC_WORKER(TEXT("Creating FProcessorWorker (%p): bInlineExecution: %s"), this, bInlineExecution ? TEXT("true") : TEXT("false"));
	}

	FProcessorWorker::~FProcessorWorker()
	{
		LOG_PROC_WORKER(TEXT("Destroying FProcessorWorker"));
		Cleanup();
	}

	void FProcessorWorker::Cleanup()
	{
		if (CommandBuffer.IsValid())
		{
			OwningQueue.ReturnCommandBufferToPool(CommandBuffer);
			CommandBuffer.Reset();
		}
	}

	FChunkWorker::FChunkWorker(FProcessingQueue& InQueue, FQueuedChunk* InChunk, bool bInUseContextCommandBuffer)
		: OwningQueue(InQueue)
		, Chunk(InChunk)
		, bUseContextCommandBuffer(bInUseContextCommandBuffer)
	{
		LOG_CHUNK_WORKER(TEXT("Creating FChunkWorker"));
	}

	FChunkWorker::~FChunkWorker()
	{
		Cleanup();
	}

	void FChunkWorker::Cleanup()
	{
		if (CommandBuffer.IsValid())
		{
			OwningQueue.ReturnCommandBufferToPool(CommandBuffer);
		}
	}

	TSharedPtr<FMassCommandBuffer>& FChunkWorker::GetOrCreateCommandBuffer()
	{
		if (!CommandBuffer.IsValid())
		{
			CommandBuffer = OwningQueue.GetOrCreateCommandBuffer();
		}
		CommandBuffer->ForceUpdateCurrentThreadID();
		return CommandBuffer;
	}

	void FChunkWorker::ExecuteChunk(bool bInlineExecution)
	{
		LOG_CHUNK_WORKER_CHUNK(Chunk, TEXT("ExecuteChunk"));
		check(Chunk);
		CHECK_THREAD_CONSISTENCY(Chunk->QueryInfo->bChunksRequireGameThread);

		static const FMassQueryRequirementIndicesMapping EmptyMapping;

		FMassExecutionContext& SourceContext = *Chunk->Params.SourceContext;

		if (bUseContextCommandBuffer && !Chunk->Params.bParallelExecution)
		{
			if (!bInlineExecution)
			{
				SourceContext.Defer().ForceUpdateCurrentThreadID();
			}
			LOG_CHUNK_WORKER_CHUNK(Chunk, TEXT("Executing chunk serially"));
			Chunk->Params.Archetype->ExecutionFunctionForChunk(
				SourceContext,
				*Chunk->Params.ExecuteFunction,
				Chunk->Params.RequirementMapping ? *Chunk->Params.RequirementMapping : EmptyMapping,
				Chunk->Params.EntityRange,
				*Chunk->Params.ChunkCondition);
		}
		else
		{
			LOG_CHUNK_WORKER_CHUNK(Chunk, TEXT("Executing chunk in parallel"));

			// need to create a local execution context for parallel execution
			FMassExecutionContext LocalExecutionContext(SourceContext);
			LocalExecutionContext.PushQuery(*Chunk->Params.SourceQuery);
			LocalExecutionContext.SetDeferredCommandBuffer(GetOrCreateCommandBuffer());
			Chunk->Params.Archetype->ExecutionFunctionForChunk(
				LocalExecutionContext,
				*Chunk->Params.ExecuteFunction,
				Chunk->Params.RequirementMapping ? *Chunk->Params.RequirementMapping : EmptyMapping,
				Chunk->Params.EntityRange,
				*Chunk->Params.ChunkCondition);

			// @todo: this is pointless and just prevents an ensure in the context destructor...
			// there's nothing inherently unsafe about a context being destroyed while it had a query on the stack
			// all the state that cares about that is local to the context
			LocalExecutionContext.PopQuery(*Chunk->Params.SourceQuery);
		}

		LOG_CHUNK_WORKER_CHUNK(Chunk, TEXT("ExecuteChunk Complete, Reporting to OwningQueue"));
		Chunk->OwningProcessorWorker->ChunkDone(Chunk);
	}

	void FQueuedProcessor::SetCurrentQuery(const FMassEntityQuery* SourceQuery)
	{
		// processors could execute queries in any order
		for (int Index = 0; Index < QueryInfos.Num(); ++Index)
		{
			if (QueryInfos[Index].SourceQuery == SourceQuery)
			{
				CurrentQueryIndex = Index;
				return;
			}
		}
		checkf(false, TEXT("Query not found! Processor attempting to run a query it doesn't own."));
	}

	void FProcessorWorker::ExecuteProcessor()
	{
		CHECK_THREAD_CONSISTENCY(bInlineExecution);
		check(RunningProcessor);

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("UE::Mass::FProcessorWorker::ExecuteProcessor: %s"), *RunningProcessor->Processor->GetProcessorName()));

		LOG_PROC_WORKER(TEXT("ExecuteProcessor: Processor=%s"), *RunningProcessor->Processor->GetProcessorName());
		LOG_PROC_WORKER(TEXT("ExecuteProcessor: QueuedProcessor=%p"), RunningProcessor);

		if (!RunningProcessor->Processor->IsActive())
		{
			FQueuedProcessor* CompletedProcessor = RunningProcessor;
			RunningProcessor = nullptr;
			OwningQueue.ProcessorFinished(*CompletedProcessor);
			return;
		}

		FMassExecutionContext ProcessorExecutionContext(GlobalExecutionContext);
#if WITH_MASSENTITY_DEBUG
		DebugCurrentExecutionContext = &ProcessorExecutionContext;
#endif
		ProcessorExecutionContext.SetDeferredCommandBuffer(CommandBuffer);

		if (UE::Mass::Tweakables::bForceInlineProcessorExecution)
		{
			ProcessorExecutionContext.SetPreQueryCallback([this](const FMassEntityQuery* SourceQuery)
				{
					RunningProcessor->SetCurrentQuery(SourceQuery);
					WaitForQueryResources();
				});

			ProcessorExecutionContext.SetPostQueryCallback([this](const FMassEntityQuery* SourceQuery)
				{
					RunningProcessor->SetCurrentQuery(SourceQuery);
					ProcessChunks();
				});

			ProcessorExecutionContext.SetChunkQueueFunc([this](FMassChunkProcessingQueueParams&& Params)
				{
					if (Params.bParallelExecution)
					{
						UE::Tasks::FTask WorkerTask = UE::Tasks::Launch(TEXT("Mass Force-Inline Parallel Chunk"),
							[this, LocalParams = MoveTemp(Params)]() mutable
							{
								static const FMassQueryRequirementIndicesMapping EmptyMapping;
								FMassExecutionContext& SourceContext = *LocalParams.SourceContext;

								TSharedPtr<FMassCommandBuffer> ChunkCommandBuffer = OwningQueue.GetOrCreateCommandBuffer();

								{
									FMassExecutionContext LocalExecutionContext(SourceContext);
									LocalExecutionContext.PushQuery(*LocalParams.SourceQuery);
									LocalExecutionContext.SetDeferredCommandBuffer(ChunkCommandBuffer);

									LocalParams.Archetype->ExecutionFunctionForChunk(
										LocalExecutionContext,
										*LocalParams.ExecuteFunction,
										LocalParams.RequirementMapping ? *LocalParams.RequirementMapping : EmptyMapping,
										LocalParams.EntityRange,
										LocalParams.ChunkCondition ? *LocalParams.ChunkCondition : FMassChunkConditionFunction{});

									LocalExecutionContext.PopQuery(*LocalParams.SourceQuery);
								}

								OwningQueue.ReturnCommandBufferToPool(ChunkCommandBuffer);
							},
							UE::Tasks::ETaskPriority::Normal);

						RunningProcessor->DispatchedChunkTasks.Add(MoveTemp(WorkerTask));
					}
					else
					{
						ExecuteChunkInline(MoveTemp(Params));
					}
				});
		}
		else
		{
			ProcessorExecutionContext.SetPreQueryCallback([this](const FMassEntityQuery* SourceQuery)
				{
					RunningProcessor->SetCurrentQuery(SourceQuery);
					WaitForQueryResources();
				});

			ProcessorExecutionContext.SetPostQueryCallback([this](const FMassEntityQuery* SourceQuery)
				{
					RunningProcessor->SetCurrentQuery(SourceQuery);
					ProcessChunks();
				});

			ProcessorExecutionContext.SetChunkQueueFunc([this](FMassChunkProcessingQueueParams&& Params)
				{
					QueueChunk(MoveTemp(Params));
				});
		}

		FMassEntityManager& EntityManager = ProcessorExecutionContext.GetEntityManagerChecked();
		FMassEntityManager::FScopedProcessing ProcessingScope = EntityManager.NewProcessingScope();

		RunningProcessor->Processor->CallExecute(EntityManager, ProcessorExecutionContext);

		LOG_PROC_WORKER(TEXT("ExecuteProcessor: Processor CallExecute complete. Proc=%p"), RunningProcessor);
		check(ChunksRemaining.load() == 0);

		FQueuedProcessor* CompletedProcessor = RunningProcessor;
#if WITH_MASSENTITY_DEBUG
		DebugCurrentExecutionContext = nullptr;
#endif
		RunningProcessor = nullptr;

		// Return the command buffer to the pool before signaling completion.
		Cleanup();

		OwningQueue.ProcessorFinished(*CompletedProcessor);
	}

	void FProcessorWorker::WaitForQueryResources()
	{
		// PreQueryCallback: the processor is about to run a query (gather chunks). Acquire both the
		// query and global-chunk contention indices up front so chunk dispatch logic doesn't have to

		LOG_PROC_WORKER(TEXT("WaitForQueryResources: Proc=%p"), RunningProcessor);

		const FQueryInfo& QueryInfo = RunningProcessor->GetCurrentQueryInfo();
		// Release processor contention and acquire query contention.
		OwningQueue.ReleaseAndReserve(RunningProcessor->ContentionIndex, QueryInfo.QueryContentionIndex);
		OwningQueue.Reserve(QueryInfo.ChunkGlobalContentionIndex);
		GlobalChunkContentionIndex = QueryInfo.ChunkGlobalContentionIndex;
		LOG_PROC_WORKER(TEXT("WaitForQueryResources: Resources locked! Proc=%p"), RunningProcessor);
	}

	void FProcessorWorker::ProcessChunks()
	{
		LOG_PROC_WORKER(TEXT("ProcessChunks: Proc=%p"), RunningProcessor);
		UE_MT_SCOPED_WRITE_ACCESS(RunningProcessor->ChunkQueueAccessDetector);

		// Query body is done; release the query contention index. Global chunk contention stays
		// held (acquired in WaitForQueryResources) through the chunk dispatch phase below.
		checkSlow(GlobalChunkContentionIndex == RunningProcessor->GetCurrentQueryInfo().ChunkGlobalContentionIndex);
		OwningQueue.Release(RunningProcessor->GetCurrentQueryInfo().QueryContentionIndex);

		TArray<int32>& Pending = RunningProcessor->PendingChunkIndices;
		TArray<FQueuedChunk>& Queue = RunningProcessor->ChunkQueue;

		if (!Pending.IsEmpty())
		{
			const int32 DirectContentionIndex = Queue[0].QueryInfo->ChunkContentionIndex;
			// Inline-vs-async dispatch follows the chunk's parallel flag. Serial chunks
			// (ForEachEntityChunk) run synchronously on whatever thread ProcessChunks is on;
			// parallel chunks (ParallelForEachEntityChunk) dispatch to worker tasks, even when
			// this processor itself is pinned to the game thread.
			const bool bRunInline = !Queue[0].Params.bParallelExecution;

			// Create listener before the dispatch loop so we don't miss completions that occur during dispatch
			FChangeListener ChangeListener(OwningQueue.QueueChangeTracker);
#if !UE_BUILD_SHIPPING
			int32 StallCount = 0;
#endif

			while (!Pending.IsEmpty())
			{
				bool bAnyDispatched = false;
				for (int32 PendingIndex = Pending.Num() - 1; PendingIndex >= 0; --PendingIndex)
				{
					int32 ChunkIndex = Pending[PendingIndex];
					FQueuedChunk& QueuedChunk = Queue[ChunkIndex];

					if (OwningQueue.TryActivateChunk(QueuedChunk, DirectContentionIndex))
					{
						Pending.RemoveAtSwap(PendingIndex, EAllowShrinking::No);
						QueuedChunk.State = FQueuedChunk::EState::Running;
						OwningQueue.MakeChunkWorker(&QueuedChunk, bRunInline);
						bAnyDispatched = true;
					}
				}

				// If a full pass activated nothing, all remaining chunks contended
				// so wait for a chunk to complete (which might free contention) before retrying
				if (!Pending.IsEmpty() && !bAnyDispatched)
				{
#if !UE_BUILD_SHIPPING
					constexpr int32 StallWarningInterval = 100;
					UE_CLOG(++StallCount % StallWarningInterval == 0, LogMass, Warning,
						TEXT("ProcessChunks waiting on contention for %d consecutive iterations with %d chunks still pending - possible deadlock. Proc=%p"),
						StallCount, Pending.Num(), RunningProcessor);
#endif
					ChangeListener.WaitForChange();
				}
			}
		}

		// Wait on all async chunk tasks (populated by parallel-chunk dispatch above; empty otherwise).
		for (UE::Tasks::FTask& Task : RunningProcessor->DispatchedChunkTasks)
		{
			Task.Wait();
		}
		RunningProcessor->DispatchedChunkTasks.Reset();
		RunningProcessor->ChunkQueue.Reset();

		LOG_PROC_WORKER(TEXT("ProcessChunks: Chunks complete, waiting for main processor resource lock. Proc=%p"), RunningProcessor);

		// Release global chunk contention and re-acquire processor contention.
		OwningQueue.ReleaseAndReserve(GlobalChunkContentionIndex, RunningProcessor->ContentionIndex);
		GlobalChunkContentionIndex = INDEX_NONE;

		if (!bInlineExecution)
		{
			// Chunk workers might have run on a different thread using this command buffer.
			// Reset the thread ID to handle deferred commands from the processor after the queue.
			CommandBuffer->ForceUpdateCurrentThreadID();
		}
	}

	void FProcessorWorker::QueueChunk(FMassChunkProcessingQueueParams&& Params)
	{
		UE_MT_SCOPED_WRITE_ACCESS(RunningProcessor->ChunkQueueAccessDetector);
		TArray<FQueuedChunk>& Queue = RunningProcessor->ChunkQueue;
		FQueuedChunk& QueuedChunk = Queue.Emplace_GetRef(FQueuedChunk(this, MoveTemp(Params)));

		QueuedChunk.QueryInfo = &RunningProcessor->GetCurrentQueryInfo();
		QueuedChunk.OwningProcessorWorker = this;
		// chunk queueing is serial, but chunk completion can happen concurrently so we track completion atomically
		++ChunksRemaining;

		const bool bShouldRunInline = !QueuedChunk.Params.bParallelExecution;

		if (bShouldRunInline && OwningQueue.TryActivateChunk(QueuedChunk, QueuedChunk.QueryInfo->ChunkContentionIndex))
		{
			QueuedChunk.State = FQueuedChunk::EState::Running;
			OwningQueue.MakeChunkWorker(&QueuedChunk, true /*bInline*/);
			Queue.RemoveAt(Queue.Num() - 1, 1, EAllowShrinking::No);

			LOG_PROC_WORKER_CHUNK(&QueuedChunk, TEXT("QueueChunk: Chunk processed inline. ChunksRemaining=%d Proc=%p"), ChunksRemaining.load(), RunningProcessor);
			return;
		}

		RunningProcessor->PendingChunkIndices.Add(Queue.Num() - 1);

		LOG_PROC_WORKER_CHUNK(&QueuedChunk, TEXT("QueueChunk: Chunk queued. ChunksRemaining=%d Proc=%p"), ChunksRemaining.load(), RunningProcessor);
	}

	void FProcessorWorker::ExecuteChunkInline(FMassChunkProcessingQueueParams&& Params)
	{
		// Mirror of FChunkWorker::ExecuteChunk's serial (non-parallel) branch, minus all of the
		// queue/contention/ChunksRemaining bookkeeping. The processor owns the SourceContext we
		// invoke against, including its command buffer, and we are on the processor's worker
		// thread so no thread-id swap is needed.
		static const FMassQueryRequirementIndicesMapping EmptyMapping;
		FMassExecutionContext& SourceContext = *Params.SourceContext;

		Params.Archetype->ExecutionFunctionForChunk(
			SourceContext,
			*Params.ExecuteFunction,
			Params.RequirementMapping ? *Params.RequirementMapping : EmptyMapping,
			Params.EntityRange,
			Params.ChunkCondition ? *Params.ChunkCondition : FMassChunkConditionFunction{});
	}

	void FProcessorWorker::ChunkDone(FQueuedChunk* InChunk)
	{
		InChunk->State = FQueuedChunk::EState::Complete;
		OwningQueue.DeactivateChunk(*InChunk);
		const int32 ChunksRemainingValue = --ChunksRemaining;

		LOG_PROC_WORKER_CHUNK(InChunk,
			TEXT("ChunkDone: ChunksRemaining=%d%s"),
			ChunksRemainingValue,
			ChunksRemainingValue <= 0 ? TEXT(" (Triggering ChunksDone)") : TEXT(""));
	}

	FQueuedChunk::FQueuedChunk(FProcessorWorker* InOwningProcessorWorker, FMassChunkProcessingQueueParams&& InParams)
		: OwningProcessorWorker(InOwningProcessorWorker)
		, Params(MoveTemp(InParams))
	{
	}

	FQueuedChunk::FQueuedChunk(FQueuedChunk&& Other) noexcept
		: State(Other.State)
		, OwningProcessorWorker(Other.OwningProcessorWorker)
		, QueryInfo(Other.QueryInfo)
		, Params(MoveTemp(Other.Params))
	{
		// reset the state of Other
		Other.State = EState::MAX;
		Other.QueryInfo = nullptr;
		Other.OwningProcessorWorker = nullptr;
	}

	void FProcessorWorker::AssignProcessor(FQueuedProcessor* QueuedProcessor)
	{
		LOG_PROC_WORKER(TEXT("Processor Assigned: Processor=%s"), *QueuedProcessor->Processor->GetProcessorName());
		RunningProcessor = QueuedProcessor;

		bInlineExecution |= RunningProcessor->bRequiresGameThread;

#if WITH_MASSENTITY_DEBUG
		QueuedProcessor->DebugCurrentProcessorWorker = this;
#endif
	}
} // namespace UE::Mass

#undef CHECK_THREAD_CONSISTENCY
#undef LOG_PROC_WORKER
#undef LOG_PROC_WORKER_CHUNK
#undef LOG_CHUNK_WORKER
#undef LOG_CHUNK_WORKER_CHUNK
#undef MASS_PROCESSING_QUEUE_LOGGING
#undef LOCTEXT_NAMESPACE