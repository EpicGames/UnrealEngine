// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassProcessingTypes.h"
#include "MassProcessor.h"
#include "MassEntityUtils.h"
#include "VisualLogger/VisualLogger.h"
#include "MassDebugger.h"
#include "Misc/CoreDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassProcessingTypes)

DEFINE_LOG_CATEGORY(LogMass);

//----------------------------------------------------------------------//
//  FMassRuntimePipeline
//----------------------------------------------------------------------//
FMassRuntimePipeline::FMassRuntimePipeline(TConstArrayView<TObjectPtr<UMassProcessor>> SeedProcessors, const EProcessorExecutionFlags WorldExecutionFlags)
	: Processors(SeedProcessors)
	, ExecutionFlags(WorldExecutionFlags)
{
	
}

FMassRuntimePipeline::FMassRuntimePipeline(TConstArrayView<UMassProcessor*> SeedProcessors, const EProcessorExecutionFlags WorldExecutionFlags)
	: ExecutionFlags(WorldExecutionFlags)
{
	Processors = ObjectPtrWrap(SeedProcessors);
}

void FMassRuntimePipeline::Reset()
{
	Processors.Reset();
}

void FMassRuntimePipeline::Initialize(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	// having nulls in Processors should be rare so we run the "remove all nulls" operation below only if we know 
	// for sure that there are any nulls to be removed
	bool bNullsFound = false;

	for (UMassProcessor* Proc : Processors)
	{
		if (Proc)
		{
			if (Proc->IsInitialized() == false)
			{
				REDIRECT_OBJECT_TO_VLOG(Proc, &Owner);
				Proc->CallInitialize(&Owner, EntityManager);
			}
		}
		else
		{
			bNullsFound = true;
		}
	}

	if (bNullsFound)
	{
		Processors.RemoveAll([](const UMassProcessor* Proc) { return Proc == nullptr; });
	}
}

void FMassRuntimePipeline::SetProcessors(TArrayView<UMassProcessor*> InProcessors)
{
	Reset();
	Processors = InProcessors;
}

void FMassRuntimePipeline::SetProcessors(TArray<TObjectPtr<UMassProcessor>>&& InProcessors)
{
	Reset();
	Processors = MoveTemp(InProcessors);
}

void FMassRuntimePipeline::CreateFromArray(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner)
{
	Reset();
	AppendOrOverrideRuntimeProcessorCopies(InProcessors, InOwner);
}

void FMassRuntimePipeline::InitializeFromArray(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	CreateFromArray(InProcessors, InOwner);
	Initialize(InOwner, EntityManager);
}

void FMassRuntimePipeline::InitializeFromClassArray(TConstArrayView<TSubclassOf<UMassProcessor>> InProcessorClasses, UObject& InOwner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Reset();

	const UWorld* World = InOwner.GetWorld();
	const EProcessorExecutionFlags WorldExecutionFlags = UE::Mass::Utils::DetermineProcessorExecutionFlags(World, ExecutionFlags);

	for (const TSubclassOf<UMassProcessor>& ProcessorClass : InProcessorClasses)
	{
		if (ProcessorClass)
		{
			UMassProcessor* CDO = ProcessorClass.GetDefaultObject();
			if (CDO && CDO->ShouldExecute(WorldExecutionFlags))
			{
				UMassProcessor* ProcInstance = NewObject<UMassProcessor>(&InOwner, ProcessorClass);
				Processors.Add(ProcInstance);
			}
			else
			{
				UE_CVLOG(CDO, &InOwner, LogMass, Log, TEXT("Skipping %s due to ExecutionFlags"), *CDO->GetName());
			}
		}
	}

	Initialize(InOwner, EntityManager);
}

bool FMassRuntimePipeline::HasProcessorOfExactClass(TSubclassOf<UMassProcessor> InClass) const
{
	UClass* TestClass = InClass.Get();
	return Processors.FindByPredicate([TestClass](const UMassProcessor* Proc){ return Proc != nullptr && Proc->GetClass() == TestClass; }) != nullptr;
}

void FMassRuntimePipeline::AppendUniqueRuntimeProcessorCopies(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	const UWorld* World = InOwner.GetWorld();
	const EProcessorExecutionFlags WorldExecutionFlags = UE::Mass::Utils::DetermineProcessorExecutionFlags(World, ExecutionFlags);
	const int32 StartingCount = Processors.Num();
		
	for (const UMassProcessor* Proc : InProcessors)
	{
		if (Proc && Proc->ShouldExecute(WorldExecutionFlags)
			&& HasProcessorOfExactClass(Proc->GetClass()) == false)
		{
			// unfortunately the const cast is required since NewObject doesn't support const Template object
			UMassProcessor* ProcCopy = NewObject<UMassProcessor>(&InOwner, Proc->GetClass(), FName(), RF_NoFlags, const_cast<UMassProcessor*>(Proc));
			Processors.Add(ProcCopy);
		}
#if WITH_MASSENTITY_DEBUG
		else if (Proc)
		{
			if (Proc->ShouldExecute(WorldExecutionFlags) == false)
			{
				UE_VLOG(&InOwner, LogMass, Log, TEXT("Skipping %s due to ExecutionFlags"), *Proc->GetName());
			}
			else if (Proc->ShouldAllowMultipleInstances() == false)
			{
				UE_VLOG(&InOwner, LogMass, Log, TEXT("Skipping %s due to it being a duplicate"), *Proc->GetName());
			}
		}
#endif // WITH_MASSENTITY_DEBUG
	}

	for (int32 NewProcIndex = StartingCount; NewProcIndex < Processors.Num(); ++NewProcIndex)
	{
		UMassProcessor* Proc = Processors[NewProcIndex];
		check(Proc);
		
		if (Proc->IsInitialized() == false)
		{
			REDIRECT_OBJECT_TO_VLOG(Proc, &InOwner);
 			Proc->CallInitialize(&InOwner, EntityManager);
		}
	}
}

void FMassRuntimePipeline::AppendOrOverrideRuntimeProcessorCopies(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner)
{
	const UWorld* World = InOwner.GetWorld();
	const EProcessorExecutionFlags WorldExecutionFlags = UE::Mass::Utils::DetermineProcessorExecutionFlags(World, ExecutionFlags);

	for (const UMassProcessor* Proc : InProcessors)
	{
		if (Proc && Proc->ShouldExecute(WorldExecutionFlags))
		{
			// unfortunately the const cast is required since NewObject doesn't support const Template object
			UMassProcessor* ProcCopy = NewObject<UMassProcessor>(&InOwner, Proc->GetClass(), FName(), RF_NoFlags, const_cast<UMassProcessor*>(Proc));
			check(ProcCopy);

			if (ProcCopy->ShouldAllowMultipleInstances())
			{
				// we don't care if there are instances of this class in Processors already
				Processors.Add(ProcCopy);
			}
			else 
			{
				const UClass* TestClass = Proc->GetClass();
				TObjectPtr<UMassProcessor>* PrevProcessor = Processors.FindByPredicate([TestClass, ProcCopy](const UMassProcessor* Proc) {
					return Proc != nullptr && Proc->GetClass() == TestClass;
				});

				if (PrevProcessor)
				{
					*PrevProcessor = ProcCopy;
				}
				else
				{
					Processors.Add(ProcCopy);
				}
			}
		}
		else
		{
			UE_CVLOG(Proc, &InOwner, LogMass, Log, TEXT("Skipping %s due to ExecutionFlags"), *Proc->GetName());
		}
	}
}

void FMassRuntimePipeline::AppendProcessor(UMassProcessor& InProcessor)
{
	Processors.Add(&InProcessor);
}

void FMassRuntimePipeline::AppendProcessors(TArrayView<TObjectPtr<UMassProcessor>> InProcessors)
{
	Processors.Append(InProcessors);
}

void FMassRuntimePipeline::AppendProcessors(TArray<TObjectPtr<UMassProcessor>>&& InProcessors)
{
	if (Processors.Num())
	{
		Processors.Append(MoveTemp(InProcessors));
	}
	else
	{
		Processors = MoveTemp(InProcessors);
	}
}

bool FMassRuntimePipeline::AppendUniqueProcessor(UMassProcessor& Processor)
{
	const int32 PreviousCount = Processors.Num();
	Processors.AddUnique(&Processor);
	return PreviousCount != Processors.Num();
}

void FMassRuntimePipeline::AppendProcessor(TSubclassOf<UMassProcessor> ProcessorClass, UObject& InOwner)
{
	check(ProcessorClass);
	UMassProcessor* ProcInstance = NewObject<UMassProcessor>(&InOwner, ProcessorClass);
	AppendProcessor(*ProcInstance);
}

bool FMassRuntimePipeline::RemoveProcessor(const UMassProcessor& InProcessor)
{
	return Processors.RemoveAll([Processor = &InProcessor](const TObjectPtr<UMassProcessor>& Element)
		{
			return Element == Processor;	
		}) > 0;
}

UMassCompositeProcessor* FMassRuntimePipeline::FindTopLevelGroupByName(FName GroupName)
{
	for (UMassProcessor* Processor : Processors)
	{
		UMassCompositeProcessor* CompositeProcessor = Cast<UMassCompositeProcessor>(Processor);
		if (CompositeProcessor && CompositeProcessor->GetGroupName() == GroupName)
		{
			return CompositeProcessor;
		}
	}
	return nullptr;
}

void FMassRuntimePipeline::SortByExecutionPriority()
{
	if (Processors.IsEmpty())
	{
		return;
	}

	Processors.RemoveAllSwap([](const UMassProcessor* Processor)
	{ 
		return Processor == nullptr; 
	});
	Processors.Sort([](const UMassProcessor& ProcessorA, const UMassProcessor& ProcessorB)
	{
		return ProcessorA.GetExecutionPriority() > ProcessorB.GetExecutionPriority();
	});
}

uint32 GetTypeHash(const FMassRuntimePipeline& Instance)
{ 
	uint32 Hash = 0;
	for (const UMassProcessor* Proc : Instance.Processors)
	{
		Hash = HashCombine(Hash, PointerHash(Proc));
	}
	return Hash;
}

//-----------------------------------------------------------------------------
// DEPRECATED
//-----------------------------------------------------------------------------
void FMassRuntimePipeline::Initialize(UObject& Owner)
{
	FMassEntityManager* EntityManager = UE::Mass::Utils::GetEntityManager(Owner.GetWorld());
	if (ensureMsgf(EntityManager, TEXT("Unable to determine the current MassEntityManager")))
	{
		Initialize(Owner, EntityManager->AsShared());
	}
}

void FMassRuntimePipeline::InitializeFromArray(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner)
{
	FMassEntityManager* EntityManager = UE::Mass::Utils::GetEntityManager(&InOwner);
	if (ensureMsgf(EntityManager, TEXT("Unable to determine the current MassEntityManager")))
	{
		InitializeFromArray(InProcessors, InOwner, EntityManager->AsShared());
	}
}

void FMassRuntimePipeline::InitializeFromClassArray(TConstArrayView<TSubclassOf<UMassProcessor>> InProcessorClasses, UObject& InOwner)
{
	FMassEntityManager* EntityManager = UE::Mass::Utils::GetEntityManager(&InOwner);
	if (ensureMsgf(EntityManager, TEXT("Unable to determine the current MassEntityManager")))
	{
		InitializeFromClassArray(InProcessorClasses,  InOwner, EntityManager->AsShared());
	}	
}

void FMassRuntimePipeline::AppendUniqueRuntimeProcessorCopies(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner)
{
	FMassEntityManager* EntityManager = UE::Mass::Utils::GetEntityManager(&InOwner);
	if (ensureMsgf(EntityManager, TEXT("Unable to determine the current MassEntityManager")))
	{
		AppendUniqueRuntimeProcessorCopies(InProcessors, InOwner, EntityManager->AsShared());
	}	
}

void FMassRuntimePipeline::SetProcessors(TArray<UMassProcessor*>&& InProcessors)
{
	SetProcessors(MakeArrayView(InProcessors.GetData(), InProcessors.Num()));
}
