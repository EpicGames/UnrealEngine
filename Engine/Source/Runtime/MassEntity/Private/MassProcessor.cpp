// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassProcessor.h"
#include "MassEntitySettings.h"
#include "MassProcessorDependencySolver.h"
#include "VisualLogger/VisualLogger.h"
#include "Engine/World.h"
#include "MassCommandBuffer.h"
#include "MassExecutionContext.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "MassQueryExecutor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassProcessor)

DECLARE_CYCLE_STAT(TEXT("MassProcessor Group Completed"), Mass_GroupCompletedTask, STATGROUP_TaskGraphTasks);
DECLARE_CYCLE_STAT(TEXT("Mass Processor Task"), STAT_Mass_DoTask, STATGROUP_Mass);
DECLARE_CYCLE_STAT(TEXT("Mass Processor Execute"), STAT_Mass_ExecuteProcessor, STATGROUP_Mass);

#if WITH_MASSENTITY_DEBUG
namespace UE::Mass::Debug
{
	bool bLogProcessingGraphEveryFrame = false;
	bool bLogNewProcessingGraph = true;

	namespace
	{
		FAutoConsoleVariableRef CVars[] = {
			{ TEXT("mass.LogProcessingGraph"), bLogProcessingGraphEveryFrame
				, TEXT("When enabled every composite processor, every frame, will log task graph tasks created while dispatching processors to other threads, along with their dependencies.")
				, ECVF_Cheat }
			, { TEXT("mass.LogNewProcessingGraph"), bLogNewProcessingGraph
				, TEXT("When enabled every time a new processing graph is created the composite processor hosting it will log it during first execution.")
				, ECVF_Cheat }
		};
	}

} // namespace UE::Mass::Debug

// change to 1 to enable more detailed processing tasks logging
#if 0
#define PROCESSOR_TASK_LOG(Fmt, ...) UE_VLOG_UELOG(this, LogMass, Verbose, Fmt, ##__VA_ARGS__)
#else
#define PROCESSOR_TASK_LOG(...) 
#endif // 0

#else
#define PROCESSOR_TASK_LOG(...)
#endif // WITH_MASSENTITY_DEBUG

class FMassProcessorTask
{
public:
	FMassProcessorTask(const TSharedPtr<FMassEntityManager>& InEntityManager, const FMassExecutionContext& InExecutionContext, UMassProcessor& InProc, bool bInManageCommandBuffer = true)
		: EntityManager(InEntityManager)
		, ExecutionContext(InExecutionContext)
		, Processor(&InProc)
		, bManageCommandBuffer(bInManageCommandBuffer)
	{}

	static TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMassProcessorTask, STATGROUP_TaskGraphTasks);
	}

	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	static ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::AnyHiPriThreadHiPriTask;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		checkf(Processor, TEXT("Expecting a valid processor to execute"));

		PROCESSOR_TASK_LOG(TEXT("+--+ Task %s started on %u"), *Processor->GetProcessorName(), FPlatformTLS::GetCurrentThreadId());
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(MassProcessorTask);
		SCOPE_CYCLE_COUNTER(STAT_Mass_DoTask);
		SCOPE_CYCLE_COUNTER(STAT_Mass_Total);

		check(EntityManager);
		FMassEntityManager& EntityManagerRef = *EntityManager.Get();
		FMassEntityManager::FScopedProcessing ProcessingScope = EntityManagerRef.NewProcessingScope();

		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Mass Processor Task");
		
		if (bManageCommandBuffer)
		{
			TSharedPtr<FMassCommandBuffer> MainSharedPtr = ExecutionContext.GetSharedDeferredCommandBuffer();
			ExecutionContext.SetDeferredCommandBuffer(MakeShareable(new FMassCommandBuffer()));
			Processor->CallExecute(EntityManagerRef, ExecutionContext);
			MainSharedPtr->MoveAppend(ExecutionContext.Defer());
		}
		else
		{
			Processor->CallExecute(EntityManagerRef, ExecutionContext);
		}
		PROCESSOR_TASK_LOG(TEXT("+--+ Task %s finished"), *Processor->GetProcessorName());
	}

private:
	TSharedPtr<FMassEntityManager> EntityManager;
	FMassExecutionContext ExecutionContext;
	UMassProcessor* Processor = nullptr;
	/** 
	 * indicates whether this task is responsible for creation of a dedicated command buffer and transferring over the 
	 * commands after processor's execution;
	 */
	bool bManageCommandBuffer = true;
};

class FMassProcessorsTask_GameThread : public FMassProcessorTask
{
public:
	FMassProcessorsTask_GameThread(const TSharedPtr<FMassEntityManager>& InEntityManager, const FMassExecutionContext& InExecutionContext, UMassProcessor& InProc)
		: FMassProcessorTask(InEntityManager, InExecutionContext, InProc)
	{}

	static ENamedThreads::Type GetDesiredThread()
	{
		// Use a high priority task so processor chains that touch the game thread will take priority over normal ticks
		return ENamedThreads::SetTaskPriority(ENamedThreads::GameThread, ENamedThreads::HighTaskPriority);
	}
};

//----------------------------------------------------------------------//
// UMassProcessor 
//----------------------------------------------------------------------//
UMassProcessor::UMassProcessor(const FObjectInitializer& ObjectInitializer)
	: UMassProcessor()
{
}

UMassProcessor::UMassProcessor()
	: ExecutionFlags(static_cast<int32>(EProcessorExecutionFlags::Server | EProcessorExecutionFlags::Standalone))
{
}

void UMassProcessor::CallInitialize(const TNotNull<UObject*> Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	if (ensure(HasAnyFlags(RF_ClassDefaultObject) == false 
		&& GetClass()->HasAnyClassFlags(CLASS_Abstract) == false))
	{

#if WITH_MASSENTITY_DEBUG
		if (EntityManager.Get().DebugHasAllDebugFeatures(FMassEntityManager::EDebugFeatures::TraceProcessors))
		{
			DebugDescription = *GetProcessorName();
			FString NetMode = EntityManager.Get().GetWorld() ? ToString(EntityManager.Get().GetWorld()->GetNetMode()) : TEXT("None");
			//                       DebugDescription       " ("  NetMode        ")"
			DebugDescription.Reserve(DebugDescription.Len() + 2 + NetMode.Len() + 1);
			DebugDescription.Append(TEXT(" ("));
			DebugDescription.Append(*NetMode);
			DebugDescription.Append(TEXT(")"));
		}
		else
		{
			DebugDescription = FString::Printf(TEXT("%s (%s)"), *GetProcessorName(), EntityManager.Get().GetWorld() ? *ToString(EntityManager.Get().GetWorld()->GetNetMode()) : TEXT("No World"));
		}
#endif

		for (FMassEntityQuery* Query : OwnedQueries)
		{
			// we should never get nulls here since OwnedQueries is private and the only way to
			// add queries to it is to go through RegisterQuery, which in turn ensures the
			// input query is a member variable of the processor.
			checkfSlow(Query, TEXT("We never expect nulls in OwnedQueries - those pointers are supposed to point at member variable."));
			Query->Initialize(EntityManager);
		}

		ConfigureQueries(EntityManager);

		ResolveRequestedArchetypes(EntityManager.Get());

		bool bNeedsGameThread = ProcessorRequirements.DoesRequireGameThreadExecution();
		for (FMassEntityQuery* QueryPtr : OwnedQueries)
		{
			CA_ASSUME(QueryPtr);
			bNeedsGameThread = (bNeedsGameThread || QueryPtr->DoesRequireGameThreadExecution());
		}

		UE_CLOGF(bRequiresGameThreadExecution != bNeedsGameThread, LogMass, Verbose
			, "%ls is marked bRequiresGameThreadExecution = %ls, while the registered queries' or processor requirements indicate the opposite"
			, *GetProcessorName(), bRequiresGameThreadExecution ? TEXT("TRUE") : TEXT("FALSE"));

		// better safe than sorry - if queries or processor requirements indicate the game thread execution is required, then we mark the whole processor as such
		bRequiresGameThreadExecution = bRequiresGameThreadExecution || bNeedsGameThread;

		InitializeInternal(*Owner, EntityManager);

		bInitialized = true;
	}
}

void UMassProcessor::ResolveRequestedArchetypes(const FMassEntityManager& EntityManager)
{
	if (bCreatedArchetypeHandlesResolved)
	{
		return;
	}

	ProcessorEntityCreationRequirements.ResolveArchetypes(EntityManager);

	for (FMassEntityQuery* QueryPtr : OwnedQueries)
	{
		CA_ASSUME(QueryPtr);

		FMassEntityCreationRequirements& QueryEntityCreationRequirements = QueryPtr->GetEntityCreationRequirements();

		QueryEntityCreationRequirements.ResolveArchetypes(EntityManager);

		bQueryHasEntityCreationRequirements = bQueryHasEntityCreationRequirements || !QueryEntityCreationRequirements.IsEmpty();
	}
	 
	bCreatedArchetypeHandlesResolved = true;
}

void UMassProcessor::ClearArchetypesCreatedHandles()
{
	bCreatedArchetypeHandlesResolved = false;
	ProcessorEntityCreationRequirements.ClearCreatedArchetypeHandles();

	for (FMassEntityQuery* QueryPtr : OwnedQueries)
	{
		CA_ASSUME(QueryPtr);

		FMassEntityCreationRequirements& QueryEntityCreationRequirements = QueryPtr->GetEntityCreationRequirements();

		QueryEntityCreationRequirements.ClearCreatedArchetypeHandles();
	}
}

void UMassProcessor::InitializeInternal(UObject&, const TSharedRef<FMassEntityManager>&)
{
	// empty in base class
}

void UMassProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>&)
{
	if (AutoExecuteQuery.IsValid())
	{
		AutoExecuteQuery->ConfigureQuery(ProcessorRequirements);
	}
	else
	{
		UE_CVLOG_UELOG(OwnedQueries.Num(), this, LogMass, Warning
			, TEXT("%s has entity queries registered. Make sure to override ConfigureQueries to configure the queries, and do not call the Super implementation")
			, *GetProcessorName());
	}
}

void UMassProcessor::SetShouldAutoRegisterWithGlobalList(const bool bAutoRegister)
{	
	if (ensureMsgf(HasAnyFlags(RF_ClassDefaultObject), TEXT("Setting bAutoRegisterWithProcessingPhases for non-CDOs has no effect")))
	{
		bAutoRegisterWithProcessingPhases = bAutoRegister;
#if WITH_EDITOR
		if (UClass* Class = GetClass())
		{
			if (FProperty* AutoRegisterProperty = Class->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMassProcessor, bAutoRegisterWithProcessingPhases)))
			{
				UpdateSinglePropertyInConfigFile(AutoRegisterProperty, *GetDefaultConfigFilename());
			}
		}
#endif // WITH_EDITOR
	}
}

void UMassProcessor::GetArchetypesMatchingOwnedQueries(const FMassEntityManager& EntityManager, TArray<FMassArchetypeHandle>& OutArchetype)
{
	UE_CLOGF(OwnedQueries.Num() == 0, LogMass, Warning, "%ls has no registered queries while being asked for matching archetypes", *GetName());

	for (FMassEntityQuery* QueryPtr : OwnedQueries)
	{
		CA_ASSUME(QueryPtr);
		QueryPtr->CacheArchetypes();

		for (const FMassArchetypeHandle& ArchetypeHandle : QueryPtr->GetArchetypes())
		{
			OutArchetype.AddUnique(ArchetypeHandle);
		}
	}
}

bool UMassProcessor::DoesAnyArchetypeMatchOwnedQueries(const FMassEntityManager& EntityManager)
{
	for (FMassEntityQuery* QueryPtr : OwnedQueries)
	{
		CA_ASSUME(QueryPtr);
		QueryPtr->CacheArchetypes();

		if (QueryPtr->GetArchetypes().Num() > 0)
		{
			return true;
		}
	}
	return false;
}

void UMassProcessor::PostInitProperties()
{
	// Capture constructor-set hard deps before Super::PostInitProperties calls LoadConfig
	TArray<FName> HardcodedExecuteAfter = ExecutionOrder.ExecuteAfter;
	TArray<FName> HardcodedExecuteBefore = ExecutionOrder.ExecuteBefore;

	Super::PostInitProperties();

	// Re-merge hardcoded deps
	for (const FName& Name : HardcodedExecuteAfter)
	{
		ExecutionOrder.ExecuteAfter.AddUnique(Name);
	}
	for (const FName& Name : HardcodedExecuteBefore)
	{
		ExecutionOrder.ExecuteBefore.AddUnique(Name);
	}

#if CPUPROFILERTRACE_ENABLED
	StatId = GetProcessorName();
#endif
}

void UMassProcessor::CallExecute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_Mass_ExecuteProcessor);
	if (LIKELY(ensureMsgf(IsActive(), TEXT("Trying to CallExecute for an inactive processor %s"), *GetProcessorName())))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*StatId);
		LLM_SCOPE_BYNAME(TEXT("Mass/ExecuteProcessor"));
		// Not using a more specific scope by default (i.e. LLM_SCOPE_BYNAME(*StatId)) since LLM is more strict regarding the provided string (no spaces or '_')

#if WITH_MASSENTITY_DEBUG
		Context.DebugSetExecutionDesc(DebugDescription);
		Context.DebugSetProcessor(this);
#endif

		bool bAllRequirementsMet = false;
		{
			// Setup output device to collect errors that can be reported by CacheSubsystemRequirements
			FStringOutputDevice ErrorOutput;
			UE::Mass::ErrorReporting::FScopedOutputDevice ScopedOutput(ErrorOutput);

			// CacheSubsystemRequirements will return true only if all requirements declared with ProcessorRequirements are met
			// meaning if it fails there's no point in calling Execute.
			// Note that we're not testing individual queries in OwnedQueries - processors can function just fine with some
			// of their queries not having anything to do.
			bAllRequirementsMet = Context.CacheSubsystemRequirements(ProcessorRequirements);
			if (!bAllRequirementsMet)
			{
				UE_VLOG_UELOG(this, LogMass, Log, TEXT("%s Skipping Execute due to subsystem requirements not being met: %s"), *GetProcessorName(), *ErrorOutput);
			}
		}

		if (bAllRequirementsMet)
		{
			Context.SetActiveArchetypeRequirements(&ProcessorEntityCreationRequirements);
			Execute(EntityManager, Context);
		}

		Context.SetActiveArchetypeRequirements(nullptr);

		if (ActivationState == EActivationState::OneShot)
		{
			MakeInactive();
		}
	}
}

void UMassProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	if (AutoExecuteQuery.IsValid())
	{
		AutoExecuteQuery->CallExecute(Context);
	}
	else
	{
		static constexpr TCHAR MessageFormat[] = TEXT("UMassProcessor::Execute should never be called without an AutoExecuteQuery set. Override the function or populate AutoExecuteQuery. Processor name: %s");
		checkf(false, MessageFormat, *GetProcessorName());
	}
}

bool UMassProcessor::ShouldAllowQueryBasedPruning(const bool bRuntimeMode) const
{
	return bRuntimeMode && QueryBasedPruning == EMassQueryBasedPruning::Prune;
}

EMassProcessingPhase UMassProcessor::GetProcessingPhase() const
{
	return ProcessingPhase;
}

void UMassProcessor::SetProcessingPhase(EMassProcessingPhase Phase)
{
	ProcessingPhase = Phase;
}

void UMassProcessor::ExportRequirements(FMassExecutionRequirements& OutRequirements) const
{
	ProcessorRequirements.ExportRequirements(OutRequirements);

	for (FMassEntityQuery* Query : OwnedQueries)
	{
		CA_ASSUME(Query);
		Query->ExportRequirements(OutRequirements);
	}

	ProcessorEntityCreationRequirements.ExportRequirements(OutRequirements);
}

bool UMassProcessor::HasEntityCreationRequirements()
{
	return bQueryHasEntityCreationRequirements || !ProcessorEntityCreationRequirements.IsEmpty();
}

void UMassProcessor::ExportArchetypesCreated(TArray<FMassArchetypeHandle>& OutArchetypes)
{
	OutArchetypes.Append(ProcessorEntityCreationRequirements.GetCreatedArchetypes());
	for (FMassEntityQuery* Query : OwnedQueries)
	{
		CA_ASSUME(Query);
		OutArchetypes.Append(Query->GetEntityCreationRequirements().GetCreatedArchetypes());
	}
}

void UMassProcessor::RegisterQuery(FMassEntityQuery& Query)
{
	const uintptr_t ThisStart = reinterpret_cast<uintptr_t>(this);
	const uintptr_t ThisEnd = ThisStart + GetClass()->GetStructureSize();
	const uintptr_t QueryStart = reinterpret_cast<uintptr_t>(&Query);
	const uintptr_t QueryEnd = QueryStart + sizeof(FMassEntityQuery);

	if (QueryStart >= ThisStart && QueryEnd <= ThisEnd)
	{
		OwnedQueries.AddUnique(&Query);
	}
	else
	{
		static constexpr TCHAR MessageFormat[] = TEXT("Registering entity query for %s while the query is not given processor's member variable. Skipping.");
		checkf(false, MessageFormat, *GetProcessorName());
		UE_LOG(LogMass, Error, MessageFormat, *GetProcessorName());
	}
}

FGraphEventRef UMassProcessor::DispatchProcessorTasks(const TSharedPtr<FMassEntityManager>& EntityManager, FMassExecutionContext& ExecutionContext, const FGraphEventArray& Prerequisites)
{
	FGraphEventRef ReturnVal;
	if (LIKELY(ensureMsgf(IsActive(), TEXT("Trying to dispatch processor task for inactive processor %s"), *GetProcessorName())))
	{
		if (bRequiresGameThreadExecution)
		{
			ReturnVal = TGraphTask<FMassProcessorsTask_GameThread>::CreateTask(&Prerequisites).ConstructAndDispatchWhenReady(EntityManager, ExecutionContext, *this);
		}
		else
		{
			ReturnVal = TGraphTask<FMassProcessorTask>::CreateTask(&Prerequisites).ConstructAndDispatchWhenReady(EntityManager, ExecutionContext, *this);
		}
	}
	return ReturnVal;
}

FString UMassProcessor::GetProcessorName() const
{
	return GetName();
}

void UMassProcessor::DebugOutputDescription(FOutputDevice& Ar, int32 Indent) const
{
#if WITH_MASSENTITY_DEBUG
	Ar.Logf(TEXT("%*s%s"), Indent, TEXT(""), *GetProcessorName());
#endif // WITH_MASSENTITY_DEBUG
}

//----------------------------------------------------------------------//
//  UMassCompositeProcessor
//----------------------------------------------------------------------//
UMassCompositeProcessor::UMassCompositeProcessor()
	: GroupName(TEXT("None"))
{
	// not auto-registering composite processors since the idea of the global processors list is to indicate all 
	// the processors doing the work while composite processors are just containers. Having said that subclasses 
	// can change this behavior if need be.
	bAutoRegisterWithProcessingPhases = false;
}

void UMassCompositeProcessor::SetChildProcessors(TArrayView<UMassProcessor*> InProcessors)
{
	ChildPipeline.SetProcessors(InProcessors);
}

void UMassCompositeProcessor::SetChildProcessors(TArray<TObjectPtr<UMassProcessor>>&& InProcessors)
{
	ChildPipeline.SetProcessors(MoveTemp(InProcessors));
}

void UMassCompositeProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	// nothing to do here since ConfigureQueries will get independently called for all the processors during their creation
}

FGraphEventRef UMassCompositeProcessor::DispatchProcessorTasks(const TSharedPtr<FMassEntityManager>& EntityManager, FMassExecutionContext& ExecutionContext, const FGraphEventArray& InPrerequisites)
{
	FGraphEventArray Events;
	Events.AddDefaulted(FlatProcessingGraph.Num());

	FGraphEventArray Prerequisites;
	// we'll fill this one with dependencies of disabled processors. We initialize it lazily
	TArray<FGraphEventArray> AdditionalEvents;
		
	for (int32 NodeIndex = 0; NodeIndex < FlatProcessingGraph.Num(); ++NodeIndex)
	{
		FDependencyNode& ProcessingNode = FlatProcessingGraph[NodeIndex];

		if (ensureMsgf(ProcessingNode.Processor, TEXT("We don't expect any group nodes at this point. If we get any there's a bug in dependencies solving.")))
		{
			Prerequisites.Reset(ProcessingNode.Dependencies.Num());
			for (const int32 DependencyIndex : ProcessingNode.Dependencies)
			{
				checkSlow(DependencyIndex < NodeIndex);
				Prerequisites.Add(Events[DependencyIndex]);
			}
			// this means there are some inactive processors so we need to consider additional dependencies
			if (AdditionalEvents.Num())
			{
				for (const int32 DependencyIndex : ProcessingNode.Dependencies)
				{
					Prerequisites.Append(AdditionalEvents[DependencyIndex]);
				}
			}

			if (ProcessingNode.Processor->IsActive())
			{
				Events[NodeIndex] = ProcessingNode.Processor->DispatchProcessorTasks(EntityManager, ExecutionContext, Prerequisites);
			}
			else
			{
				if (AdditionalEvents.Num() == 0)
				{
					// lazy initialization
					AdditionalEvents.AddDefaulted(FlatProcessingGraph.Num());
				}
				// if the processor is not going to run at all we store its Prerequisites so that
				// processors waiting for this given processor to finish will keep their place
				// in the overall processing graph
				// NOTE: this is safer than just ignoring the dependencies since even though this
				// processor is not running, the subsequent processors might unknowingly rely on
				// implicit dependencies that the current processor was ensuring. 
				AdditionalEvents[NodeIndex].Append(MoveTemp(Prerequisites));
			}
		}
	}

#if WITH_MASSENTITY_DEBUG
	if (UE::Mass::Debug::bLogProcessingGraphEveryFrame || bDebugLogNewProcessingGraph)
	{
		FScopedCategoryAndVerbosityOverride LogOverride(TEXT("LogMass"), ELogVerbosity::Log);

		for (int i = 0; i < FlatProcessingGraph.Num(); ++i)
		{
			FDependencyNode& ProcessingNode = FlatProcessingGraph[i];
			FString DependenciesDesc;
			for (const int32 DependencyIndex : ProcessingNode.Dependencies)
			{
				DependenciesDesc += FString::Printf(TEXT("%s, "), *FlatProcessingGraph[DependencyIndex].Name.ToString());
			}

			check(ProcessingNode.Processor);
			if (Events[i].IsValid())
			{
				PROCESSOR_TASK_LOG(TEXT("Task %u %s%s%s"), Events[i]->GetTraceId(), *ProcessingNode.Processor->GetProcessorName()
					, DependenciesDesc.Len() > 0 ? TEXT(" depends on ") : TEXT(""), *DependenciesDesc);
			}
			else
			{
				ensureMsgf(ProcessingNode.Processor->IsActive() == false, TEXT("This path is expected to trigger only for inactive processors"))
				PROCESSOR_TASK_LOG(TEXT("Task [INACTIVE] %s%s%s"), *ProcessingNode.Processor->GetProcessorName()
					, DependenciesDesc.Len() > 0 ? TEXT(" depends on ") : TEXT(""), *DependenciesDesc);
			}
		}

		bDebugLogNewProcessingGraph = false;
	}
#endif // WITH_MASSENTITY_DEBUG

	FGraphEventRef CompletionEvent = FFunctionGraphTask::CreateAndDispatchWhenReady([this](){}
		, GET_STATID(Mass_GroupCompletedTask), &Events, ENamedThreads::AnyHiPriThreadHiPriTask);

	return CompletionEvent;
}

void UMassCompositeProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	for (UMassProcessor* Proc : ChildPipeline.GetMutableProcessors())
	{
		if (LIKELY(ensure(Proc) && Proc->IsActive()))
		{
			Proc->CallExecute(EntityManager, Context);
		}
	}
}

void UMassCompositeProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	ChildPipeline.Initialize(Owner, EntityManager);
	Super::InitializeInternal(Owner, EntityManager);
}

void UMassCompositeProcessor::SetProcessors(TArrayView<UMassProcessor*> InProcessorInstances, const TSharedPtr<FMassEntityManager>& EntityManager)
{
	// figure out dependencies
	FMassProcessorDependencySolver Solver(InProcessorInstances);
	TArray<FMassProcessorOrderInfo> SortedProcessors;
	Solver.ResolveDependencies(SortedProcessors, EntityManager);

	UpdateProcessorsCollection(SortedProcessors);

	if (Solver.IsSolvingForSingleThread() == false)
	{
		BuildFlatProcessingGraph(SortedProcessors);
	}
}

void UMassCompositeProcessor::BuildFlatProcessingGraph(TConstArrayView<FMassProcessorOrderInfo> SortedProcessors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BuildFlatProcessingGraph);
#if !MASS_DO_PARALLEL
	UE_LOGF(LogMass, Warning
		, "MassCompositeProcessor::BuildFlatProcessingGraph is not expected to run in a single-threaded Mass setup. The flat graph will not be used at runtime.");
#endif // MASS_DO_PARALLEL

	FlatProcessingGraph.Reset();

	// this part is creating an ordered, flat list of processors that can be executed in sequence
	// with subsequent task only depending on the elements prior on the list
	TMap<FName, int32> NameToDependencyIndex;
	NameToDependencyIndex.Reserve(SortedProcessors.Num());
	TArray<int32> SuperGroupDependency;
	for (const FMassProcessorOrderInfo& Element : SortedProcessors)
	{
		NameToDependencyIndex.Add(Element.Name, FlatProcessingGraph.Num());

		// we don't expect to get any "group" nodes here. If it happens it indicates a bug in dependency solving
		checkSlow(Element.Processor);
		FDependencyNode& Node = FlatProcessingGraph.Add_GetRef({ Element.Name, Element.Processor });
		Node.Dependencies.Reserve(Element.Dependencies.Num());
		for (FName DependencyName : Element.Dependencies)
		{
			checkSlow(DependencyName.IsNone() == false);
			Node.Dependencies.Add(NameToDependencyIndex.FindChecked(DependencyName));
		}
#if WITH_MASSENTITY_DEBUG
		Node.SequenceIndex = Element.SequenceIndex;
#endif // WITH_MASSENTITY_DEBUG
	}

#if WITH_MASSENTITY_DEBUG
	FScopedCategoryAndVerbosityOverride LogOverride(TEXT("LogMass"), ELogVerbosity::Log);
	UE_LOGF(LogMass, Log, "%ls flat processing graph:", *GroupName.ToString());

	int32 Index = 0;
	for (const FDependencyNode& ProcessingNode : FlatProcessingGraph)
	{
		FString DependenciesDesc;
		for (const int32 DependencyIndex : ProcessingNode.Dependencies)
		{
			DependenciesDesc += FString::Printf(TEXT("%d, "), DependencyIndex);
		}
		if (ProcessingNode.Processor)
		{
			UE_LOGF(LogMass, Log, "[%2d]%*ls%ls%ls%ls", Index, ProcessingNode.SequenceIndex * 2, TEXT(""), *ProcessingNode.Processor->GetProcessorName()
				, DependenciesDesc.Len() > 0 ? TEXT(" depends on ") : TEXT(""), *DependenciesDesc);
		}
		++Index;
	}

	bDebugLogNewProcessingGraph = UE::Mass::Debug::bLogNewProcessingGraph;
#endif // WITH_MASSENTITY_DEBUG
}

void UMassCompositeProcessor::UpdateProcessorsCollection(TArrayView<FMassProcessorOrderInfo> InOutOrderedProcessors, EProcessorExecutionFlags InWorldExecutionFlags)
{
	TArray<TObjectPtr<UMassProcessor>> ExistingProcessors(ChildPipeline.GetMutableProcessors());
	ChildPipeline.Reset();

	const UWorld* World = GetWorld();
	const UMassEntitySettings* Settings = GetDefault<UMassEntitySettings>();
	check(Settings);

	const EProcessorExecutionFlags WorldExecutionFlags = UE::Mass::Utils::DetermineProcessorExecutionFlags(World, InWorldExecutionFlags);
	const FMassProcessingPhaseConfig& PhaseConfig = Settings->GetProcessingPhaseConfig(ProcessingPhase);

	for (FMassProcessorOrderInfo& ProcessorInfo : InOutOrderedProcessors)
	{
		if (ensureMsgf(ProcessorInfo.NodeType == FMassProcessorOrderInfo::EDependencyNodeType::Processor, TEXT("Encountered unexpected FMassProcessorOrderInfo::EDependencyNodeType while populating %s"), *GetGroupName().ToString()))
		{
			checkSlow(ProcessorInfo.Processor);
			if (ProcessorInfo.Processor->ShouldExecute(WorldExecutionFlags))
			{
				// we want to reuse existing processors to maintain state. It's recommended to keep processors state-less
				// but we already have processors that do have some state, like signaling processors.
				// the following search only makes sense for "single instance" processors
				if (ProcessorInfo.Processor->ShouldAllowMultipleInstances() == false)
				{
					TObjectPtr<UMassProcessor>* FoundProcessor = ExistingProcessors.FindByPredicate([ProcessorClass = ProcessorInfo.Processor->GetClass()](TObjectPtr<UMassProcessor>& Element)
						{
							return Element && (Element->GetClass() == ProcessorClass);
						});

					if (FoundProcessor)
					{
						// overriding the stored value since the InOutOrderedProcessors can get used after the call and it 
						// needs to reflect the actual work performed
						ProcessorInfo.Processor = FoundProcessor->Get();
					}
				}

				CA_ASSUME(ProcessorInfo.Processor);
				ChildPipeline.AppendProcessor(*ProcessorInfo.Processor);
			}
		}
	}
}

FString UMassCompositeProcessor::GetProcessorName() const
{
	return GroupName.ToString();
}

void UMassCompositeProcessor::DebugOutputDescription(FOutputDevice& Ar, int32 Indent) const
{
#if WITH_MASSENTITY_DEBUG
	if (ChildPipeline.Num() == 0)
	{
		Ar.Logf(TEXT("%*sGroup %s: []"), Indent, TEXT(""), *GroupName.ToString());
	}
	else
	{
		Ar.Logf(TEXT("%*sGroup %s:"), Indent, TEXT(""), *GroupName.ToString());
		for (UMassProcessor* Proc : ChildPipeline.GetProcessors())
		{
			check(Proc);
			Ar.Logf(TEXT("\n"));
			Proc->DebugOutputDescription(Ar, Indent + 3);
		}
	}
#endif // WITH_MASSENTITY_DEBUG
}

void UMassCompositeProcessor::SetProcessingPhase(EMassProcessingPhase Phase)
{
	Super::SetProcessingPhase(Phase);
	for (UMassProcessor* Proc : ChildPipeline.GetMutableProcessors())
	{
		Proc->SetProcessingPhase(Phase);
	}
}

void UMassCompositeProcessor::SetGroupName(FName NewName)
{
	GroupName = NewName;
#if CPUPROFILERTRACE_ENABLED
	StatId = GroupName.ToString();
#endif
}

void UMassCompositeProcessor::AddGroupedProcessor(FName RequestedGroupName, UMassProcessor& Processor)
{
	if (RequestedGroupName.IsNone() || RequestedGroupName == GroupName)
	{
		ChildPipeline.AppendProcessor(Processor);
	}
	else
	{
		FString RemainingGroupName;
		UMassCompositeProcessor* GroupProcessor = FindOrAddGroupProcessor(RequestedGroupName, &RemainingGroupName);
		check(GroupProcessor);
		GroupProcessor->AddGroupedProcessor(FName(*RemainingGroupName), Processor);
	}
}

UMassCompositeProcessor* UMassCompositeProcessor::FindOrAddGroupProcessor(FName RequestedGroupName, FString* OutRemainingGroupName)
{
	UMassCompositeProcessor* GroupProcessor = nullptr;
	const FString NameAsString = RequestedGroupName.ToString();
	FString TopGroupName;
	if (NameAsString.Split(TEXT("."), &TopGroupName, OutRemainingGroupName))
	{
		RequestedGroupName = FName(*TopGroupName);
	}
	GroupProcessor = ChildPipeline.FindTopLevelGroupByName(RequestedGroupName);

	if (GroupProcessor == nullptr)
	{
		check(GetOuter());
		GroupProcessor = NewObject<UMassCompositeProcessor>(GetOuter());
		GroupProcessor->SetGroupName(RequestedGroupName);
		ChildPipeline.AppendProcessor(*GroupProcessor);
	}

	return GroupProcessor;
}

//-----------------------------------------------------------------------------
// DEPRECATED
//-----------------------------------------------------------------------------

void UMassProcessor::Initialize(UObject& Owner)
{
	FMassEntityManager* EntityManager = UE::Mass::Utils::GetEntityManager(Owner.GetWorld());
	if (ensureMsgf(EntityManager, TEXT("Unable to determine the current MassEntityManager")))
	{
		InitializeInternal(Owner, EntityManager->AsShared());
	}
}

void UMassCompositeProcessor::SetChildProcessors(TArray<UMassProcessor*>&& InProcessors)
{
	SetChildProcessors(MakeArrayView(InProcessors.GetData(), InProcessors.Num()));
}
