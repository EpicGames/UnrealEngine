// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassProcessorDependencySolver.h"
#include "MassProcessingTypes.h"
#include "MassProcessor.h"
#include "MassTypeManager.h"
#include "Logging/MessageLog.h"
#include "HAL/FileManager.h"
#include "Engine/World.h"
#if WITH_MASSENTITY_DEBUG
#include "Algo/Count.h"
#endif // WITH_MASSENTITY_DEBUG

#define LOCTEXT_NAMESPACE "Mass"

DEFINE_LOG_CATEGORY_STATIC(LogMassDependencies, Warning, All);

namespace UE::Mass::Private
{
	FString NameViewToString(TConstArrayView<FName> View)
	{
		if (View.Num() == 0)
		{
			return TEXT("[]");
		}
		FString ReturnVal = FString::Printf(TEXT("[%s"), *View[0].ToString());
		for (int i = 1; i < View.Num(); ++i)
		{
			ReturnVal += FString::Printf(TEXT(", %s"), *View[i].ToString());
		}
		return ReturnVal + TEXT("]");
	}

	bool DoArchetypeContainersOverlap(TConstArrayView<FMassArchetypeHandle> A, const TArray<FMassArchetypeHandle>& B)
	{
		for (const FMassArchetypeHandle& HandleA : A)
		{
			if (B.Contains(HandleA))
			{
				return true;
			}
		}
		return false;
	}

#if WITH_MASSENTITY_DEBUG
	void LogCycle(TArray<FMassProcessorDependencySolver::FNode> AllNodes, TConstArrayView<int32> CycleIndices, TArray<uint32>& InOutReportedCycleHashes)
	{
		check(CycleIndices.Num());
		// We extract unique indices involved in the cycle below.
		// Note that we want to preserve the order since it will provide more meaningful debugging context.
		// But we do find the "lowest node index" so that we can generate a deterministic
		// hash representing the cycle, regardless of which node was being processed when the cycle was found.
		// We use this information to not report the same cycle multiple times.

		// Finding the cycle start (the first node that has been encountered more than once)
		int32 CycleStartElementIndex = INDEX_NONE;
		for (const int32 CycleNodeIndex : CycleIndices)
		{
			if (Algo::Count(CycleIndices, CycleNodeIndex) > 1)
			{
				CycleStartElementIndex = CycleIndices.Find(CycleNodeIndex);
				break;
			}
		}

		// Finding the cycle length by finding the other occurrence of cycle-starting node
		const int32 CycleLength = MakeArrayView(&CycleIndices[CycleStartElementIndex + 1], CycleIndices.Num() - CycleStartElementIndex - 1).Find(CycleIndices[CycleStartElementIndex]) + 1;
		check(CycleLength > 0);

		TConstArrayView<int32> CycleView = MakeArrayView(&CycleIndices[CycleStartElementIndex], CycleLength);
		// Find the deterministic cycle start, only used for hash generation.
		const int32* MinElementIndex = Algo::MinElement(CycleView);
		check(MinElementIndex);
		const int32 LowestCycleElementIndex = CycleView.Find(*MinElementIndex);

		// Calculate cycle's hash
		int32 ElementIndex = LowestCycleElementIndex;
		uint32 CycleHash = static_cast<uint32>(CycleView[ElementIndex++]);
		for (int32 CycleCounter = 1; CycleCounter < CycleView.Num(); ++CycleCounter)
		{
			ElementIndex %= CycleView.Num();
			CycleHash = HashCombine(CycleHash, CycleView[ElementIndex++]);
		}

		if (InOutReportedCycleHashes.Find(CycleHash) == INDEX_NONE)
		{
			InOutReportedCycleHashes.Add(CycleHash);

			UE_LOGF(LogMassDependencies, Error, "Detected processing dependency cycle:");

			for (const int32 CycleNodeIndex : CycleView)
			{
				if (const UMassProcessor* Processor = AllNodes[CycleNodeIndex].Processor)
				{
					UE_LOGF(LogMassDependencies, Warning, "\t%ls, group: %ls, before: %ls, after %ls"
						, *Processor->GetName()
						, *Processor->GetExecutionOrder().ExecuteInGroup.ToString()
						, *NameViewToString(Processor->GetExecutionOrder().ExecuteBefore)
						, *NameViewToString(Processor->GetExecutionOrder().ExecuteAfter));
				}
				else
				{
					// group
					UE_LOGF(LogMassDependencies, Warning, "\tGroup %ls", *AllNodes[CycleNodeIndex].Name.ToString());
				}
			}
		}
	}
#endif // WITH_MASSENTITY_DEBUG

	bool bProcessorExecutionPriorityEnabled = true;
	bool bPickHigherPriorityNodesRegardlessOfRequirements = true;
	namespace
	{
		FAutoConsoleVariableRef ConsoleVariables[] =
		{
			FAutoConsoleVariableRef(
				TEXT("mass.dependencies.ProcessorExecutionPriorityEnabled"),
				bProcessorExecutionPriorityEnabled,
				TEXT("Controls whether UMassProcessor.ExecutionPriority value is being used during dependency calculations"),
				ECVF_Default)
			, FAutoConsoleVariableRef(
				TEXT("mass.dependencies.PickHigherPriorityNodesRegardlessOfRequirements"),
				bPickHigherPriorityNodesRegardlessOfRequirements,
				TEXT("If enabled, will result in lower priority nodes not being picked, even if they could run without obstructing anything else"),
				ECVF_Default)
		};
	}
} // namespace UE::Mass::Private

//----------------------------------------------------------------------//
//  FMassExecutionRequirements
//----------------------------------------------------------------------//
void FMassExecutionRequirements::Append(const FMassExecutionRequirements& Other)
{
	for (int i = 0; i < EMassAccessOperation::MAX; ++i)
	{
		Fragments[i] += Other.Fragments[i];
		IndirectFragments[i] += Other.IndirectFragments[i];
		ChunkFragments[i] += Other.ChunkFragments[i];
		SharedFragments[i] += Other.SharedFragments[i];
		RequiredSubsystems[i] += Other.RequiredSubsystems[i];
		SparseElements[i] += Other.SparseElements[i];
	}
	ConstSharedFragments.Read += Other.ConstSharedFragments.Read;

	RequiredAllTags += Other.RequiredAllTags;
	RequiredAnyTags += Other.RequiredAnyTags;
	RequiredNoneTags += Other.RequiredNoneTags;
	// note that we're deliberately ignoring optional tags, they play no role here.

	// signal that it requires recalculation;
	ResourcesUsedCount = INDEX_NONE;
}

void FMassExecutionRequirements::CountResourcesUsed()
{
	ResourcesUsedCount = ConstSharedFragments.Read.CountStoredTypes();

	for (int i = 0; i < EMassAccessOperation::MAX; ++i)
	{
		ResourcesUsedCount += Fragments[i].CountStoredTypes();
		ResourcesUsedCount += IndirectFragments[i].CountStoredTypes();
		ResourcesUsedCount += ChunkFragments[i].CountStoredTypes();
		ResourcesUsedCount += SharedFragments[i].CountStoredTypes();
		ResourcesUsedCount += RequiredSubsystems[i].CountStoredTypes();
		ResourcesUsedCount += SparseElements[i].CountStoredTypes();
	}
}

int32 FMassExecutionRequirements::GetTotalBitsUsedCount()
{
	CountResourcesUsed();

	return ResourcesUsedCount + RequiredAllTags.CountStoredTypes()
		+ RequiredAnyTags.CountStoredTypes() + RequiredNoneTags.CountStoredTypes();
}

bool FMassExecutionRequirements::IsEmpty() const
{
	return Fragments.IsEmpty() && IndirectFragments.IsEmpty() && ChunkFragments.IsEmpty()
		&& SharedFragments.IsEmpty() && ConstSharedFragments.IsEmpty() && RequiredSubsystems.IsEmpty()
		&& SparseElements.IsEmpty()
		&& RequiredAllTags.IsEmpty() && RequiredAnyTags.IsEmpty() && RequiredNoneTags.IsEmpty();
}

FMassArchetypeCompositionDescriptor FMassExecutionRequirements::AsCompositionDescriptor() const
{
	return FMassArchetypeCompositionDescriptor(Fragments.Read + Fragments.Write + IndirectFragments.Read + IndirectFragments.Write
		, RequiredAllTags + RequiredAnyTags
		, ChunkFragments.Read + ChunkFragments.Write
		, SharedFragments.Read + SharedFragments.Write
		, ConstSharedFragments.Read);
}

//----------------------------------------------------------------------//
//  FProcessorDependencySolver::FResourceUsage
//----------------------------------------------------------------------//
FMassProcessorDependencySolver::FResourceUsage::FResourceUsage(const TArray<FNode>& InAllNodes)
	: AllNodesView(InAllNodes)
{
	for (int i = 0; i < EMassAccessOperation::MAX; ++i)
	{
		FragmentsAccess[i].Access.AddZeroed(FMassFragmentBitSet::GetMaxNum());
		IndirectFragmentsAccess[i].Access.AddZeroed(FMassFragmentBitSet::GetMaxNum());
		ChunkFragmentsAccess[i].Access.AddZeroed(FMassChunkFragmentBitSet::GetMaxNum());
		SharedFragmentsAccess[i].Access.AddZeroed(FMassSharedFragmentBitSet::GetMaxNum());
		RequiredSubsystemsAccess[i].Access.AddZeroed(FMassExternalSubsystemBitSet::GetMaxNum());
		SparseElementsAccess[i].Access.AddZeroed(FMassElementBitSet::GetMaxNum());
	}
}

template<typename TBitSet>
void FMassProcessorDependencySolver::FResourceUsage::HandleElementType(TMassExecutionAccess<FResourceAccess>& ElementAccess
	, const TMassExecutionAccess<TBitSet>& TestedRequirements, FMassProcessorDependencySolver::FNode& InOutNode, const int32 NodeIndex, bool bSkipArchetypeOverlapTest)
{
	using UE::Mass::Private::DoArchetypeContainersOverlap;

	// when considering subsystem access we don't care about archetypes, so we cache the information whether
	// we're dealing with subsystems and use that to potentially short-circuit the checks below.
	constexpr bool bSubsystems = std::is_same_v<TBitSet, FMassExternalSubsystemBitSet>;
	bSkipArchetypeOverlapTest = bSkipArchetypeOverlapTest || bSubsystems;

	// for every bit set in TestedRequirements we do the following:
	// 1. For every read-only requirement we make InOutNode depend on the currently stored Writer of this resource
	//    - note that this operation is not destructive, meaning we don't destructively consume the data, since all 
	//      subsequent read access to the given resource will also depend on the Writer
	//    - note 2: we also fine tune what we store as a dependency for InOutNode by checking if InOutNode's archetype
	//      overlap with whoever the current Writer is 
	//    - this will result in InOutNode wait for the current Writer to finish before starting its own work and 
	//      that's exactly what we need to do to avoid accessing data while it's potentially being written
	// 2. For every read-write requirement we make InOutNode depend on all the readers and writers currently stored. 
	//    - once that's done we clean currently stored Readers and Writers since every subsequent operation on this 
	//      resource will be blocked by currently considered InOutNode (as the new Writer)
	//    - again, we do check corresponding archetype collections overlap
	//    - similarly to the read operation waiting on write operations in pt 1. we want to hold off the write 
	//      operations to be performed by InOutNode until all currently registered (and conflicting) writers and readers 
	//      are done with their operations 
	// 3. For all accessed resources we store information that InOutNode is accessing it
	//    - we do this so that the following nodes know that they'll have to wait for InOutNode if an access 
	//      conflict arises. 

	// 1. For every read only requirement we make InOutNode depend on the currently stored Writer of this resource
	for (auto It = TestedRequirements.Read.GetIndexIterator(); It; ++It)
	{
		for (int32 UserIndex : ElementAccess.Write.Access[*It].Users)
		{
			if (UserIndex != NodeIndex && (bSkipArchetypeOverlapTest || DoArchetypeContainersOverlap(AllNodesView[UserIndex].ValidArchetypes, InOutNode.ValidArchetypes)))
			{
				InOutNode.OriginalDependencies.Add(UserIndex);
			}
		}
	}

	// 2. For every read-write requirement we make InOutNode depend on all the readers and writers currently stored. 
	for (auto It = TestedRequirements.Write.GetIndexIterator(); It; ++It)
	{
		for (int32 i = ElementAccess.Read.Access[*It].Users.Num() - 1; i >= 0; --i)
		{
			const int32 UserIndex = ElementAccess.Read.Access[*It].Users[i];
			if (UserIndex != NodeIndex && (bSkipArchetypeOverlapTest || DoArchetypeContainersOverlap(AllNodesView[UserIndex].ValidArchetypes, InOutNode.ValidArchetypes)))
			{	
				InOutNode.OriginalDependencies.Add(UserIndex);
				ElementAccess.Read.Access[*It].Users.RemoveAtSwap(i);
			}
		}

		for (int32 i = ElementAccess.Write.Access[*It].Users.Num() - 1; i >= 0; --i)
		{
			const int32 UserIndex = ElementAccess.Write.Access[*It].Users[i];
			if (UserIndex != NodeIndex && (bSkipArchetypeOverlapTest || DoArchetypeContainersOverlap(AllNodesView[UserIndex].ValidArchetypes, InOutNode.ValidArchetypes)))
			{
				InOutNode.OriginalDependencies.Add(UserIndex);
				ElementAccess.Write.Access[*It].Users.RemoveAtSwap(i);
			}
		}
	}

	// 3. For all accessed resources we store information that InOutNode is accessing it
	for (auto It = TestedRequirements.Read.GetIndexIterator(); It; ++It)
	{
		// mark Element at index indicated by It as being used in mode EMassAccessOperation(i) by NodeIndex
		ElementAccess.Read.Access[*It].Users.Add(NodeIndex);
	}
	for (auto It = TestedRequirements.Write.GetIndexIterator(); It; ++It)
	{
		// mark Element at index indicated by It as being used in mode EMassAccessOperation(i) by NodeIndex
		ElementAccess.Write.Access[*It].Users.Add(NodeIndex);
	}
}

template<typename TBitSet>
bool FMassProcessorDependencySolver::FResourceUsage::CanAccess(const TMassExecutionAccess<TBitSet>& StoredElements, const TMassExecutionAccess<TBitSet>& TestedElements)
{
	// see if there's an overlap of tested write operations with existing read & write operations, as well as 
	// tested read operations with existing write operations
	
	return !(
		// if someone's already writing to what I want to write
		TestedElements.Write.HasAny(StoredElements.Write)
		// or if someone's already reading what I want to write
		|| TestedElements.Write.HasAny(StoredElements.Read)
		// or if someone's already writing what I want to read
		|| TestedElements.Read.HasAny(StoredElements.Write)
	);
}

bool FMassProcessorDependencySolver::FResourceUsage::HasArchetypeConflict(TMassExecutionAccess<FResourceAccess> ElementAccess, const TArray<FMassArchetypeHandle>& InArchetypes) const
{
	using UE::Mass::Private::DoArchetypeContainersOverlap;

	// this function is being run when we've already determined there's an access conflict on given ElementsAccess,
	// meaning whoever's asking is trying to access Elements that are already being used. We can still grant access 
	// though provided that none of the current users of Element access the same archetypes the querier does (as provided 
	// by InArchetypes).
	// @todo this operation could be even more efficient and precise if we tracked which operation (read/write) and which
	// specific Element were conflicting and the we could limit the check to that. That would however significantly 
	// complicate the code and would require a major refactor to keep things clean.
	for (int i = 0; i < EMassAccessOperation::MAX; ++i)
	{
		for (const FResourceUsers& Resource : ElementAccess[i].Access)
		{
			for (const int32 UserIndex : Resource.Users)
			{
				if (DoArchetypeContainersOverlap(AllNodesView[UserIndex].ValidArchetypes, InArchetypes))
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool FMassProcessorDependencySolver::FResourceUsage::CanAccessRequirements(
	const FMassExecutionRequirements& Tested, const TArray<FMassArchetypeHandle>& InArchetypes) const
{
	// Direct fragments: allow archetype fallback.
	const bool bDirectOK =
		(CanAccess<FMassFragmentBitSet>(Requirements.Fragments, Tested.Fragments) ||
			!HasArchetypeConflict(FragmentsAccess, InArchetypes));

	// Indirect fragments: archetype-agnostic -> no fallback permitted.
	const bool bIndirectOK =
		CanAccess<FMassFragmentBitSet>(Requirements.IndirectFragments, Tested.IndirectFragments);

	// Cross-type contention: whenever either side is indirect, treat as global (no archetype fallback).
	const bool bIndirectVsDirectOK =
		!(Tested.IndirectFragments.Write.HasAny(Requirements.Fragments.Write)
			|| Tested.IndirectFragments.Write.HasAny(Requirements.Fragments.Read)
			|| Tested.IndirectFragments.Read.HasAny(Requirements.Fragments.Write));

	const bool bDirectVsIndirectOK =
		!(Tested.Fragments.Write.HasAny(Requirements.IndirectFragments.Write)
			|| Tested.Fragments.Write.HasAny(Requirements.IndirectFragments.Read)
			|| Tested.Fragments.Read.HasAny(Requirements.IndirectFragments.Write));

	const bool bChunkOK =
		(CanAccess<FMassChunkFragmentBitSet>(Requirements.ChunkFragments, Tested.ChunkFragments) ||
			!HasArchetypeConflict(ChunkFragmentsAccess, InArchetypes));

	const bool bSharedOK =
		(CanAccess<FMassSharedFragmentBitSet>(Requirements.SharedFragments, Tested.SharedFragments) ||
			!HasArchetypeConflict(SharedFragmentsAccess, InArchetypes));

	const bool bSubsystemsOK =
		CanAccess<FMassExternalSubsystemBitSet>(Requirements.RequiredSubsystems, Tested.RequiredSubsystems);

	const bool bSparseElementsOK =
		CanAccess<FMassElementBitSet>(Requirements.SparseElements, Tested.SparseElements);

	return bDirectOK && bIndirectOK && bIndirectVsDirectOK && bDirectVsIndirectOK
		&& bChunkOK && bSharedOK && bSubsystemsOK && bSparseElementsOK;
}


void FMassProcessorDependencySolver::FResourceUsage::SubmitNode(const int32 NodeIndex, FNode& InOutNode)
{
	// Populate Node data access dependencies:
	// Fragments Direct-Direct:
	HandleElementType<FMassFragmentBitSet>(FragmentsAccess, InOutNode.Requirements.Fragments, InOutNode, NodeIndex);
	// Fragments Indirect-Indirect:
	HandleElementType<FMassFragmentBitSet>(IndirectFragmentsAccess, InOutNode.Requirements.IndirectFragments, InOutNode, NodeIndex, /*bSkipArchetypeOverlapTest*/ true);
	// Fragments Indirect-Direct:
	HandleElementType<FMassFragmentBitSet>(IndirectFragmentsAccess, InOutNode.Requirements.Fragments, InOutNode, NodeIndex, /*bSkipArchetypeOverlapTest*/ true);
	// Fragments Direct-Indirect:
	HandleElementType<FMassFragmentBitSet>(FragmentsAccess, InOutNode.Requirements.IndirectFragments, InOutNode, NodeIndex, /*bSkipArchetypeOverlapTest*/ true);
	// Chunk Fragments:
	HandleElementType<FMassChunkFragmentBitSet>(ChunkFragmentsAccess, InOutNode.Requirements.ChunkFragments, InOutNode, NodeIndex);
	// (Mutable) Shared Fragments:
	HandleElementType<FMassSharedFragmentBitSet>(SharedFragmentsAccess, InOutNode.Requirements.SharedFragments, InOutNode, NodeIndex);
	// Subsystems:
	HandleElementType<FMassExternalSubsystemBitSet>(RequiredSubsystemsAccess, InOutNode.Requirements.RequiredSubsystems, InOutNode, NodeIndex);
	// Sparse Elements (archetype-agnostic like indirect fragments):
	HandleElementType<FMassElementBitSet>(SparseElementsAccess, InOutNode.Requirements.SparseElements, InOutNode, NodeIndex, /*bSkipArchetypeOverlapTest*/ true);

	// note that on purpose we're not pushing ConstSharedFragments - those are always only read, no danger of conflicting access
	// so there's no point in tracking them

	Requirements.Append(InOutNode.Requirements);
}

//----------------------------------------------------------------------//
//  FProcessorDependencySolver::FNode
//----------------------------------------------------------------------//
bool FMassProcessorDependencySolver::FNode::IncreaseWaitingNodesCount(TArrayView<FMassProcessorDependencySolver::FNode> InAllNodes
	, const int32 IterationsLimit, TArray<int32>& OutCycleIndices)
{
	// cycle-protection check. If true it means we have a cycle and the whole algorithm result will be unreliable 
	if (IterationsLimit < 0)
	{
		OutCycleIndices.Add(NodeIndex);
		return false;
	}

	++TotalWaitingNodes;

	for (const int32 DependencyIndex : OriginalDependencies)
	{
		check(&InAllNodes[DependencyIndex] != this);
		if (InAllNodes[DependencyIndex].IncreaseWaitingNodesCount(InAllNodes, IterationsLimit - 1, OutCycleIndices) == false)
		{
			OutCycleIndices.Add(NodeIndex);
			return false;
		}
	}

	return true;
}

bool FMassProcessorDependencySolver::FNode::IncreaseWaitingNodesCountAndPriority(TArrayView<FNode> InAllNodes, const int32 IterationsLimit
	, TArray<int32>& OutCycleIndices, const int32 InChildPriority)
{
	// cycle-protection check. If true it means we have a cycle and the whole algorithm result will be unreliable 
	if (IterationsLimit < 0)
	{
		OutCycleIndices.Add(NodeIndex);
		return false;
	}

	++TotalWaitingNodes;
	UpdateExecutionPriority(InChildPriority);

	for (const int32 DependencyIndex : OriginalDependencies)
	{
		check(&InAllNodes[DependencyIndex] != this);
		if (InAllNodes[DependencyIndex].IncreaseWaitingNodesCountAndPriority(InAllNodes, IterationsLimit - 1, OutCycleIndices, MaxExecutionPriority) == false)
		{
			OutCycleIndices.Add(NodeIndex);
			return false;
		}
	}

	return true;
}

//----------------------------------------------------------------------//
//  FProcessorDependencySolver
//----------------------------------------------------------------------//
FMassProcessorDependencySolver::FMassProcessorDependencySolver(TArrayView<UMassProcessor* const> InProcessors, const bool bIsGameRuntime)
	: Processors(InProcessors)
	, bGameRuntime(bIsGameRuntime)
{}

bool FMassProcessorDependencySolver::PerformSolverStep(FResourceUsage& ResourceUsage, TArray<int32>& InOutIndicesRemaining, TArray<int32>& OutNodeIndices)
{
	int32 AcceptedNodeIndex = INDEX_NONE;
	int32 FallbackAcceptedNodeIndex = INDEX_NONE;

	const int32 HighestPriority = AllNodes[InOutIndicesRemaining[0]].MaxExecutionPriority;
	for (int32 i = 0; i < InOutIndicesRemaining.Num(); ++i)
	{
		const int32 NodeIndex = InOutIndicesRemaining[i];
		if (AllNodes[NodeIndex].TransientDependencies.Num() == 0)
		{
			// if we're solving dependencies for a single thread use we don't need to fine-tune the order based on resources nor archetypes
			if (bSingleThreadTarget || ResourceUsage.CanAccessRequirements(AllNodes[NodeIndex].Requirements, AllNodes[NodeIndex].ValidArchetypes))
			{
				AcceptedNodeIndex = NodeIndex;
				break;
			}
			else if (FallbackAcceptedNodeIndex == INDEX_NONE)
			{
				// if none of the nodes left can "cleanly" execute (i.e. without conflicting with already stored nodes)
				// we'll just pick this one up and go with it. 
				FallbackAcceptedNodeIndex = NodeIndex;
			}
			else if (UE::Mass::Private::bPickHigherPriorityNodesRegardlessOfRequirements 
				&& AllNodes[NodeIndex].MaxExecutionPriority < HighestPriority)
			{
				// subsequent nodes are of lower execution priority, we break now and will use FallbackAcceptedNodeIndex
				check(FallbackAcceptedNodeIndex != INDEX_NONE);
				checkf(UE::Mass::Private::bProcessorExecutionPriorityEnabled == true
					, TEXT("We never expect to hit this case when execution priorities are disabled - all nodes should have the same priority."))
				break;
			}
		}
	}

	if (AcceptedNodeIndex != INDEX_NONE || FallbackAcceptedNodeIndex != INDEX_NONE)
	{
		const int32 NodeIndex = AcceptedNodeIndex != INDEX_NONE ? AcceptedNodeIndex : FallbackAcceptedNodeIndex;

		FNode& Node = AllNodes[NodeIndex];

		// Note that this is not an unexpected event and will happen during every dependency solving. It's a part 
		// of the algorithm. We initially look for all the things we can run without conflicting with anything else. 
		// But that can't last forever, at some point we'll end up in a situation where every node left waits for 
		// something that has been submitted already. Then we just pick one of the waiting ones (the one indicated by 
		// FallbackAcceptedNodeIndex), "run it" and proceed.
		UE_CLOGF(AcceptedNodeIndex == INDEX_NONE, LogMassDependencies, Verbose, "No dependency-free node can be picked, due to resource requirements. Picking %ls as the next node."
			, *Node.Name.ToString());

		ResourceUsage.SubmitNode(NodeIndex, Node);
		InOutIndicesRemaining.RemoveSingle(NodeIndex);
		OutNodeIndices.Add(NodeIndex);
		for (const int32 DependencyIndex : Node.OriginalDependencies)
		{
			Node.SequencePositionIndex = FMath::Max(Node.SequencePositionIndex, AllNodes[DependencyIndex].SequencePositionIndex);
		}
		++Node.SequencePositionIndex;

		for (const int32 RemainingNodeIndex : InOutIndicesRemaining)
		{
			AllNodes[RemainingNodeIndex].TransientDependencies.RemoveSingleSwap(NodeIndex, EAllowShrinking::No);
		}
		
		return true;
	}

	return false;
}

void FMassProcessorDependencySolver::CreateSubGroupNames(FName InGroupName, TArray<FString>& SubGroupNames)
{
	// the function will convert composite group name into a series of progressively more precise group names
	// so "A.B.C" will result in ["A", "A.B", "A.B.C"]

	SubGroupNames.Reset();
	FString GroupNameAsString = InGroupName.ToString();
	FString TopGroupName;

	while (GroupNameAsString.Split(TEXT("."), &TopGroupName, &GroupNameAsString))
	{
		SubGroupNames.Add(TopGroupName);
	}
	SubGroupNames.Add(GroupNameAsString);
	
	for (int i = 1; i < SubGroupNames.Num(); ++i)
	{
		SubGroupNames[i] = FString::Printf(TEXT("%s.%s"), *SubGroupNames[i - 1], *SubGroupNames[i]);
	}
}

int32 FMassProcessorDependencySolver::CreateNodes(UMassProcessor& Processor)
{
	check(Processor.GetClass());
	// for processors supporting multiple instances we use processor name rather than processor's class name for
	// dependency calculations. This makes the user responsible for fine-tuning per-processor dependencies. 
	const FName ProcName = Processor.ShouldAllowMultipleInstances() 
		? Processor.GetFName()
		: Processor.GetClass()->GetFName();

	if (const int32* NodeIndexPtr = NodeIndexMap.Find(ProcName))
	{
		if (Processor.ShouldAllowMultipleInstances())
		{
			UE_LOGF(LogMassDependencies, Warning, "%s Processor %ls, name %ls, already registered. This processor class does suport duplicates, but individual instances need to have a unique name."
				, __FUNCTION__, *Processor.GetFullName(), *ProcName.ToString());
		}
		else
		{
			UE_LOGF(LogMassDependencies, Warning, "%s Processor %ls already registered. Duplicates are not supported by this processor class."
				, __FUNCTION__, *ProcName.ToString());
		}
		return *NodeIndexPtr;
	}

	const FMassProcessorExecutionOrder& ExecutionOrder = Processor.GetExecutionOrder();

	// first figure out the groups so that the group nodes come before the processor nodes, this is required for child
	// nodes to inherit group's dependencies like in scenarios where some processor required to ExecuteBefore a given group
	int32 ParentGroupNodeIndex = INDEX_NONE;
	if (ExecutionOrder.ExecuteInGroup.IsNone() == false)
	{
		TArray<FString> AllGroupNames;
		CreateSubGroupNames(ExecutionOrder.ExecuteInGroup, AllGroupNames);
	
		check(AllGroupNames.Num() > 0);

		for (const FString& GroupName : AllGroupNames)
		{
			const FName GroupFName(GroupName);
			int32* LocalGroupIndex = NodeIndexMap.Find(GroupFName);
			// group name hasn't been encountered yet - create it
			if (LocalGroupIndex == nullptr)
			{
				int32 NewGroupNodeIndex = AllNodes.Num();
				NodeIndexMap.Add(GroupFName, NewGroupNodeIndex);
				FNode& GroupNode = AllNodes.Add_GetRef({ GroupFName, nullptr, NewGroupNodeIndex });
				// just ignore depending on the dummy "root" node
				if (ParentGroupNodeIndex != INDEX_NONE)
				{
					GroupNode.OriginalDependencies.Add(ParentGroupNodeIndex);
					AllNodes[ParentGroupNodeIndex].SubNodeIndices.Add(NewGroupNodeIndex);
				}

				ParentGroupNodeIndex = NewGroupNodeIndex;
			}
			else
			{	
				ParentGroupNodeIndex = *LocalGroupIndex;
			}

		}
	}

	const int32 NodeIndex = AllNodes.Num();
	NodeIndexMap.Add(ProcName, NodeIndex);
	FNode& ProcessorNode = AllNodes.Add_GetRef({ ProcName, &Processor, NodeIndex });

	ProcessorNode.ExecuteAfter.Append(ExecutionOrder.ExecuteAfter);
	ProcessorNode.ExecuteBefore.Append(ExecutionOrder.ExecuteBefore);
	Processor.ExportRequirements(ProcessorNode.Requirements);
	// we're clearing out information about the thread-safe subsystems since
	// we don't need to consider them while tracking subsystem access for thread-safety purposes
	ProcessorNode.Requirements.RequiredSubsystems.Write -= MultiThreadedSystemsBitSet;
	ProcessorNode.Requirements.RequiredSubsystems.Read -= MultiThreadedSystemsBitSet;
	ProcessorNode.Requirements.CountResourcesUsed();
	ProcessorNode.MaxExecutionPriority = UE::Mass::Private::bProcessorExecutionPriorityEnabled
		? Processor.GetExecutionPriority()
		: 0;

	if (ParentGroupNodeIndex > 0)
	{
		AllNodes[ParentGroupNodeIndex].SubNodeIndices.Add(NodeIndex);
	}

	return NodeIndex;
}

void FMassProcessorDependencySolver::BuildDependencies()
{
	// at this point we have collected all the known processors and groups in AllNodes so we can transpose 
	// A.ExecuteBefore(B) type of dependencies into B.ExecuteAfter(A)
	for (int32 NodeIndex = 0; NodeIndex < AllNodes.Num(); ++NodeIndex)
	{
		for (int i = 0; i < AllNodes[NodeIndex].ExecuteBefore.Num(); ++i)
		{
			const FName BeforeDependencyName = AllNodes[NodeIndex].ExecuteBefore[i];
			int32 DependentNodeIndex = INDEX_NONE;
			int32* DependentNodeIndexPtr = NodeIndexMap.Find(BeforeDependencyName);
			if (DependentNodeIndexPtr == nullptr)
			{
				// missing dependency. Adding a "dummy" node representing those to still support ordering based on missing groups or processors 
				// For example, if Processor A and B declare dependency, respectively, "Before C" and "After C" we still 
				// expect A to come before B regardless of whether C exists or not.
				
				DependentNodeIndex = AllNodes.Num();
				NodeIndexMap.Add(BeforeDependencyName, DependentNodeIndex);
				AllNodes.Add({ BeforeDependencyName, nullptr, DependentNodeIndex });

				UE_LOGF(LogMassDependencies, Log, "Unable to find dependency \"%ls\" declared by %ls. Creating a dummy dependency node."
					, *BeforeDependencyName.ToString(), *AllNodes[NodeIndex].Name.ToString());
			}
			else
			{
				DependentNodeIndex = *DependentNodeIndexPtr;
			}

			check(AllNodes.IsValidIndex(DependentNodeIndex));
			AllNodes[DependentNodeIndex].ExecuteAfter.Add(AllNodes[NodeIndex].Name);
		}
		AllNodes[NodeIndex].ExecuteBefore.Reset();
	}

	// at this point all nodes contain:
	// - single "original dependency" pointing at its parent group
	// - ExecuteAfter populated with node names

	// Now, for every Name in ExecuteAfter we do the following:
	//	if Name represents a processor, add it as "original dependency"
	//	else, if Name represents a group:
	//		- append all group's child node names to ExecuteAfter
	// 
	for (int32 NodeIndex = 0; NodeIndex < AllNodes.Num(); ++NodeIndex)
	{
		for (int i = 0; i < AllNodes[NodeIndex].ExecuteAfter.Num(); ++i)
		{
			const FName AfterDependencyName = AllNodes[NodeIndex].ExecuteAfter[i];
			int32* PrerequisiteNodeIndexPtr = NodeIndexMap.Find(AfterDependencyName);
			int32 PrerequisiteNodeIndex = INDEX_NONE;

			if (PrerequisiteNodeIndexPtr == nullptr)
			{
				// missing dependency. Adding a "dummy" node representing those to still support ordering based on missing groups or processors 
				// For example, if Processor A and B declare dependency, respectively, "Before C" and "After C" we still 
				// expect A to come before B regardless of whether C exists or not.

				PrerequisiteNodeIndex = AllNodes.Num();
				NodeIndexMap.Add(AfterDependencyName, PrerequisiteNodeIndex);
				AllNodes.Add({ AfterDependencyName, nullptr, PrerequisiteNodeIndex });

				UE_LOGF(LogMassDependencies, Log, "Unable to find dependency \"%ls\" declared by %ls. Creating a dummy dependency node."
					, *AfterDependencyName.ToString(), *AllNodes[NodeIndex].Name.ToString());
			}
			else
			{
				PrerequisiteNodeIndex = *PrerequisiteNodeIndexPtr;
			}

			const FNode& PrerequisiteNode = AllNodes[PrerequisiteNodeIndex];

			if (PrerequisiteNode.IsGroup())
			{
				for (int32 SubNodeIndex : PrerequisiteNode.SubNodeIndices)
				{
					AllNodes[NodeIndex].ExecuteAfter.AddUnique(AllNodes[SubNodeIndex].Name);
				}
			}
			else
			{
				AllNodes[NodeIndex].OriginalDependencies.AddUnique(PrerequisiteNodeIndex);
			}
		}

		// if this node is a group push all the dependencies down on all the children
		// by design all child nodes come after group nodes so the child nodes' dependencies have not been processed yet
		if (AllNodes[NodeIndex].IsGroup() && AllNodes[NodeIndex].SubNodeIndices.Num())
		{
			for (int32 PrerequisiteNodeIndex : AllNodes[NodeIndex].OriginalDependencies)
			{
				checkSlow(PrerequisiteNodeIndex != NodeIndex);
				// in case of processor nodes we can store it directly
				if (AllNodes[PrerequisiteNodeIndex].IsGroup() == false)
				{
					for (int32 ChildNodeIndex : AllNodes[NodeIndex].SubNodeIndices)
					{
						AllNodes[ChildNodeIndex].OriginalDependencies.AddUnique(PrerequisiteNodeIndex);
					}
				}
				// special case - if dependency is a group and we haven't processed that group yet, we need to add it by name
				else if (PrerequisiteNodeIndex > NodeIndex)
				{
					const FName& PrerequisiteName = AllNodes[PrerequisiteNodeIndex].Name;
					for (int32 ChildNodeIndex : AllNodes[NodeIndex].SubNodeIndices)
					{
						AllNodes[ChildNodeIndex].ExecuteAfter.AddUnique(PrerequisiteName);
					}
				}
			}
		}
	}
}

void FMassProcessorDependencySolver::LogNode(const FNode& Node, int Indent)
{
	using UE::Mass::Private::NameViewToString;

	if (Node.IsGroup())
	{
		UE_LOGF(LogMassDependencies, Log, "%*ls%ls before:%ls after:%ls", Indent, TEXT(""), *Node.Name.ToString()
			, *NameViewToString(Node.ExecuteBefore)
			, *NameViewToString(Node.ExecuteAfter));

		for (const int32 NodeIndex : Node.SubNodeIndices)
		{
			LogNode(AllNodes[NodeIndex], Indent + 4);
		}
	}
	else
	{
		CA_ASSUME(Node.Processor); // as implied by Node.IsGroup() == false
		UE_LOGF(LogMassDependencies, Log, "%*ls%ls before:%ls after:%ls", Indent, TEXT(""), *Node.Name.ToString()
			, *NameViewToString(Node.Processor->GetExecutionOrder().ExecuteBefore)
			, *NameViewToString(Node.Processor->GetExecutionOrder().ExecuteAfter));
	}
}

void FMassProcessorDependencySolver::Solve(TArray<FMassProcessorOrderInfo>& OutResult)
{
	using UE::Mass::Private::NameViewToString;

	if (AllNodes.Num() == 0)
	{
		return;
	}

	// for more efficient cycle detection and breaking it will be useful to know how many
	// nodes do not depend on anything - we can use this number as a limit for the longest dependency chain
	int32 TotalDependingNodes = 0;

	for (FNode& Node : AllNodes)
	{
		Node.TransientDependencies = Node.OriginalDependencies;
		Node.TotalWaitingNodes = 0;
		TotalDependingNodes += (Node.OriginalDependencies.Num() > 0) ? 1 : 0;
	}

	TArray<int32> CycleIndices;
#if WITH_MASSENTITY_DEBUG
	TArray<uint32> ReportedCycleHashes;
#endif // WITH_MASSENTITY_DEBUG

	TArray<int32> IndicesRemaining;
	// @todo code duplication in this if-else block is temporary, will be reduced to one or the other
	// once the bProcessorExecutionPriorityEnabled feature is accepted or removed. 
	if (UE::Mass::Private::bProcessorExecutionPriorityEnabled)
	{
		IndicesRemaining.Reserve(AllNodes.Num());
		for (int32 NodeIndex = 0; NodeIndex < AllNodes.Num(); ++NodeIndex)
		{
			// skip all the group nodes, all group dependencies have already been converted to individual processor dependencies
			if (AllNodes[NodeIndex].IsGroup() == false)
			{
				IndicesRemaining.Add(NodeIndex);
				if (AllNodes[NodeIndex].IncreaseWaitingNodesCountAndPriority(AllNodes, TotalDependingNodes, CycleIndices) == false)
				{
#if WITH_MASSENTITY_DEBUG
					// we have a cycle. Report it here
					UE::Mass::Private::LogCycle(AllNodes, CycleIndices, ReportedCycleHashes);
#endif // WITH_MASSENTITY_DEBUG
					CycleIndices.Reset();
				}
			}
		}

		IndicesRemaining.Sort([this](const int32 IndexA, const int32 IndexB){
			return AllNodes[IndexA].MaxExecutionPriority > AllNodes[IndexB].MaxExecutionPriority
				|| (AllNodes[IndexA].MaxExecutionPriority == AllNodes[IndexB].MaxExecutionPriority
					&& AllNodes[IndexA].TotalWaitingNodes > AllNodes[IndexB].TotalWaitingNodes);
		});
	}
	else
	{
		IndicesRemaining.Reserve(AllNodes.Num());
		for (int32 NodeIndex = 0; NodeIndex < AllNodes.Num(); ++NodeIndex)
		{
			// skip all the group nodes, all group dependencies have already been converted to individual processor dependencies
			if (AllNodes[NodeIndex].IsGroup() == false)
			{
				IndicesRemaining.Add(NodeIndex);
				if (AllNodes[NodeIndex].IncreaseWaitingNodesCount(AllNodes, TotalDependingNodes, CycleIndices) == false)
				{
#if WITH_MASSENTITY_DEBUG
					// we have a cycle. Report it here
					UE::Mass::Private::LogCycle(AllNodes, CycleIndices, ReportedCycleHashes);
#endif // WITH_MASSENTITY_DEBUG
					CycleIndices.Reset();
				}
			}
		}

		IndicesRemaining.Sort([this](const int32 IndexA, const int32 IndexB){
			return AllNodes[IndexA].TotalWaitingNodes > AllNodes[IndexB].TotalWaitingNodes;
		});
	}

	// this is where we'll be tracking what's being accessed by whom
	FResourceUsage ResourceUsage(AllNodes);

	TArray<int32> SortedNodeIndices;
	SortedNodeIndices.Reserve(AllNodes.Num());

	while (IndicesRemaining.Num())
	{
		const bool bStepSuccessful = PerformSolverStep(ResourceUsage, IndicesRemaining, SortedNodeIndices);

		if (bStepSuccessful == false)
		{
			UE_LOGF(LogMassDependencies, Error, "Encountered processing dependency cycle - cutting the chain at an arbitrary location.");

			// remove first dependency
			// note that if we're in a cycle handling scenario every node does have some dependencies left
			const int32 DependencyNodeIndex = AllNodes[IndicesRemaining[0]].TransientDependencies.Pop(EAllowShrinking::No);
			// we need to remove this dependency from original dependencies as well, otherwise we'll still have the cycle
			// in the data being produces as a result of the whole algorithm
			AllNodes[IndicesRemaining[0]].OriginalDependencies.Remove(DependencyNodeIndex);
		}
	}

	// now we have the desired order in SortedNodeIndices. We have to traverse it to add to OutResult
	for (int i = 0; i < SortedNodeIndices.Num(); ++i)
	{
		const int32 NodeIndex = SortedNodeIndices[i];

		TArray<FName> DependencyNames;
		for (const int32 DependencyIndex : AllNodes[NodeIndex].OriginalDependencies)
		{
			DependencyNames.AddUnique(AllNodes[DependencyIndex].Name);
		}

		// at this point we expect SortedNodeIndices to only point to regular processors (i.e. no groups)
		if (ensure(AllNodes[NodeIndex].Processor != nullptr))
		{
			OutResult.Add({ AllNodes[NodeIndex].Name, AllNodes[NodeIndex].Processor, FMassProcessorOrderInfo::EDependencyNodeType::Processor, DependencyNames, AllNodes[NodeIndex].SequencePositionIndex });
		}
	}
}

void FMassProcessorDependencySolver::ResolveDependencies(TArray<FMassProcessorOrderInfo>& OutResult, TSharedPtr<FMassEntityManager> EntityManager, FMassProcessorDependencySolver::FResult* InOutOptionalResult)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("Mass ResolveDependencies");

	if (Processors.Num() == 0)
	{
		return;
	}

	FScopedCategoryAndVerbosityOverride LogOverride(TEXT("LogMass"), ELogVerbosity::Log);

	if (InOutOptionalResult)
	{
		DependencyGraphFileName = InOutOptionalResult->DependencyGraphFileName;
	}

	UE_LOGF(LogMassDependencies, Log, "Gathering dependencies data:");

	AllNodes.Reset();
	NodeIndexMap.Reset();
	// as the very first node we add a "root" node that represents the "top level group" and also simplifies the rest
	// of the lookup code - if a processor declares it's in group None or depends on Node it we don't need to check that 
	// explicitly. 
	AllNodes.Add(FNode(FName(), nullptr, 0));
	NodeIndexMap.Add(FName(), 0);

	const bool bCreateVirtualArchetypes = (!EntityManager);
	if (bCreateVirtualArchetypes)
	{
		// create FMassEntityManager instance that we'll use to sort out processors' overlaps
		// the idea for this is that for every processor we have we create an archetype matching given processor's requirements. 
		// Once that's done we have a collection of "virtual" archetypes our processors expect. Then we ask every processor 
		// to cache the archetypes they'd accept, using processors' owned queries. The idea is that some of the nodes will 
		// end up with more than just the virtual archetype created for that specific node. The practice proved the idea correct. 
		EntityManager = MakeShareable(new FMassEntityManager());
	}

	GatherSubsystemInformation(EntityManager->GetTypeManager());

	// gather the processors information first
	for (UMassProcessor* Processor : Processors)
	{
		if (Processor == nullptr)
		{
			UE_LOGF(LogMassDependencies, Warning, "%ls nullptr found in Processors collection being processed", ANSI_TO_TCHAR(__FUNCTION__));
			continue;
		}

		const int32 ProcessorNodeIndex = CreateNodes(*Processor);

		if (bCreateVirtualArchetypes)
		{
			// this line is a part of a nice trick we're doing here utilizing EntityManager's archetype creation based on 
			// what each processor expects, and EntityQuery's capability to cache archetypes matching its requirements (used below)
			EntityManager->CreateArchetype(AllNodes[ProcessorNodeIndex].Requirements.AsCompositionDescriptor());

			if (AllNodes[ProcessorNodeIndex].Processor->HasEntityCreationRequirements())
			{
				AllNodes[ProcessorNodeIndex].Processor->ClearArchetypesCreatedHandles();

				// ResolveRequestedArchetypes creates the actual archetypes
				AllNodes[ProcessorNodeIndex].Processor->ResolveRequestedArchetypes(*EntityManager.Get());
			}
		}
	}

	UE_LOGF(LogMassDependencies, Verbose, "Pruning processors...");

	int32 PrunedProcessorsCount = 0;
	for (FNode& Node : AllNodes)
	{
		if (Node.IsGroup() == false)
		{
			CA_ASSUME(Node.Processor); // as implied by Node.IsGroup() == false

			const bool bDoQueryBasedPruning = Node.Processor->ShouldAllowQueryBasedPruning(bGameRuntime);

			// we gather archetypes for processors that have queries OR allow query-based pruning.
			// The main point of this condition is to allow calling GetArchetypesMatchingOwnedQueries
			// on pruning-supporting processors, while having no queries - that will emmit a warning
			// that will let the user know their processor is misconfigured.
			// We do collect archetype information for the processors that never get pruned because we're
			// using this information for the dependency calculations, regardless of ShouldAllowQueryBasedPruning
			if (bDoQueryBasedPruning || Node.Processor->GetOwnedQueriesNum() || Node.Processor->HasEntityCreationRequirements())
			{
				// for each processor-representing node we cache information on which archetypes among the once we've created 
				// above (see the EntityManager.CreateArchetype call in the previous loop) match this processor. 
				Node.Processor->GetArchetypesMatchingOwnedQueries(*EntityManager.Get(), Node.ValidArchetypes);
				Node.Processor->ExportArchetypesCreated(Node.ValidArchetypes);
			}

			if (bCreateVirtualArchetypes)
			{
				// reset the archetype handles if they're virtual
				Node.Processor->ClearArchetypesCreatedHandles();
			}

			// prune the archetype-less processors
			if (Node.ValidArchetypes.Num() == 0 && bDoQueryBasedPruning)
			{
				UE_LOGF(LogMassDependencies, Verbose, "\t%ls", *Node.Processor->GetName());

				if (InOutOptionalResult)
				{
					InOutOptionalResult->PrunedProcessors.Add(Node.Processor);
				}

				// clearing out the processor will result in the rest of the algorithm to treat it as a group - we still 
				// want to preserve the configured ExecuteBefore and ExecuteAfter dependencies
				Node.Processor = nullptr;
				++PrunedProcessorsCount;
			}
		}
	}

	UE_LOGF(LogMassDependencies, Verbose, "Number of processors pruned: %d", PrunedProcessorsCount);

	check(AllNodes.Num());
	LogNode(AllNodes[0]);

	BuildDependencies();

	// now none of the processor nodes depend on groups - we replaced these dependencies with depending directly 
	// on individual processors. However, we keep the group nodes around since we store the dependencies via index, so 
	// removing nodes would mess that up. Solve below ignores group nodes and OutResult will not have any groups once its done.

	Solve(OutResult);

	UE_LOGF(LogMassDependencies, Verbose, "Dependency order:");
	for (const FMassProcessorOrderInfo& Info : OutResult)
	{
		UE_LOGF(LogMassDependencies, Verbose, "\t%ls", *Info.Name.ToString());
	}

	int32 MaxSequenceLength = 0;
	for (FNode& Node : AllNodes)
	{
		MaxSequenceLength = FMath::Max(MaxSequenceLength, Node.SequencePositionIndex);
	}

	UE_LOGF(LogMassDependencies, Verbose, "Max sequence length: %d", MaxSequenceLength);

	if (InOutOptionalResult)
	{
		InOutOptionalResult->MaxSequenceLength = MaxSequenceLength;
		InOutOptionalResult->ArchetypeDataVersion = EntityManager->GetArchetypeDataVersion();
	}
}

bool FMassProcessorDependencySolver::IsResultUpToDate(const FMassProcessorDependencySolver::FResult& InResult, TSharedPtr<FMassEntityManager> EntityManager)
{
	// Any archetype version change invalidates the cached result. New archetypes can introduce
	// fragment-access overlap between processors whose ValidArchetypes lists were disjoint at
	// solve time (see FResourceUsage::HandleElementType - Read<->Write dependency edges are
	// gated on DoArchetypeContainersOverlap). A previously edge-free pair becomes a data race
	// if a single new archetype matches both processors' queries, so a pure version check is
	// the correct invalidation signal - this also subsumes pruned-processor revival, since any
	// revival requires a new matching archetype, which always bumps the version.
	return EntityManager && (InResult.ArchetypeDataVersion == EntityManager->GetArchetypeDataVersion());
}

void FMassProcessorDependencySolver::GatherSubsystemInformation(const UE::Mass::FTypeManager& TypeManager)
{
	using namespace UE::Mass;

	if (TypeManager.IsEmpty())
	{
		return;
	}

	for (FTypeManager::FSubsystemTypeConstIterator SubsystemTypeIterator = TypeManager.MakeSubsystemIterator(); SubsystemTypeIterator; ++SubsystemTypeIterator)
	{
		if (const FTypeInfo* TypeInfo = TypeManager.GetTypeInfo(*SubsystemTypeIterator))
		{
			const FSubsystemTypeTraits* SubsystemTraits = TypeInfo->GetAsSystemTraits();
			check(SubsystemTraits);
			if (SubsystemTraits->bThreadSafeWrite)
			{
				const UClass* SubsystemClass = SubsystemTypeIterator->GetClass();
				check(SubsystemClass);
				MultiThreadedSystemsBitSet.Add(SubsystemClass);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE