// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassQueryExecutor.h"
#include "MassExecutionContext.h"

namespace UE::Mass
{

FQueryExecutor::FQueryExecutor(FMassEntityQuery& InQuery, UObject* InLogOwner)
	: BoundQuery(&InQuery)
	, LogOwner(InLogOwner)
{
}

FMassEntityQuery FQueryExecutor::DummyQuery;

FQueryExecutor::FQueryExecutor()
	: BoundQuery(&DummyQuery)
{
}

void FQueryExecutor::CallExecute(FMassExecutionContext& Context)
{
#if WITH_MASSENTITY_DEBUG
	ValidateAccessors();
#endif

	AccessorsPtr->SetupForExecute(Context);

	Execute(Context);
}

void FQueryExecutor::ConfigureQuery(FMassSubsystemRequirements& ProcessorRequirements)
{
	AccessorsPtr->ConfigureQuery(*BoundQuery, ProcessorRequirements);
}

#if WITH_MASSENTITY_DEBUG
void FQueryExecutor::ValidateAccessors()
{
	const UPTRINT ExecutorStart = reinterpret_cast<UPTRINT>(this);
	const UPTRINT ExecutorEnd = ExecutorStart + DebugSize;

	if (AccessorsPtr)
	{
		const UPTRINT AccessorsStart = reinterpret_cast<UPTRINT>(AccessorsPtr);
		checkf(ExecutorStart <= AccessorsStart && AccessorsStart <= ExecutorEnd, TEXT("Accessors assigned to a FQueryExecutor must be member variables of that struct."));
	}
}
#endif //WITH_MASSENTITY_DEBUG

} // namespace UE::Mass