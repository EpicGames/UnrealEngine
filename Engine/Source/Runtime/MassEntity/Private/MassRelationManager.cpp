// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRelationManager.h"
#include "MassEntityBuilder.h"
#include "MassEntityTypes.h"
#include "MassEntityManager.h"
#include "MassRelationObservers.h"
#include "MassExecutionContext.h"
#include "MassCommands.h"
#include "MassArchetypeGroupCommands.h"
#include "Algo/RemoveIf.h"

DECLARE_CYCLE_STAT(TEXT("Mass Relation IsSubject"), STAT_Mass_IsSubject, STATGROUP_Mass);
DECLARE_CYCLE_STAT(TEXT("Mass Relation IsSubject Recursive"), STAT_Mass_IsSubjectRecursive, STATGROUP_Mass);

namespace UE::Mass
{
	namespace Relations::Private
	{
		inline void HandleRole(const FMassEntityManager& EntityManager, FRelationData& RelationData, const FMassEntityHandle& OperatorEntity, UE::Mass::ERelationRole Role
			, const FMassEntityHandle& RoleEntity, const FMassEntityHandle& RelationEntity
			, TArray<FMassEntityHandle>& InOutRelationEntitiesToDestroy)
		{
			const int32 RoleIndex = static_cast<int32>(Role);
			if (RelationData.Traits.RoleTraits[RoleIndex].RequiresExternalMapping == EExternalMappingRequired::Yes)
			{
				TArray<FMassRelationRoleInstanceHandle>& RoleData = RelationData.RoleMap.FindOrAdd(OperatorEntity)[RoleIndex];

				const FMassRelationRoleInstanceHandle NewInstanceHandle = FMassRelationRoleInstanceHandle::Create(Role, RoleEntity, RelationEntity);

				// if objects are exclusive we need to destroy the previous instance
				if (RelationData.Traits.RoleTraits[RoleIndex].bExclusive
					&& RoleData.Num()
					&& RoleData[0] != NewInstanceHandle)
				{
					FMassRelationRoleInstanceHandle PreviousInstance = RoleData.Pop(EAllowShrinking::No);
					const FMassEntityHandle PreviousRelationEntity = PreviousInstance.GetRelationEntityHandle(EntityManager);
					InOutRelationEntitiesToDestroy.Add(PreviousRelationEntity);
				}

#if WITH_MASSENTITY_DEBUG
				const int32 ExistingIndex = RoleData.Find(NewInstanceHandle);
				ensureMsgf(ExistingIndex == INDEX_NONE, TEXT("Given relation instance handle is already present in role mapping"));
#endif // WITH_MASSENTITY_DEBUG

				RoleData.Add(NewInstanceHandle);
			}
		}

		void AddRoleInstance(const FMassEntityManager& EntityManager, FRelationData& RelationData, const FMassEntityHandle Subject, const FMassEntityHandle Object, const FMassEntityHandle RelationEntityHandle
			, TArray<FMassEntityHandle>& InOutRelationEntitiesToDestroy)
		{
			HandleRole(EntityManager, RelationData, Subject, ERelationRole::Object, Object, RelationEntityHandle, InOutRelationEntitiesToDestroy);
			HandleRole(EntityManager, RelationData, Object, ERelationRole::Subject, Subject, RelationEntityHandle, InOutRelationEntitiesToDestroy);
		}
	}

	struct FScopedRecursiveLimit
	{
		FScopedRecursiveLimit(int32& Limit)
			: LimitRef(Limit)
		{
			++LimitRef;
			check(LimitRef < 10);
		}
		~FScopedRecursiveLimit()
		{
			--LimitRef;
		}
		int32& LimitRef;
	};

	//-----------------------------------------------------------------------------
	// FRelationData
	//-----------------------------------------------------------------------------
	FRelationData::FRelationData(const FRelationTypeTraits& InTraits)
		: Traits(InTraits)
	{
	}

	TArray<FMassEntityHandle> FRelationData::GetParticipants(const FMassEntityManager& EntityManager, const FMassEntityHandle RoleEntity, const ERelationRole QueriedRole) const
	{
		TArray<FMassEntityHandle> ReturnSubjects;
		if (const FRelationInstanceData* InstanceData = RoleMap.Find(RoleEntity))
		{
			ReturnSubjects.Reserve((*InstanceData)[QueriedRole].Num());
		}
		ForEachParticipant(EntityManager, RoleEntity, QueriedRole, [&ReturnSubjects](FMassEntityHandle Handle)
		{
			ReturnSubjects.Add(Handle);
		});
		return ReturnSubjects;
	}

	void FRelationData::ForEachParticipant(const FMassEntityManager& EntityManager, const FMassEntityHandle RoleEntity,
		const ERelationRole QueriedRole, TFunctionRef<void(FMassEntityHandle)> Callback) const
	{
		const int32 QueriedRoleIndex = static_cast<int32>(QueriedRole);

		ensureMsgf(Traits.RoleTraits[QueriedRoleIndex].RequiresExternalMapping == EExternalMappingRequired::Yes,
			TEXT("Fetching relation participants for role %s, while role traits explicitly prevent generic mapping"),
			*LexToString(QueriedRole));

		if (const FRelationInstanceData* InstanceData = RoleMap.Find(RoleEntity))
		{
			for (const FMassRelationRoleInstanceHandle RelationInstanceHandle : (*InstanceData)[QueriedRoleIndex])
			{
				const FMassEntityHandle RoleEntityHandle = RelationInstanceHandle.GetRoleEntityHandle(EntityManager);
				if (RoleEntityHandle.IsValid())
				{
					Callback(RoleEntityHandle);
				}
			}
		}
	}

	bool FRelationData::HasParticipants(const FMassEntityManager& EntityManager, const FMassEntityHandle RoleEntity,
		const ERelationRole QueriedRole) const
	{
		const int32 QueriedRoleIndex = static_cast<int32>(QueriedRole);
		if (const FRelationInstanceData* InstanceData = RoleMap.Find(RoleEntity))
		{
			for (const FMassRelationRoleInstanceHandle RelationInstanceHandle : (*InstanceData)[QueriedRoleIndex])
			{
				if (RelationInstanceHandle.GetRoleEntityHandle(EntityManager).IsValid())
				{
					return true;
				}
			}
		}
		return false;
	}

	//-----------------------------------------------------------------------------
	// FRelationManager::FHierarchyEntitiesContainer
	//-----------------------------------------------------------------------------
	void FRelationManager::FHierarchyEntitiesContainer::StoreUnique(const uint32 Depth, TArray<FMassEntityHandle>& InOutArray)
	{
		for (int32 HandleIndex = InOutArray.Num() - 1; HandleIndex >= 0; --HandleIndex)
		{
			bool bIsAlreadyInSet = false;
			ExistingElements.FindOrAdd(InOutArray[HandleIndex], &bIsAlreadyInSet);
			if (bIsAlreadyInSet)
			{
				InOutArray.RemoveAtSwap(HandleIndex);
			}
		}

		if (InOutArray.Num())
		{
			if (static_cast<uint32>(ContainerPerLevel.Num()) <= Depth)
			{
				ContainerPerLevel.AddDefaulted(Depth - ContainerPerLevel.Num() + 1);
			}
			ContainerPerLevel[Depth].Append(InOutArray);
		}
	}

	void FRelationManager::FHierarchyEntitiesContainer::StoreUnique(const uint32 Depth, const FMassEntityHandle Handle)
	{
		bool bIsAlreadyInSet = false;
		ExistingElements.FindOrAdd(Handle, &bIsAlreadyInSet);
		if (bIsAlreadyInSet == false)
		{
			if (static_cast<uint32>(ContainerPerLevel.Num()) <= Depth)
			{
				ContainerPerLevel.AddDefaulted(Depth - ContainerPerLevel.Num() + 1);
			}
			ContainerPerLevel[Depth].Add(Handle);
		}
	}

	//-----------------------------------------------------------------------------
	// FRelationManager
	//-----------------------------------------------------------------------------
	FRelationManager::FRelationManager(FMassEntityManager& InEntityManager)
		: EntityManager(InEntityManager)
		, TypeManager(InEntityManager.GetTypeManager())
	{
	}

	TArray<FMassEntityHandle> FRelationManager::CreateRelationInstances(const FTypeHandle RelationTypeHandle, TArrayView<FMassEntityHandle> Subjects, TArrayView<FMassEntityHandle> Objects)
	{
		if (!ensureMsgf(RelationTypeHandle.IsValid(), TEXT("Invalid RelationTypeHandle passed to %hs"), __FUNCTION__))
		{
			return {};
		}
		if (!ensureMsgf(Subjects.Num(), TEXT("Relation needs a valid Subjects entity")) 
			|| !ensureMsgf(Objects.Num(), TEXT("Relation needs a valid Objects entity"))
			// we allow number mismatch if there's a single object or subject, then we create 1-to-many relation
			|| !testableEnsureMsgf(Objects.Num() == Subjects.Num(), TEXT("Relation Objects count need match Subjects")))
		{
			return {};
		}

		FRelationData& RelationData = GetRelationDataChecked(RelationTypeHandle);

		int32 NumOfRelations = Subjects.Num();
		for (int32 PairIndex = 0; PairIndex < NumOfRelations; )
		{
			const FMassEntityHandle Object = Objects[PairIndex];
			const FMassEntityHandle Subject = Subjects[PairIndex];
			bool bPairValid = true;

			if (!testableEnsureMsgf(Subject.IsValid(), TEXT("Relation needs a valid Subject entity")) 
				|| !testableEnsureMsgf(Object.IsValid(), TEXT("Relation needs a valid Object entity")))
			{
				bPairValid = false;
			}
			if (!testableEnsureMsgf(RelationData.Traits.bHierarchical == false || Subject != Object
				, TEXT("Hierarchical relation requires the subject and object to be different")))
			{
				bPairValid = false;
			}
			if (!testableEnsureMsgf(IsSubjectOfRelation(RelationData, Subject, Object) == false
				, TEXT("Relation between the two entities already exists")))
			{
				bPairValid = false;
			}

			if (bPairValid)
			{
				++PairIndex;
			}
			else
			{
				if (PairIndex + 1 < NumOfRelations)
				{
					Swap(Objects[PairIndex], Objects[NumOfRelations - 1]);
					Swap(Subjects[PairIndex], Subjects[NumOfRelations - 1]);
				}
				--NumOfRelations;
			}
		}

		if (NumOfRelations == 0)
		{
			return {};
		}

		// some pairs have been removed, we need to update the views and pretend the invalid pairs were never there
		if (NumOfRelations != Subjects.Num())
		{
			Objects = MakeArrayView(&Objects[0], NumOfRelations);
			Subjects = MakeArrayView(&Subjects[0], NumOfRelations);
		}

		FEntityBuilder EntityBuilder(EntityManager);
		FMassRelationFragment& RelationFragment = EntityBuilder.Add_GetRef<FMassRelationFragment>();
		TNotNull<const UScriptStruct*> RelationTypeTag = RelationTypeHandle.GetScriptStruct();
		EntityBuilder.Add(RelationTypeTag);

		TSharedRef<UE::Mass::ObserverManager::FObserverLock> ObserversLock = EntityManager.GetOrMakeObserversLock();

		TArray<FMassEntityHandle> CreatedRelationEntities;
		TArray<FMassEntityHandle> RelationEntitiesToDestroy;

		for (int32 PairIndex = 0; PairIndex < NumOfRelations; ++PairIndex)
		{
			const FMassEntityHandle Object = Objects[FMath::Min(Objects.Num() - 1, PairIndex)];
			const FMassEntityHandle Subject = Subjects[FMath::Min(Subjects.Num() - 1, PairIndex)];

			RelationFragment.Subject = Subject;
			RelationFragment.Object = Object;

			CreatedRelationEntities.Add(EntityBuilder.CommitAndReprepare());
			const FMassEntityHandle RelationEntityHandle = CreatedRelationEntities.Last();

			Relations::Private::AddRoleInstance(EntityManager, RelationData, Subject, Object, RelationEntityHandle, RelationEntitiesToDestroy);
		}

		if (RelationEntitiesToDestroy.Num())
		{
			EntityManager.Defer().DestroyEntities(MoveTemp(RelationEntitiesToDestroy));
		}

		TNotNull<const UScriptStruct*> SubjectElement = RelationData.Traits.RoleTraits[static_cast<int32>(ERelationRole::Subject)].Element
			? TNotNull<const UScriptStruct*>(RelationData.Traits.RoleTraits[static_cast<int32>(ERelationRole::Subject)].Element)
			: RelationTypeTag;
		TNotNull<const UScriptStruct*> ObjectElement = RelationData.Traits.RoleTraits[static_cast<int32>(ERelationRole::Object)].Element
			? TNotNull<const UScriptStruct*>(RelationData.Traits.RoleTraits[static_cast<int32>(ERelationRole::Object)].Element)
			: RelationTypeTag;

		if (EntityManager.IsProcessing())
		{
			TUniquePtr<FMassCommandAddElement> ElementAddingCommand = MakeUnique<FMassCommandAddElement>(SubjectElement);

			ElementAddingCommand->Add(Subjects);

			if (ObjectElement != SubjectElement)
			{
				EntityManager.Defer().PushUniqueCommand(MoveTemp(ElementAddingCommand));
				ElementAddingCommand = MakeUnique<FMassCommandAddElement>(ObjectElement);
			}
			
			ElementAddingCommand->Add(Objects);

			EntityManager.Defer().PushUniqueCommand(MoveTemp(ElementAddingCommand));
		}
		else
		{
			EntityManager.BatchAddElementToEntities(Objects, ObjectElement);
			EntityManager.BatchAddElementToEntities(Subjects, SubjectElement);
		}

		if (RelationData.Traits.bHierarchical)
		{
			FHierarchyEntitiesContainer SubSubjects;
			uint32 MinDepth = static_cast<uint32>(-1);

			for (int32 PairIndex = 0; PairIndex < Subjects.Num(); ++PairIndex)
			{
				const FMassEntityHandle Object = Objects[FMath::Min(Objects.Num() - 1, PairIndex)];
				const FMassEntityHandle Subject = Subjects[FMath::Min(Subjects.Num() - 1, PairIndex)];

				FArchetypeGroupHandle ObjectsGroup = EntityManager.GetGroupForEntity(Object, RelationData.Traits.RegisteredGroupType);
				if (ObjectsGroup.IsValid() == false)
				{
					ObjectsGroup = FArchetypeGroupHandle(RelationData.Traits.RegisteredGroupType, 0);
				}

				const uint32 StartingDepth = ObjectsGroup.GetGroupID();
				MinDepth = FMath::Min(StartingDepth, MinDepth);

				SubSubjects.StoreUnique(StartingDepth, Object);
				SubSubjects.StoreUnique(StartingDepth + 1, Subject);

				GatherHierarchy(RelationData, Subject, SubSubjects, StartingDepth + 2);
			}

			if (EntityManager.IsProcessing())
			{
				for (uint32 Depth = MinDepth; Depth < SubSubjects.Num(); ++Depth)
				{
					FArchetypeGroupHandle GroupHandle(RelationData.Traits.RegisteredGroupType, Depth);

					EntityManager.Defer().PushCommand<UE::Mass::Command::FGroupEntities>(GroupHandle, MoveTemp(SubSubjects[Depth]));
				}
			}
			else
			{
				for (uint32 Depth = MinDepth; Depth < SubSubjects.Num(); ++Depth)
				{
					FArchetypeGroupHandle GroupHandle(RelationData.Traits.RegisteredGroupType, Depth);

					TArray<FMassArchetypeEntityCollection> CollectionsAtLevel;
					UE::Mass::Utils::CreateEntityCollections(EntityManager, SubSubjects[Depth]
						, FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates
						, CollectionsAtLevel);

					EntityManager.BatchGroupEntities(GroupHandle, CollectionsAtLevel);
				}
			}
		}

		return CreatedRelationEntities;
	}

	bool FRelationManager::DestroyRelationInstance(const FTypeHandle RelationTypeHandle, const FMassEntityHandle Subject, const FMassEntityHandle Object)
	{
		const FRelationData& RelationData = GetRelationDataChecked(RelationTypeHandle);
		ensure(RelationData.Traits.RoleTraits[0].RequiresExternalMapping != EExternalMappingRequired::No
			|| RelationData.Traits.RoleTraits[1].RequiresExternalMapping != EExternalMappingRequired::No);

		if (const FRelationData::FRelationInstanceData* InstanceData = RelationData.RoleMap.Find(Subject))
		{
			for (const FMassRelationRoleInstanceHandle RelationInstanceHandle : (*InstanceData)[static_cast<uint32>(ERelationRole::Object)])
			{
				if (RelationInstanceHandle.GetRoleEntityIndex() == Object.Index)
				{
					const FMassEntityHandle RelationEntityHandle = RelationInstanceHandle.GetRelationEntityHandle(EntityManager);
					if (RelationEntityHandle.IsValid())
					{
						EntityManager.DestroyEntity(RelationEntityHandle);
						return true;
					}
				}
			}
		}
		return false;
	}

	bool FRelationManager::DestroyRelationInstance(const FMassRelationRoleInstanceHandle RelationHandle) const
	{
		const FMassEntityHandle RelationEntityHandle = RelationHandle.GetRelationEntityHandle(EntityManager);
		if (RelationEntityHandle.IsValid())
		{
			EntityManager.DestroyEntity(RelationEntityHandle);
			return true;
		}
		return false;
	}

	TArray<FMassEntityHandle> FRelationManager::GetRelationSubjects(const FRelationData& RelationData, const FMassEntityHandle ObjectEntity) const
	{
		return RelationData.GetParticipants(EntityManager, ObjectEntity, ERelationRole::Subject);
	}

	TArray<FMassEntityHandle> FRelationManager::GetRelationObjects(const FRelationData& RelationData, const FMassEntityHandle SubjectEntity) const
	{
		return RelationData.GetParticipants(EntityManager, SubjectEntity, ERelationRole::Object);
	}

	void FRelationManager::ForEachRelationSubject(const FTypeHandle RelationTypeHandle, const FMassEntityHandle ObjectEntity,
		TFunctionRef<void(FMassEntityHandle)> Callback) const
	{
		const FRelationData& RelationData = GetRelationDataChecked(RelationTypeHandle);
		RelationData.ForEachParticipant(EntityManager, ObjectEntity, ERelationRole::Subject, Callback);
	}

	void FRelationManager::ForEachRelationObject(const FTypeHandle RelationTypeHandle, const FMassEntityHandle SubjectEntity,
		TFunctionRef<void(FMassEntityHandle)> Callback) const
	{
		const FRelationData& RelationData = GetRelationDataChecked(RelationTypeHandle);
		RelationData.ForEachParticipant(EntityManager, SubjectEntity, ERelationRole::Object, Callback);
	}

	bool FRelationManager::HasRelationSubjects(const FTypeHandle RelationTypeHandle, const FMassEntityHandle ObjectEntity) const
	{
		const FRelationData& RelationData = GetRelationDataChecked(RelationTypeHandle);
		return RelationData.HasParticipants(EntityManager, ObjectEntity, ERelationRole::Subject);
	}

	bool FRelationManager::HasRelationObjects(const FTypeHandle RelationTypeHandle, const FMassEntityHandle SubjectEntity) const
	{
		const FRelationData& RelationData = GetRelationDataChecked(RelationTypeHandle);
		return RelationData.HasParticipants(EntityManager, SubjectEntity, ERelationRole::Object);
	}

	void FRelationManager::GatherHierarchy(const FRelationData& RelationData, const FMassEntityHandle SubjectHandle, FHierarchyEntitiesContainer& SubSubjects, const uint32 Depth) const
	{
		// getting subject's subjects
		TArray<FMassEntityHandle> LocalSubjects = GetRelationSubjects(RelationData, SubjectHandle);

		if (LocalSubjects.Num() > 0)
		{
			// this call will store the handles in LocalSubjects, but only the ones that have not been
			// stored earlier. This prevents the function getting stuck due to cycles.
			SubSubjects.StoreUnique(Depth, LocalSubjects);
			for (const FMassEntityHandle& SubSubjectHandle : LocalSubjects)
			{
				GatherHierarchy(RelationData, SubSubjectHandle, SubSubjects, Depth + 1);
			}
		}
	}

	void FRelationManager::GetRelationEntities(TConstArrayView<FMassRelationRoleInstanceHandle> RelationEntitiesContainer, TArray<FMassEntityHandle>& InOutEntityHandles) const
	{
		InOutEntityHandles.Reserve(RelationEntitiesContainer.Num() + InOutEntityHandles.Num());
		for (const FMassRelationRoleInstanceHandle& Handle : RelationEntitiesContainer)
		{
			InOutEntityHandles.Add(Handle.GetRelationEntityHandle(EntityManager));
		}
	}

	TArray<FMassEntityHandle> FRelationManager::GetRelationEntities(TConstArrayView<FMassRelationRoleInstanceHandle> RelationEntitiesContainer) const
	{
		TArray<FMassEntityHandle> EntityHandles;
		GetRelationEntities(RelationEntitiesContainer, EntityHandles);
		return EntityHandles;
	}

	void FRelationManager::GetRoleEntities(TConstArrayView<FMassRelationRoleInstanceHandle> RelationEntitiesContainer, TArray<FMassEntityHandle>& InOutEntityHandles) const
	{
		InOutEntityHandles.Reserve(RelationEntitiesContainer.Num() + InOutEntityHandles.Num());
		for (const FMassRelationRoleInstanceHandle& Handle : RelationEntitiesContainer)
		{
			InOutEntityHandles.Add(Handle.GetRoleEntityHandle(EntityManager));
		}
	}

	TArray<FMassEntityHandle> FRelationManager::GetRoleEntities(TConstArrayView<FMassRelationRoleInstanceHandle> RelationEntitiesContainer) const
	{
		TArray<FMassEntityHandle> EntityHandles;
		GetRoleEntities(RelationEntitiesContainer, EntityHandles);
		return EntityHandles;
	}

	FRelationData& FRelationManager::CreateRelationData(const FTypeHandle RelationTypeHandle)
	{
		FRelationData* RelationData = RelationsDataMap.Find(RelationTypeHandle);
		if (ensureMsgf(RelationData == nullptr
			, TEXT("%hs: relation of type %s already registered"), __FUNCTION__, *RelationTypeHandle.GetFName().ToString()))
		{
			FRelationTypeTraits Traits = TypeManager.GetRelationTypeChecked(RelationTypeHandle);

			// @todo we should check every role's traits and see if RequiresExternalMapping == EExternalMappingRequired::No,
			// and if so, make sure all the other required bits are provided (like "other role getter").

			RelationData = &RelationsDataMap.Add(RelationTypeHandle, Traits);
		}

		return *RelationData;
	}

	bool FRelationManager::IsSubjectOfRelation(const FTypeHandle RelationTypeHandle, const FMassEntityHandle Subject, const FMassEntityHandle Object) const
	{
		check(RelationTypeHandle.IsValid());
		if (const FRelationData* RelationData = GetRelationData(RelationTypeHandle))
		{
			return IsSubjectOfRelation(*RelationData, Subject, Object);
		}
		return false;
	}

	bool FRelationManager::IsSubjectOfRelation(const FRelationData& RelationData, const FMassEntityHandle Subject, const FMassEntityHandle Object) const
	{
		SCOPE_CYCLE_COUNTER(STAT_Mass_IsSubject);

		if (Subject != Object
			&& Subject.IsValid()
			&& Object.IsValid())
		{
			if (const FRelationData::FRelationInstanceData* SubjectData = RelationData.RoleMap.Find(Subject))
			{
				for (const FMassRelationRoleInstanceHandle& RelationInstanceHandle : (*SubjectData)[static_cast<int32>(ERelationRole::Object)])
				{
					if (RelationInstanceHandle.GetRoleEntityHandle(EntityManager) == Object)
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	bool FRelationManager::IsSubjectOfRelationRecursive(const FTypeHandle RelationTypeHandle, const FMassEntityHandle Subject, const FMassEntityHandle Object) const
	{
		check(RelationTypeHandle.IsValid());
		if (const FRelationData* RelationData = GetRelationData(RelationTypeHandle))
		{
			return IsSubjectOfRelationRecursive(*RelationData, Subject, Object);
		}
		return false;
	}

	bool FRelationManager::IsSubjectOfRelationRecursive(const FRelationData& RelationData, const FMassEntityHandle Subject, const FMassEntityHandle Object) const
	{
		// @todo the function can be optimized based on information whether Subject or Object is exclusive. If both are, it can be optimized even further.
		// for now naive implementation
		SCOPE_CYCLE_COUNTER(STAT_Mass_IsSubjectRecursive);

		if (Subject != Object
			&& Subject.IsValid()
			&& Object.IsValid())
		{
			TArray<FMassEntityHandle> Subjects;
			TArray<FMassEntityHandle> Objects;
			Subjects.Add(Subject);
			
			constexpr int32 IterationsLimit = 64;
			for (int32 SubjectIndex = 0; SubjectIndex < Subjects.Num() && ensure(SubjectIndex < IterationsLimit); ++SubjectIndex)
			{
				if (const FRelationData::FRelationInstanceData* SubjectData = RelationData.RoleMap.Find(Subjects[SubjectIndex]))
				{
					GetRoleEntities((*SubjectData)[static_cast<int32>(ERelationRole::Object)], Objects);

					if (Objects.Find(Object) != INDEX_NONE)
					{
						return true;
					}

					for (const FMassEntityHandle& ObjectHandle : Objects)
					{
						Subjects.AddUnique(ObjectHandle);
					}
					Objects.Reset();
				}
			}
		}

		return false;
	}

	void FRelationManager::OnRelationTypeRegistered(const FTypeHandle InRegisteredTypeHandle, const FRelationTypeTraits& RelationTypeTraits)
	{
		FRelationData& RelationData = CreateRelationData(InRegisteredTypeHandle);
		ensureMsgf(RelationData.Traits.bHierarchical == false || RelationData.Traits.RegisteredGroupType.IsValid()
			, TEXT("Hierarchical relationships need a valid registered group size. Failed by relationship %s")
			, *RelationData.Traits.GetFName().ToString());

		TArray<TSubclassOf<UMassRelationObserver>> ObserversToCreate = 
		{
			RelationTypeTraits.RelationEntityCreationObserverClass.Get()
			, RelationTypeTraits.RelationEntityDestructionObserverClass.Get()
			, UMassRelationEntityGuardDog::StaticClass()
		};

		if (RelationTypeTraits.SubjectEntityDestructionObserverClass.Get() == UMassRelationRoleDestruction::StaticClass()
			&& RelationTypeTraits.ObjectEntityDestructionObserverClass.Get() == UMassRelationRoleDestruction::StaticClass())
		{
			UMassRelationRoleDestruction::AddObserverInstances(EntityManager.GetObserverManager(), InRegisteredTypeHandle, RelationData.Traits);
		}
		else
		{
			ObserversToCreate.Add(RelationTypeTraits.SubjectEntityDestructionObserverClass.Get());
			ObserversToCreate.Add(RelationTypeTraits.ObjectEntityDestructionObserverClass.Get());
		}

		for (const TSubclassOf<UMassRelationObserver>& ObserverClass : ObserversToCreate)
		{
			if (ObserverClass)
			{
				UMassRelationObserver* ObserverProcessor = NewObject<UMassRelationObserver>(EntityManager.GetOwner(), ObserverClass);
				if (ObserverProcessor->ConfigureRelationObserver(InRegisteredTypeHandle, RelationData.Traits))
				{
					EntityManager.GetObserverManager().AddObserverInstance(ObserverProcessor);
				}
			}
		}
	}
} // namespace UE::Mass

#undef WITH_RELATIONSHIP_VALIDATION
