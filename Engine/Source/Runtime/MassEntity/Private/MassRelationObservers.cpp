// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRelationObservers.h"
#include "MassTypeManager.h"
#include "MassExecutionContext.h"
#include "MassObserverManager.h"

//-----------------------------------------------------------------------------
// UMassRelationObserver
//-----------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassRelationObservers)
UMassRelationObserver::UMassRelationObserver()
	: EntityQuery(*this)
{
	bAutoRegisterWithObserverRegistry = false;
	ObservedOperations = EMassObservedOperationFlags::None;
}

void UMassRelationObserver::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);
}

void UMassRelationObserver::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	checkf(!ObservedTypes.IsEmpty(), TEXT("ObservedTypes must be populated before ConfigureQueries is called"));
	const UScriptStruct* PrimaryObservedType = GetSingleObservedTypeChecked();
	check(PrimaryObservedType);
	checkf(UE::Mass::IsA<FMassFragment>(PrimaryObservedType) || UE::Mass::IsA<FMassTag>(PrimaryObservedType)
		, TEXT("Only tags and fragments are valid observed types for RelationObservers. Received %s")
		, *PrimaryObservedType->GetName());

	if (UE::Mass::IsA<FMassTag>(PrimaryObservedType))
	{
		EntityQuery.AddTagRequirement(*PrimaryObservedType, EMassFragmentPresence::All);
	}
	else // UE::Mass::IsA<FMassFragment>(PrimaryObservedType)
	{
		EntityQuery.AddRequirement(PrimaryObservedType, ObservedTypeAccess);
	}

	if (bAutoAddRelationFragmentRequirement || bAutoAddRelationTagRequirement)
	{
		const UE::Mass::FRelationManager& RelationManager = EntityManager->GetRelationManager();
		const UE::Mass::FRelationData& RelationData = RelationManager.GetRelationDataChecked(RelationTypeHandle);

		if (bAutoAddRelationTagRequirement)
		{
			const TNotNull<const UScriptStruct*> RelationTag = RelationData.Traits.GetRelationTagType();
			if (RelationTag != PrimaryObservedType)
			{
				EntityQuery.AddTagRequirement(*RelationTag, EMassFragmentPresence::All);
			}
		}

		if (bAutoAddRelationFragmentRequirement)
		{
			const TNotNull<const UScriptStruct*> RelationFragmentType = RelationData.Traits.RelationFragmentType;
			if (UE::Mass::IsA<FMassFragment>(RelationFragmentType) && RelationFragmentType != PrimaryObservedType)
			{
				EntityQuery.AddRequirement(RelationFragmentType, RelationFragmentAccessType, EMassFragmentPresence::All);
			}
		}
	}

	DebugDescription = FString::Printf(TEXT("RelationType: %s ObservedType: %s Operation: %s")
		, *RelationTypeHandle.ToString(), *PrimaryObservedType->GetName()
		, *LexToString(ObservedOperations));
}

bool UMassRelationObserver::ConfigureRelationObserver(UE::Mass::FTypeHandle InRegisteredTypeHandle, const UE::Mass::FRelationTypeTraits& Traits)
{
	ObservedTypes.Reset();
	ObservedTypes.Add(Traits.RelationFragmentType);
	RelationTypeHandle = InRegisteredTypeHandle;
	return true;
}

//-----------------------------------------------------------------------------
// UMassRelationEntityCreation
//-----------------------------------------------------------------------------
UMassRelationEntityCreation::UMassRelationEntityCreation()
{
	ObservedOperations = EMassObservedOperationFlags::CreateEntity;
	ExecutionPriority = RelationCreationObserverExecutionPriority;
}

void UMassRelationEntityCreation::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	/*
	// here's example code showing how the freshly-created relation entity can be utilized.
	// at the current state of MassRelations implementation we don't need to do anything here. 

	struct FRelationInstanceRegistration
	{
		FMassEntityHandle SubjectHandle;
		FMassEntityHandle ObjectHandle;
		FMassEntityHandle RelationEntityHandle;
	};

	TArray<FRelationInstanceRegistration> RelationInstances;
	EntityQuery.ForEachEntityChunk(Context, [&RelationInstances, ObservedType = GetSingleObservedTypeChecked()](FMassExecutionContext& ExecutionContext)
	{
		TConstArrayView<FMassRelationFragment> RelationFragments = ExecutionContext.GetFragmentView<FMassRelationFragment>(ObservedType);
		for (FMassExecutionContext::FEntityIterator EntityIt = ExecutionContext.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			RelationInstances.Add({
				RelationFragments[EntityIt].Subject
				, RelationFragments[EntityIt].Object
				, ExecutionContext.GetEntity(EntityIt)
			});
		}
	});
	*/
}

//-----------------------------------------------------------------------------
// UMassRelationEntityGuardDog
//-----------------------------------------------------------------------------
UMassRelationEntityGuardDog::UMassRelationEntityGuardDog()
{
	ObservedOperations = EMassObservedOperationFlags::RemoveElement;
}

void UMassRelationEntityGuardDog::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& InContext)
{
#if WITH_MASSENTITY_DEBUG
	EntityQuery.CacheArchetypes();
	const FMassArchetypeEntityCollection& EntityCollection = InContext.GetEntityCollection();
	ensureMsgf(EntityQuery.GetArchetypes().Contains(EntityCollection.GetArchetype()) == false
		, TEXT("Trying to remove private-implementation-detail fragments from a relation entity is not supported!"));
#endif // WITH_MASSENTITY_DEBUG
}

//-----------------------------------------------------------------------------
// UMassRelationEntityDestruction
//-----------------------------------------------------------------------------
UMassRelationEntityDestruction::UMassRelationEntityDestruction()
{
	ObservedOperations = EMassObservedOperationFlags::DestroyEntity;
}

void UMassRelationEntityDestruction::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	TArray<TPair<FMassEntityHandle, FMassEntityHandle>> EntitiesToClearOut;

	EntityQuery.ForEachEntityChunk(Context, [&EntitiesToClearOut, ObservedType = GetSingleObservedTypeChecked()](FMassExecutionContext& ExecutionContext)
	{
		TConstArrayView<FMassRelationFragment> RelationFragments = ExecutionContext.GetFragmentView<FMassRelationFragment>(ObservedType);
		for (FMassExecutionContext::FEntityIterator EntityIt = ExecutionContext.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			EntitiesToClearOut.Add({RelationFragments[EntityIt].Subject, RelationFragments[EntityIt].Object});
		}
	});

	if (EntitiesToClearOut.Num())
	{
		UE::Mass::FRelationManager& RelationManager = Context.GetEntityManagerChecked().GetRelationManager();
		UE::Mass::FRelationData& RelationData = RelationManager.GetRelationDataChecked(RelationTypeHandle);

		for (const TPair<FMassEntityHandle, FMassEntityHandle>& Pair : EntitiesToClearOut)
		{
			if (UE::Mass::FRelationData::FRelationInstanceData* InstanceData = RelationData.RoleMap.Find(Pair.Get<0>()))
			{
				(*InstanceData)[static_cast<int32>(UE::Mass::ERelationRole::Object)].RemoveAllSwap(
					FMassRelationRoleInstanceHandle::FMassRelationRoleInstanceHandleFinder(Pair.Get<1>())
					, EAllowShrinking::No);
			}
			if (UE::Mass::FRelationData::FRelationInstanceData* InstanceData = RelationData.RoleMap.Find(Pair.Get<1>()))
			{
				(*InstanceData)[static_cast<int32>(UE::Mass::ERelationRole::Subject)].RemoveAllSwap(
					FMassRelationRoleInstanceHandle::FMassRelationRoleInstanceHandleFinder(Pair.Get<0>())
					, EAllowShrinking::No);
			}
		}
	}
}

//-----------------------------------------------------------------------------
// UMassRelationRoleDestruction
//-----------------------------------------------------------------------------
UMassRelationRoleDestruction::UMassRelationRoleDestruction()
{
	ObservedOperations = EMassObservedOperationFlags::DestroyEntity;
	bAutoAddRelationFragmentRequirement = false;
	bAutoAddRelationTagRequirement = false;
}

void UMassRelationRoleDestruction::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, ExecuteFunction);
}

void UMassRelationRoleDestruction::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::ConfigureQueries(EntityManager);
	// extend the query with "none" reliance on the relation entity's data fragment
	// @todo missing feature - this would cause the query to always fail to find archetypes. We need a way to
	//		add "external requirements" to queries, that we'd use to calculate dependencies, but not use for binding

	if (ensureMsgf(ExcludedRelationFragmentType
		, TEXT("We don't expect ExcludedRelationFragmentType to be null. Make sure ConfigureRelationObserver has been called first.")))
	{
		EntityQuery.AddRequirement(ExcludedRelationFragmentType, EMassFragmentAccess::None, EMassFragmentPresence::None);
	}
}

bool UMassRelationRoleDestruction::ConfigureRelationObserver(UE::Mass::FTypeHandle InRegisteredTypeHandle, const UE::Mass::FRelationTypeTraits& Traits)
{
	static_assert(static_cast<uint8>(UE::Mass::ERelationRole::MAX) == 2, "Current implementation relies on there being only two roles.");

	const int32 RoleAsIndex = RelationRole != UE::Mass::ERelationRole::MAX ? static_cast<int32>(RelationRole) : 0;
	const int32 OppositeRoleIndex = static_cast<int32>((RelationRole == UE::Mass::ERelationRole::Object) ? UE::Mass::ERelationRole::Subject : UE::Mass::ERelationRole::Object);
	bool bExecutionFunctionAssigned = false;

	// this processor is specifically implemented for handling destruction of regular entities
	// so we're using Traits.RelationFragmentType, which is only added to the relation entities,
	// to filter those out. This will ensure we only get entities that played a Role in the given relationship.
	ExcludedRelationFragmentType = Traits.RelationFragmentType;

	// note there we're deliberately not calling Super::ConfigureRelationObserver, since we're going to observe role-specific elements
	RelationTypeHandle = InRegisteredTypeHandle;
	ObservedTypes.Reset();
	if (Traits.RoleTraits[RoleAsIndex].Element)
	{
		ObservedTypes.Add(Traits.RoleTraits[RoleAsIndex].Element);
	}
	else
	{
		TNotNull<const UScriptStruct*> RelationTypeTag = RelationTypeHandle.GetScriptStruct();
		ObservedTypes.Add(RelationTypeTag);
	}

	switch (Traits.RoleTraits[RoleAsIndex].DestructionPolicy)
	{
		case UE::Mass::ERemovalPolicy::CleanUp:
			RelationFragmentAccessType = EMassFragmentAccess::None;
			if (RelationRole != UE::Mass::Relations::ERelationRole::MAX)
			{
				ExecuteFunction = [InRegisteredTypeHandle, OppositeRoleIndex](FMassExecutionContext& ExecutionContext)
					{
						UE::Mass::FRelationManager& RelationManager = ExecutionContext.GetEntityManagerChecked().GetRelationManager();
						UE::Mass::FRelationData& RelationData = RelationManager.GetRelationDataChecked(InRegisteredTypeHandle);
						for (FMassExecutionContext::FEntityIterator EntityIt = ExecutionContext.CreateEntityIterator(); EntityIt; ++EntityIt)
						{
							TArray<FMassRelationRoleInstanceHandle>& Container = RelationData.RoleMap.FindChecked(ExecutionContext.GetEntity(EntityIt))[OppositeRoleIndex];
							ExecutionContext.Defer().DestroyEntities(RelationManager.GetRelationEntities(Container));
							Container.Empty();
						}
					};
			}
			else
			{
				// this observer will only get called once and needs to handle both sides of the relation
				ExecuteFunction = [InRegisteredTypeHandle](FMassExecutionContext& ExecutionContext)
					{
						UE::Mass::FRelationManager& RelationManager = ExecutionContext.GetEntityManagerChecked().GetRelationManager();
						UE::Mass::FRelationData& RelationData = RelationManager.GetRelationDataChecked(InRegisteredTypeHandle);
						for (FMassExecutionContext::FEntityIterator EntityIt = ExecutionContext.CreateEntityIterator(); EntityIt; ++EntityIt)
						{
							UE::Mass::FRelationData::FRelationInstanceData& InstanceData = RelationData.RoleMap.FindChecked(ExecutionContext.GetEntity(EntityIt));

							TArray<FMassRelationRoleInstanceHandle>& SubjectsContainer = InstanceData[static_cast<uint8>(UE::Mass::ERelationRole::Subject)];
							// this will instigate destruction of relations where Entity is the object
							ExecutionContext.Defer().DestroyEntities(RelationManager.GetRelationEntities(SubjectsContainer));
							SubjectsContainer.Empty();

							TArray<FMassRelationRoleInstanceHandle>& ObjectsContainer = InstanceData[static_cast<uint8>(UE::Mass::ERelationRole::Object)];
							// this will instigate destruction of relations where Entity is the subject
							ExecutionContext.Defer().DestroyEntities(RelationManager.GetRelationEntities(ObjectsContainer));
							ObjectsContainer.Empty();
						}
					};
			}
			bExecutionFunctionAssigned = true;
			break;
		case UE::Mass::ERemovalPolicy::Destroy:
			RelationFragmentAccessType = EMassFragmentAccess::ReadOnly;
			// destroy the other side of the relationship, and the relation entity
			if (RelationRole != UE::Mass::Relations::ERelationRole::MAX)
			{
				ExecuteFunction = [InRegisteredTypeHandle, OppositeRoleIndex](FMassExecutionContext& ExecutionContext)
				{
					UE::Mass::FRelationManager& RelationManager = ExecutionContext.GetEntityManagerChecked().GetRelationManager();
					UE::Mass::FRelationData& RelationData = RelationManager.GetRelationDataChecked(InRegisteredTypeHandle);

					TArray<FMassEntityHandle> EntitiesToDestroy;

					for (FMassExecutionContext::FEntityIterator EntityIt = ExecutionContext.CreateEntityIterator(); EntityIt; ++EntityIt)
					{
						TArray<FMassRelationRoleInstanceHandle>& Container = RelationData.RoleMap.FindChecked(ExecutionContext.GetEntity(EntityIt))[OppositeRoleIndex];
						RelationManager.GetRelationEntities(Container, EntitiesToDestroy);
						RelationManager.GetRoleEntities(Container, EntitiesToDestroy);
						Container.Empty();
					}

					ExecutionContext.Defer().DestroyEntities(EntitiesToDestroy);
				};
			}
			else
			{
				// this implementation handles both sides of relations
				ExecuteFunction = [InRegisteredTypeHandle](FMassExecutionContext& ExecutionContext)
				{
					UE::Mass::FRelationManager& RelationManager = ExecutionContext.GetEntityManagerChecked().GetRelationManager();
					UE::Mass::FRelationData& RelationData = RelationManager.GetRelationDataChecked(InRegisteredTypeHandle);

					TArray<FMassEntityHandle> EntitiesToDestroy;

					for (FMassExecutionContext::FEntityIterator EntityIt = ExecutionContext.CreateEntityIterator(); EntityIt; ++EntityIt)
					{
						UE::Mass::FRelationData::FRelationInstanceData& InstanceData = RelationData.RoleMap.FindChecked(ExecutionContext.GetEntity(EntityIt));

						TArray<FMassRelationRoleInstanceHandle>& SubjectsContainer = InstanceData[static_cast<uint8>(UE::Mass::ERelationRole::Subject)];
						// this will instigate destruction of both the relation entities and the relation subjects
						RelationManager.GetRelationEntities(SubjectsContainer, EntitiesToDestroy);
						RelationManager.GetRoleEntities(SubjectsContainer, EntitiesToDestroy);
						SubjectsContainer.Empty();

						TArray<FMassRelationRoleInstanceHandle>& ObjectsContainer = InstanceData[static_cast<uint8>(UE::Mass::ERelationRole::Object)];
						// this will instigate destruction of both the relation entities and the relation objects
						RelationManager.GetRelationEntities(ObjectsContainer, EntitiesToDestroy);
						RelationManager.GetRoleEntities(ObjectsContainer, EntitiesToDestroy);
						ObjectsContainer.Empty();
					}

					ExecutionContext.Defer().DestroyEntities(EntitiesToDestroy);
				};
			}
			bExecutionFunctionAssigned = true;
			break;
		case UE::Mass::ERemovalPolicy::Splice:
			// use my Object as my Subject's Object, and vice versa
			ExecuteFunction = [InRegisteredTypeHandle](FMassExecutionContext& ExecutionContext)
			{
				static auto DestroyRelationEntitiesAndGetRoleEntities = [](
					const FMassExecutionContext& LocalExecutionContext
					, const UE::Mass::FRelationManager& RelationManager
					, const UE::Mass::FRelationData::FRelationInstanceData& InstanceData
					, const UE::Mass::ERelationRole Role) -> TArray<FMassEntityHandle>
				{
					const TArray<FMassRelationRoleInstanceHandle>& RoleRelationInstanceHandles = InstanceData[Role];

					TArray<FMassEntityHandle> RoleRelationEntityHandles = RelationManager.GetRelationEntities(RoleRelationInstanceHandles);
					LocalExecutionContext.Defer().DestroyEntities(MoveTemp(RoleRelationEntityHandles));

					return RelationManager.GetRoleEntities(RoleRelationInstanceHandles);
				};

				UE::Mass::FRelationManager& RelationManager = ExecutionContext.GetEntityManagerChecked().GetRelationManager();
				UE::Mass::FRelationData& RelationData = RelationManager.GetRelationDataChecked(InRegisteredTypeHandle);

				for (FMassExecutionContext::FEntityIterator EntityIt = ExecutionContext.CreateEntityIterator(); EntityIt; ++EntityIt)
				{
					const FMassEntityHandle RoleEntity = ExecutionContext.GetEntity(EntityIt);
					if (UE::Mass::FRelationData::FRelationInstanceData* InstanceData = RelationData.RoleMap.Find(RoleEntity))
					{
						// @todo optimize array allocations here - we could create the arrays outside and re-use
						TArray<FMassEntityHandle> SubjectEntities = DestroyRelationEntitiesAndGetRoleEntities(ExecutionContext, RelationManager, *InstanceData, UE::Mass::ERelationRole::Subject);
						TArray<FMassEntityHandle> ObjectEntities = DestroyRelationEntitiesAndGetRoleEntities(ExecutionContext, RelationManager, *InstanceData, UE::Mass::ERelationRole::Object);
						
						RelationManager.CreateRelationInstances(InRegisteredTypeHandle, MakeArrayView(SubjectEntities), MakeArrayView(ObjectEntities));
					}
				}
			};
			bExecutionFunctionAssigned = true;
			break;
		case UE::Mass::ERemovalPolicy::Custom: // fall through on purpose
		default: ;
	}

	return bExecutionFunctionAssigned;
}

void UMassRelationRoleDestruction::AddObserverInstances(FMassObserverManager& ObserverManager, UE::Mass::FTypeHandle InRegisteredTypeHandle, const UE::Mass::FRelationTypeTraits& Traits)
{
	static_assert(static_cast<int32>(UE::Mass::ERelationRole::MAX) == 2, "This implementation is tailored to there being only two roles to any relation");

	// if both roles have the same policy and the same element, we can get away with creating only one observer
	// as long as we configure it appropriately
	if (Traits.RoleTraits[1].Element == Traits.RoleTraits[0].Element && Traits.RoleTraits[1].DestructionPolicy == Traits.RoleTraits[0].DestructionPolicy)
	{
		if (Traits.RoleTraits[0].DestructionPolicy != UE::Mass::ERemovalPolicy::Custom)
		{
			UMassRelationRoleDestruction* ObserverProcessor = NewObject<UMassRelationRoleDestruction>();
			ObserverProcessor->RelationRole = UE::Mass::ERelationRole::MAX;
			if (ObserverProcessor->ConfigureRelationObserver(InRegisteredTypeHandle, Traits))
			{
				ObserverManager.AddObserverInstance(ObserverProcessor);
			}
		}
	}
	else
	{
		for (int32 RoleIndex = 0; RoleIndex < static_cast<int32>(UE::Mass::ERelationRole::MAX); ++RoleIndex)
		{
			if (Traits.RoleTraits[RoleIndex].DestructionPolicy != UE::Mass::ERemovalPolicy::Custom)
			{
				UMassRelationRoleDestruction* ObserverProcessor = NewObject<UMassRelationRoleDestruction>();
				ObserverProcessor->RelationRole = static_cast<UE::Mass::ERelationRole>(RoleIndex);
				if (ObserverProcessor->ConfigureRelationObserver(InRegisteredTypeHandle, Traits))
				{
					ObserverManager.AddObserverInstance(ObserverProcessor);
				}
			}
		}
	}
}
