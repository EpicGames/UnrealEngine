// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRequirementAccessDetector.h"
#if WITH_MASSENTITY_DEBUG
#include "MassEntityQuery.h"
#include "HAL/IConsoleManager.h"
#include "MassEntityManager.h"

namespace UE::Mass::Private
{
	bool bTrackRequirementsAccess = false;
	
	FAutoConsoleVariableRef CVarTrackRequirementsAccess(TEXT("mass.debug.TrackRequirementsAccess"), bTrackRequirementsAccess
		, TEXT("Enables Mass processing debugging mode where we monitor thread-safety of query requirements access."));
} // namespace UE::Mass::Private

void FMassRequirementAccessDetector::Initialize()
{
	check(IsInGameThread());
	AddDetectors(FMassExternalSubsystemBitSet::FStructTrackerWrapper::StructTracker);
}

void FMassRequirementAccessDetector::AddDetectors(const FStructTracker& StructTracker)
{
	TConstArrayView<TWeakObjectPtr<const UStruct>> Types = StructTracker.DebugGetAllStructTypes<UStruct>();
	for (TWeakObjectPtr<const UStruct> Type : Types)
	{
		if (Type.Get())
		{
			Detectors.Add(Type.Get(), MakeShareable(new FRWAccessDetector()));
		}
	}
}

void FMassRequirementAccessDetector::RequireAccess(const FMassEntityQuery& Query)
{
	if (UE::Mass::Private::bTrackRequirementsAccess)
	{
		Operation(Query.RequiredConstSubsystems, &FRWAccessDetector::AcquireReadAccess);
		Operation(Query.RequiredMutableSubsystems, &FRWAccessDetector::AcquireWriteAccess);
		
		Acquire(Query.FragmentRequirements);
		Acquire(Query.ChunkFragmentRequirements);
		Acquire(Query.ConstSharedFragmentRequirements);
		Acquire(Query.SharedFragmentRequirements);
	}
}

void FMassRequirementAccessDetector::ReleaseAccess(const FMassEntityQuery& Query)
{
	if (UE::Mass::Private::bTrackRequirementsAccess)
	{
		Operation(Query.RequiredConstSubsystems, &FRWAccessDetector::ReleaseReadAccess);
		Operation(Query.RequiredMutableSubsystems, &FRWAccessDetector::ReleaseWriteAccess);

		Release(Query.FragmentRequirements);
		Release(Query.ChunkFragmentRequirements);
		Release(Query.ConstSharedFragmentRequirements);
		Release(Query.SharedFragmentRequirements);
	}
}

namespace UE::Mass::Debug
{
	//-----------------------------------------------------------------------------
	// FScopedRequirementAccessDetector
	//-----------------------------------------------------------------------------
	FScopedRequirementAccessDetector::FScopedRequirementAccessDetector(const FMassEntityQuery& InQuery)
		: EntityManager(InQuery.GetEntityManager())
		, Query(InQuery)
	{
		EntityManager->GetRequirementAccessDetector().RequireAccess(InQuery);
	}

	FScopedRequirementAccessDetector::~FScopedRequirementAccessDetector()
	{
		EntityManager->GetRequirementAccessDetector().ReleaseAccess(Query);
	}
} // namespace UE::Mass::Debug

#endif // WITH_MASSENTITY_DEBUG
