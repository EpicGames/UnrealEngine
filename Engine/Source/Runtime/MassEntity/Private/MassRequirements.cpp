// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRequirements.h"
#include "MassArchetypeData.h"
#include "MassProcessorDependencySolver.h"
#include "MassTypeManager.h"
#include "MassEntityManager.h"
#include "Misc/OutputDeviceNull.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassRequirements)

#if WITH_MASSENTITY_DEBUG
#include "MassRequirementAccessDetector.h"
#endif // WITH_MASSENTITY_DEBUG

namespace UE::Mass::ErrorReporting
{
thread_local FOutputDevice* GOutputDevice = nullptr;
FOutputDeviceNull GNullOutput;

FOutputDevice& GetOutputDevice()
{
	return GOutputDevice ? *GOutputDevice : GNullOutput;
}

FScopedOutputDevice::FScopedOutputDevice(FOutputDevice& OutputDevice)
{
	ActiveOutputDevice = GOutputDevice;
	GOutputDevice = &OutputDevice;
}

FScopedOutputDevice::~FScopedOutputDevice()
{
	GOutputDevice = ActiveOutputDevice;
}

} // namespace UE::Mass::ErrorReporting

namespace UE::Mass::Private
{
	template<typename TContainer>
	void ExportRequirements(TConstArrayView<FMassFragmentRequirementDescription> Requirements, TMassExecutionAccess<TContainer>& Out)
	{
		for (const FMassFragmentRequirementDescription& Requirement : Requirements)
		{
			if (Requirement.Presence != EMassFragmentPresence::None)
			{
				check(Requirement.StructType);
				if (Requirement.AccessMode == EMassFragmentAccess::ReadOnly)
				{
					Out.Read.Add(Requirement.StructType);
				}
				else if (Requirement.AccessMode == EMassFragmentAccess::ReadWrite)
				{
					Out.Write.Add(Requirement.StructType);
				}
			}
		}
	}

	template<>
	void ExportRequirements<FMassConstSharedFragmentBitSet>(TConstArrayView<FMassFragmentRequirementDescription> Requirements
		, TMassExecutionAccess<FMassConstSharedFragmentBitSet>& Out)
	{
		for (const FMassFragmentRequirementDescription& Requirement : Requirements)
		{
			if (Requirement.Presence != EMassFragmentPresence::None)
			{
				check(Requirement.StructType);
				if (ensureMsgf(Requirement.AccessMode == EMassFragmentAccess::ReadOnly, TEXT("ReadOnly is the only supported AccessMode for ConstSharedFragments")))
				{
					Out.Read.Add(Requirement.StructType);
				}
			}
		}
	}
} // namespace UE::Mass::Private

//-----------------------------------------------------------------------------
// FMassSubsystemRequirements
//-----------------------------------------------------------------------------
void FMassSubsystemRequirements::ExportRequirements(FMassExecutionRequirements& OutRequirements) const
{
	OutRequirements.RequiredSubsystems.Read += RequiredConstSubsystems;
	OutRequirements.RequiredSubsystems.Write += RequiredMutableSubsystems;
}

void FMassSubsystemRequirements::Reset()
{
	RequiredConstSubsystems.Reset();
	RequiredMutableSubsystems.Reset();
	bRequiresGameThreadExecution = false;
}

bool FMassSubsystemRequirements::IsGameThreadOnlySubsystem(const TSubclassOf<USubsystem> SubsystemClass, const TSharedRef<FMassEntityManager>& EntityManager)
{
	const UE::Mass::FTypeInfo* TypeInfo = EntityManager->GetTypeManager().GetTypeInfo(SubsystemClass);
	if (ensureMsgf(TypeInfo, TEXT("Failed to find type information for %s"), *GetNameSafe(SubsystemClass)))
	{
		const UE::Mass::FSubsystemTypeTraits* SystemTraits = TypeInfo->GetAsSystemTraits();
		if (ensureMsgf(SystemTraits, TEXT("Type information for %s doesn't represent subsystem traits"), *GetNameSafe(SubsystemClass)))
		{
			return SystemTraits->bGameThreadOnly;
		}
	}
	// using `true` as default as the safer one of the options
	// since it's safer to run everything on GT rather than on an arbitrary thread
	return true;
}

//-----------------------------------------------------------------------------
// FMassFragmentRequirements
//-----------------------------------------------------------------------------
FMassFragmentRequirements::FMassFragmentRequirements(const TSharedPtr<FMassEntityManager>& EntityManager)
{
	if (ensure(EntityManager))
	{
		Initialize(EntityManager.ToSharedRef());
	}
}

FMassFragmentRequirements::FMassFragmentRequirements(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Initialize(EntityManager);
}

void FMassFragmentRequirements::Initialize(const TSharedRef<FMassEntityManager>& EntityManager)
{
	UE_CLOGF(CachedEntityManager && (CachedEntityManager != EntityManager), LogMass, Warning
		, "Trying to initialize FMassFragmentRequirements with another entity manager");
	if (bInitialized)
	{
		return;
	}

	CachedEntityManager = EntityManager;
	bInitialized = true;
}

FMassFragmentRequirements& FMassFragmentRequirements::ClearTagRequirements(const FMassTagBitSet& TagsToRemoveBitSet)
{
	RequiredAllTags.Remove(TagsToRemoveBitSet);
	RequiredAnyTags.Remove(TagsToRemoveBitSet);
	RequiredNoneTags.Remove(TagsToRemoveBitSet);
	RequiredOptionalTags.Remove(TagsToRemoveBitSet);

	return *this;
}

uint32 GetTypeHash(const FMassFragmentRequirements& Instance)
{
	// @todo consider calculating hash only for non-empty elements, or any other optimization
	uint32 Hash = FCrc::MemCrc32(Instance.FragmentRequirements.GetData(), Instance.FragmentRequirements.Num() * sizeof(FMassFragmentRequirementDescription));
	Hash = HashCombine(Hash, FCrc::MemCrc32(Instance.ChunkFragmentRequirements.GetData(), Instance.ChunkFragmentRequirements.Num() * sizeof(FMassFragmentRequirementDescription)));
	Hash = HashCombine(Hash, FCrc::MemCrc32(Instance.ConstSharedFragmentRequirements.GetData(), Instance.ConstSharedFragmentRequirements.Num() * sizeof(FMassFragmentRequirementDescription)));
	Hash = HashCombine(Hash, FCrc::MemCrc32(Instance.SharedFragmentRequirements.GetData(), Instance.SharedFragmentRequirements.Num() * sizeof(FMassFragmentRequirementDescription)));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredAllTags));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredAnyTags));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredNoneTags));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredOptionalTags));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredAllFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredAnyFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredOptionalFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredNoneFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredAllSparseElements));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredAnySparseElements));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredOptionalSparseElements));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredNoneSparseElements));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredAllChunkFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredOptionalChunkFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredNoneChunkFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredAllSharedFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredOptionalSharedFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredNoneSharedFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredAllConstSharedFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredOptionalConstSharedFragments));
	Hash = HashCombine(Hash, GetTypeHash(Instance.RequiredNoneConstSharedFragments));
	return HashCombine(Hash, PointerHash(Instance.CachedEntityManager.Get()));
}

void FMassFragmentRequirements::SortRequirements()
{
	// we're sorting the Requirements the same way ArchetypeData's FragmentConfig is sorted (see FMassArchetypeData::Initialize)
	// so that when we access ArchetypeData.FragmentConfigs in FMassArchetypeData::BindRequirementsWithMapping
	// (via GetFragmentData call) the access is sequential (i.e. not random) and there's a higher chance the memory
	// FragmentConfigs we want to access have already been fetched and are available in processor cache.
	FragmentRequirements.Sort(FScriptStructSortOperator());
	ChunkFragmentRequirements.Sort(FScriptStructSortOperator());
	ConstSharedFragmentRequirements.Sort(FScriptStructSortOperator());
	SharedFragmentRequirements.Sort(FScriptStructSortOperator());
}

bool FMassFragmentRequirements::IsGameThreadOnlySharedFragment(TNotNull<const UScriptStruct*> SharedFragmentType) const
{
	checkf(CachedEntityManager, TEXT("Not having a cached EntityManager at this point is not expected."));

	const UE::Mass::FTypeInfo* TypeInfo = CachedEntityManager->GetTypeManager().GetTypeInfo(SharedFragmentType);
	if (ensureMsgf(TypeInfo, TEXT("Failed to find type information for %s"), *SharedFragmentType->GetName()))
	{
		const UE::Mass::FSharedFragmentTypeTraits* SharedFragmentTraits = TypeInfo->GetAsSharedFragmentTraits();
		if (ensureMsgf(SharedFragmentTraits, TEXT("Type information for %s doesn't represent shared fragment traits"), *SharedFragmentType->GetName()))
		{
			return SharedFragmentTraits->bGameThreadOnly;
		}
	}
	// using `true` as default as the safer one of the options
	// since it's safer to run everything on GT rather than on an arbitrary thread
	return true;
}

FORCEINLINE void FMassFragmentRequirements::CacheProperties() const
{
	if (bPropertiesCached == false)
	{
		bHasPositiveRequirements = !(RequiredAllTags.IsEmpty()
			&& RequiredAnyTags.IsEmpty()
			&& RequiredAllFragments.IsEmpty()
			&& RequiredAnyFragments.IsEmpty()
			&& RequiredAllChunkFragments.IsEmpty()
			&& RequiredAllSharedFragments.IsEmpty()
			&& RequiredAllConstSharedFragments.IsEmpty());

		bHasNegativeRequirements = !(RequiredNoneTags.IsEmpty()
			&& RequiredNoneFragments.IsEmpty()
			&& RequiredNoneChunkFragments.IsEmpty()
			&& RequiredNoneSharedFragments.IsEmpty()
			&& RequiredNoneConstSharedFragments.IsEmpty());

		// note that we're not including RequiredOptionalSparseElements on purpose here
		// as that's just a way to signal that a given sparse element might get accessed
		// and does not affect archetype filtering
		bHasOptionalRequirements = !(RequiredOptionalFragments.IsEmpty()
			&& RequiredOptionalTags.IsEmpty()
			&& RequiredOptionalChunkFragments.IsEmpty()
			&& RequiredOptionalSharedFragments.IsEmpty()
			&& RequiredOptionalConstSharedFragments.IsEmpty());

		bHasSparseRequirements = !(RequiredAllSparseElements.IsEmpty() 
			&& RequiredAnySparseElements.IsEmpty()
			&& RequiredNoneSparseElements.IsEmpty());

		bPropertiesCached = true;
	}
}

bool FMassFragmentRequirements::CheckValidity() const
{
	CacheProperties();
	// @todo we need to add more sophisticated testing somewhere to detect contradicting requirements - like having and not having a given tag.
	// note: bHasSparseRequirements not being tested on purpose - sparse-only requirements are not valid
	return bHasPositiveRequirements || bHasNegativeRequirements || bHasOptionalRequirements;
}

bool FMassFragmentRequirements::IsEmpty() const
{
	CacheProperties();
	return !bHasPositiveRequirements && !bHasNegativeRequirements && !bHasOptionalRequirements && !bHasSparseRequirements;
}

bool FMassFragmentRequirements::DoesMatchAnyOptionals(const FMassElementBitSet& ArchetypeCompositionBitSet) const
{
	return bHasOptionalRequirements
		&& ArchetypeCompositionBitSet.HasAny(RequiredOptionalFragments | RequiredOptionalTags | RequiredOptionalChunkFragments | RequiredOptionalSharedFragments | RequiredOptionalConstSharedFragments);
}

bool FMassFragmentRequirements::DoesArchetypeMatchRequirements(const FMassArchetypeHandle& ArchetypeHandle) const
{
	check(ArchetypeHandle.IsValid());
	const FMassArchetypeData* Archetype = FMassArchetypeHelper::ArchetypeDataFromHandle(ArchetypeHandle);
	CA_ASSUME(Archetype);

	return DoesArchetypeMatchRequirements(Archetype->GetCompositionBitSet());
}
	
bool FMassFragmentRequirements::DoesArchetypeMatchRequirements(const FMassElementBitSet& ArchetypeCompositionBitSet) const
{
	CacheProperties();

	const bool bPassNegativeFilter = bHasNegativeRequirements == false
		|| ArchetypeCompositionBitSet.HasNone(RequiredNoneFragments | RequiredNoneTags | RequiredNoneChunkFragments | RequiredNoneSharedFragments | RequiredNoneConstSharedFragments);

	if (bPassNegativeFilter)
	{
		if (bHasPositiveRequirements)
		{
			return ArchetypeCompositionBitSet.HasAll(RequiredAllFragments)
				&& ArchetypeCompositionBitSet.HasAll(RequiredAllTags)
				&& ArchetypeCompositionBitSet.HasAll(RequiredAllChunkFragments | RequiredAllSharedFragments | RequiredAllConstSharedFragments)
				&& ((RequiredAnyFragments.IsEmpty() && RequiredAnyTags.IsEmpty()) 
					|| ArchetypeCompositionBitSet.HasAny(RequiredAnyFragments | RequiredAnyTags));
		}
		else if (bHasOptionalRequirements)
		{
			return DoesMatchAnyOptionals(ArchetypeCompositionBitSet);
		}
		// else - it's fine, we passed all the filters that have been set up
		return true;
	}
	return false;
}

void FMassFragmentRequirements::ExportRequirements(FMassExecutionRequirements& OutRequirements) const
{
	using UE::Mass::Private::ExportRequirements;
	ExportRequirements<FMassFragmentBitSet>(FragmentRequirements, OutRequirements.Fragments);
	ExportRequirements<FMassChunkFragmentBitSet>(ChunkFragmentRequirements, OutRequirements.ChunkFragments);
	ExportRequirements<FMassSharedFragmentBitSet>(SharedFragmentRequirements, OutRequirements.SharedFragments);
	ExportRequirements<FMassConstSharedFragmentBitSet>(ConstSharedFragmentRequirements, OutRequirements.ConstSharedFragments);

	//indirect fragment access doesn't support EMassFragmentPresence::None so we can just add them directly
	OutRequirements.IndirectFragments.Read += IndirectReadOnlyFragments;
	OutRequirements.IndirectFragments.Write += IndirectReadWriteFragments;

	OutRequirements.RequiredAllTags += RequiredAllTags;
	OutRequirements.RequiredAnyTags += RequiredAnyTags;
	OutRequirements.RequiredNoneTags += RequiredNoneTags;
	// not exporting optional tags by design

	// we treat all sparse elements access as ReadWrite, at the moment. 
	OutRequirements.SparseElements.Write += RequiredAllSparseElements;
	OutRequirements.SparseElements.Write += RequiredAnySparseElements;
	OutRequirements.SparseElements.Write += RequiredOptionalSparseElements;
}

void FMassFragmentRequirements::Reset()
{
	FragmentRequirements.Reset();
	ChunkFragmentRequirements.Reset();
	ConstSharedFragmentRequirements.Reset();
	SharedFragmentRequirements.Reset();
	RequiredAllTags.Reset();
	RequiredAnyTags.Reset();
	RequiredNoneTags.Reset();
	RequiredOptionalTags.Reset();
	RequiredAllFragments.Reset();
	RequiredAnyFragments.Reset();
	RequiredOptionalFragments.Reset();
	RequiredNoneFragments.Reset();
	RequiredAllSparseElements.Reset();
	RequiredAnySparseElements.Reset();
	RequiredOptionalSparseElements.Reset();
	RequiredNoneSparseElements.Reset();
	RequiredAllChunkFragments.Reset();
	RequiredOptionalChunkFragments.Reset();
	RequiredNoneChunkFragments.Reset();
	RequiredAllSharedFragments.Reset();
	RequiredOptionalSharedFragments.Reset();
	RequiredNoneSharedFragments.Reset();
	RequiredAllConstSharedFragments.Reset();
	RequiredOptionalConstSharedFragments.Reset();
	RequiredNoneConstSharedFragments.Reset();

	IncrementalChangesCount = 0;

	bPropertiesCached = false;

	// note that we're not resetting bInitialized nor CachedEntityManager, on purpose
	// the point of this function is to just reset the contents while still being able
	// to add elements to it. This "requirements" instance is now "empty" but still valid
}

//-----------------------------------------------------------------------------
// FMassEntityCreationRequirements
//-----------------------------------------------------------------------------
void FMassEntityCreationRequirements::ExportRequirements(FMassExecutionRequirements& Out) const
{
	FMassElementBitSet ElementsBitSet;
	for (const FMassElementBitSet& Composition : Compositions)
	{
		ElementsBitSet += Composition;
	}

	Out.Fragments.Write += ElementsBitSet.Get<FMassFragmentBitSet>();
	Out.ChunkFragments.Write += ElementsBitSet.Get<FMassChunkFragmentBitSet>();
	Out.SharedFragments.Write += ElementsBitSet.Get<FMassSharedFragmentBitSet>();

	// Note: ConstSharedFragments are "read-only" by nature; declaring write on them
	// would be misleading. We still need *archetype-level* exclusivity, which the solver
	// enforces via ValidArchetypes overlap. So no write on const shared here.
	Out.RequiredAllTags += ElementsBitSet.Get<FMassTagBitSet>();

	Out.ResourcesUsedCount = INDEX_NONE; // force recomputation
}

void FMassEntityCreationRequirements::ResolveArchetypes(const FMassEntityManager& EntityManager)
{
	ArchetypeHandles.Reset(Compositions.Num());

	for (const FMassElementBitSet& Composition : Compositions)
	{
		ArchetypeHandles.Add(const_cast<FMassEntityManager&>(EntityManager).CreateArchetype(FMassArchetypeCompositionDescriptor(Composition)));
	}
}

//-----------------------------------------------------------------------------
// DEPRECATED
//-----------------------------------------------------------------------------
FMassFragmentRequirements::FMassFragmentRequirements(std::initializer_list<UScriptStruct*> InitList)
{
	for (const UScriptStruct* FragmentType : InitList)
	{
		AddRequirement(FragmentType, EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);
	}
}

FMassFragmentRequirements::FMassFragmentRequirements(TConstArrayView<const UScriptStruct*> InitList)
{
	for (const UScriptStruct* FragmentType : InitList)
	{
		AddRequirement(FragmentType, EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);
	}
}
