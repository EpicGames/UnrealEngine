// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSettings)


//----------------------------------------------------------------------//
//  UMassModuleSettings
//----------------------------------------------------------------------//
void UMassModuleSettings::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) && !GetClass()->HasAnyClassFlags(CLASS_Abstract))
	{
		// register with UMassGameplaySettings
		GetMutableDefault<UMassSettings>()->RegisterModuleSettings(*this);
	}
}

//----------------------------------------------------------------------//
//  UMassSettings
//----------------------------------------------------------------------//
void UMassSettings::RegisterModuleSettings(UMassModuleSettings& SettingsCDO)
{
	ensureMsgf(SettingsCDO.HasAnyFlags(RF_ClassDefaultObject), TEXT("Registered ModuleSettings need to be its class's CDO"));

	// we should consider a replacement in case we're hot-reloading
	FName EntryName = SettingsCDO.GetClass()->GetFName();

#if WITH_EDITOR
	static const FName DisplayNameMeta("DisplayName"); 
	// try reading better name from meta data, available only in editor. Besides, we don't really care about this out 
	// side of editor. We could even skip populating ModuleSettings but we'll leave it as is for now.
	const FString& DisplayNameValue = SettingsCDO.GetClass()->GetMetaData(DisplayNameMeta);
	if (DisplayNameValue.Len())
	{
		EntryName = *DisplayNameValue;
	}
#endif // WITH_EDITOR

	TObjectPtr<UMassModuleSettings>& FoundModuleEntry = ModuleSettings.FindOrAdd(EntryName, &SettingsCDO);
	FoundModuleEntry = &SettingsCDO;
}

