// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSparseElementsStorage.h"
#include "MassEntityManager.h"

namespace UE::Mass
{
inline void SanitizeEntityHandleArray(TArray<FMassEntityHandle>& InOutEntityHandles)
{
	// sorting because data is stored in ascending Index order - EntityHandle.Index is used as the index to TypePool's elements
	InOutEntityHandles.Sort();
	if (InOutEntityHandles.IsEmpty() == false && InOutEntityHandles[0].IsSet() == false)
	{
		InOutEntityHandles.RemoveAll([](const FMassEntityHandle& Handle)
			{
				return Handle.IsSet() == false;
			});
	}
}

//-----------------------------------------------------------------------------
// FSparseElementsStorage::FTypeConfig
//-----------------------------------------------------------------------------
FSparseElementsStorage::FTypeConfig::FTypeConfig()
	: ElementType(FMassElement::StaticStruct())
{
	// this is a dummy constructor, used by FTypePool::DummyPool
}

//-----------------------------------------------------------------------------
// FSparseElementsStorage::FTypePool
//-----------------------------------------------------------------------------
FSparseElementsStorage::FTypePool::~FTypePool()
{
	for (FPackedStructDataChunk& PackedData : Chunks)
	{
		if (PackedData.RawMemory && PackedData.NumElements > 0 && PackedData.OccupationMask.IsEmpty() == false)
		{
			for (TBitArray<>::FConstIterator It(PackedData.OccupationMask); It; ++It)
			{
				if (It.GetValue())
				{
					Config.GetType()->DestroyStruct(PackedData.GetElement(It.GetIndex(), Config.GetElementSize()));
				}
			}
		}
	}
}

TNotNull<uint8*> FSparseElementsStorage::FTypePool::Add(const uint32 EntityIndex)
{
	const uint32 ChunkIndex = GetChunkIndex(EntityIndex);
	const uint32 ElementIndex = GetElementIndex(EntityIndex);

	FPackedStructDataChunk& PackedData = GetOrCreatePackedData(ChunkIndex);
	if (PackedData.HasElement(ElementIndex) == false)
	{
		uint8* Memory = PackedData.Add(ElementIndex, Config.GetElementSize());
		Config.GetType()->InitializeDefaultValue(Memory);
		++NumElements; 
		return Memory;
	}
	
	return PackedData.GetElementUnsafe(ElementIndex, Config.GetElementSize());
}

TNotNull<uint8*> FSparseElementsStorage::FTypePool::Add(const uint32 EntityIndex, TNotNull<const uint8*> SourceData)
{
	TNotNull<uint8*> ElementPtr = Add(EntityIndex);
	Config.GetType()->CopyScriptStruct(ElementPtr, SourceData);
	return ElementPtr;
}

TNotNull<uint8*> FSparseElementsStorage::FTypePool::AddMove(const uint32 EntityIndex, TNotNull<uint8*> SourceData)
{
	TNotNull<uint8*> ElementPtr = Add(EntityIndex);
	// MoveAssignScriptStruct uses C++ move assignment for types that opt in via WithMoveAssign,
	// falling back to CopyScriptStruct for types without the trait.
	Config.GetType()->MoveAssignScriptStruct(ElementPtr, SourceData);
	return ElementPtr;
}

FStructView FSparseElementsStorage::FTypePool::Add_GetView(const uint32 EntityIndex)
{	
	return { Config.GetType(), Add(EntityIndex) };
}

bool FSparseElementsStorage::FTypePool::Remove(const uint32 EntityIndex)
{
	const uint32 ChunkIndex = GetChunkIndex(EntityIndex);
	const uint32 ElementIndex = GetElementIndex(EntityIndex);

	if (FPackedStructDataChunk* PackedData = GetPackedData(ChunkIndex))
	{
		if (uint8* Memory = PackedData->Remove(ElementIndex, Config.GetElementSize()))
		{
			Config.GetType()->DestroyStruct(Memory);
			FMemory::Memzero(Memory, Config.GetElementSize());
			--NumElements; 
			return true;
		}
	}
	return false;
}

FConstStructView FSparseElementsStorage::FTypePool::Get(const uint32 EntityIndex) const
{
	return FConstStructView(const_cast<FTypePool*>(this)->GetMutable(EntityIndex));
}

FStructView FSparseElementsStorage::FTypePool::GetMutable(const uint32 EntityIndex)
{
	const uint32 ChunkIndex = GetChunkIndex(EntityIndex);
	const uint32 ElementIndex = GetElementIndex(EntityIndex);

	if (FPackedStructDataChunk* PackedData = GetPackedData(ChunkIndex))
	{
		uint8* Memory = PackedData->GetElement(ElementIndex, Config.GetElementSize());
		return { Config.GetType(), Memory };
	}
	return {};
}

FStructView FSparseElementsStorage::FTypePool::GetOrCreateMutable(const uint32 EntityIndex)
{
	const uint32 ChunkIndex = GetChunkIndex(EntityIndex);
	const uint32 ElementIndex = GetElementIndex(EntityIndex);

	FPackedStructDataChunk& PackedData = GetOrCreatePackedData(ChunkIndex);
	// note that it's fine for Memory to be nullptr. The return value should always be checked for validity.
	uint8* Memory = PackedData.GetElement(ElementIndex, Config.GetElementSize());
	return {Config.GetType(), Memory};
}

FSparseElementsStorage::FPackedStructDataChunk* FSparseElementsStorage::FTypePool::GetPackedData(const uint32 ChunkIndex)
{
	return Chunks.IsValidIndex(ChunkIndex) ? &Chunks[ChunkIndex] : nullptr;
}

const FSparseElementsStorage::FPackedStructDataChunk* FSparseElementsStorage::FTypePool::GetPackedData(const uint32 ChunkIndex) const
{
	return Chunks.IsValidIndex(ChunkIndex) ? &Chunks[ChunkIndex] : nullptr;
}

FSparseElementsStorage::FPackedStructDataChunk& FSparseElementsStorage::FTypePool::GetOrCreatePackedData(const uint32 ChunkIndex)
{
	if (Chunks.IsValidIndex(ChunkIndex) == false)
	{
		Chunks.AddDefaulted(ChunkIndex - Chunks.Num() + 1);
	}

	FPackedStructDataChunk& PackedData = Chunks[ChunkIndex];

	if (PackedData.IsInitialized() == false)
	{
		PackedData.Init(Config.ElementsPerChunk, Config.GetElementSize());
	}

	return PackedData;
}

FSparseElementsStorage::FTypePool& FSparseElementsStorage::FTypePool::GetDummyPool()
{
	static FTypePool DummyPool;
	return DummyPool;
}

//-----------------------------------------------------------------------------
// FSparseElementsStorage::FSparseElementIterator 
//-----------------------------------------------------------------------------
FMassEntityHandle FSparseElementIterator::GetEntityHandle(const FMassEntityManager& EntityManager) const
{
	return EntityManager.CreateEntityIndexHandle(GetEntityIndex());
}

//-----------------------------------------------------------------------------
// FSparseElementsStorage
//-----------------------------------------------------------------------------
void FSparseElementsStorage::Deinitialize()
{
	TypePools.Reset();
}

FSparseElementsStorage::FTypePool& FSparseElementsStorage::GetOrCreateTypePool(TNotNull<const UScriptStruct*> ElementType, const int32 InTypeIndex)
{
	const int32 TypeIndex = InTypeIndex == INDEX_NONE ? FMassElementBitSet::GetTypeIndex(ElementType) : InTypeIndex;

	if (TypePools.IsValidIndex(TypeIndex) == false)
	{
		TypePools.EmplaceAt(TypeIndex, FTypeConfig(ElementType));
	}

	return TypePools[TypeIndex];
}

const FSparseElementsStorage::FTypePool* FSparseElementsStorage::GetTypePool(TNotNull<const UScriptStruct*> ElementType) const
{
	const int32 TypeIndex = FMassElementBitSet::GetTypeIndex(ElementType);
	return TypePools.IsValidIndex(TypeIndex) ? &TypePools[TypeIndex] : nullptr;
}

FSparseElementsStorage::FTypePool* FSparseElementsStorage::GetTypePool(TNotNull<const UScriptStruct*> ElementType)
{
	const int32 TypeIndex = FMassElementBitSet::GetTypeIndex(ElementType);
	return TypePools.IsValidIndex(TypeIndex) ? &TypePools[TypeIndex] : nullptr;
}

FStructView FSparseElementsStorage::AddElementToEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType)
{
	checkSlow(!UE::Mass::IsA<FMassTag>(ElementType));
	const int32 TypeIndex = FMassElementBitSet::GetTypeIndex(ElementType);
	FTypePool& TypePool = GetOrCreateTypePool(ElementType, TypeIndex);
	return TypePool.Add_GetView(EntityHandle.Index);
}

bool FSparseElementsStorage::RemoveElementFromEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType)
{
	FTypePool* TypePool = GetTypePool(ElementType);
	return TypePool && TypePool->Remove(EntityHandle.Index);
}

template<typename TContainer>
void FSparseElementsStorage::BatchAddElementToEntities(TContainer InEntityHandles, TNotNull<const UScriptStruct*> ElementType, TArray<FStructView>* OutAddedElements)
{
	const int32 TypeIndex = FMassElementBitSet::GetTypeIndex(ElementType);

	FTypePool& TypePool = GetOrCreateTypePool(ElementType, TypeIndex);
	// @todo TypePool.Add can be optimized by checking for index continuity and whether the following index is in the same chunk.
	if (OutAddedElements)
	{
		for (const FMassEntityHandle& EntityHandle : InEntityHandles)
		{
			OutAddedElements->Add(TypePool.Add_GetView(EntityHandle.Index));
		}
	}
	else
	{
		for (const FMassEntityHandle& EntityHandle : InEntityHandles)
		{
			TypePool.Add(EntityHandle.Index);
		}
	}
}

void FSparseElementsStorage::BatchAddElementToEntities(TArray<FMassEntityHandle>& InOutEntityHandles, TNotNull<const UScriptStruct*> ElementType, TArray<FStructView>* OutAddedElements)
{
	if (!ensureMsgf(UE::Mass::IsA<FMassElement>(ElementType) && !UE::Mass::IsA<FMassTag>(ElementType)
		, TEXT("%hs: %s is not an element type"), __FUNCTION__, *ElementType->GetName()))
	{
		return;
	}
	if (InOutEntityHandles.IsEmpty())
	{
		return;
	}

	SanitizeEntityHandleArray(InOutEntityHandles);
	BatchAddElementToEntities<TArray<FMassEntityHandle>&>(InOutEntityHandles, ElementType, OutAddedElements);
}

void FSparseElementsStorage::BatchAddElementToEntities(TArrayView<const FMassEntityHandle> InEntityHandles, TNotNull<const UScriptStruct*> ElementType, TArray<FStructView>* OutAddedElements)
{
	if (!ensureMsgf(UE::Mass::IsA<FMassElement>(ElementType) && !UE::Mass::IsA<FMassTag>(ElementType)
		, TEXT("%hs: %s is not an element type"), __FUNCTION__, *ElementType->GetName()))
	{
		return;
	}
	if (InEntityHandles.IsEmpty())
	{
		return;
	}

	BatchAddElementToEntities<TArrayView<const FMassEntityHandle>>(InEntityHandles, ElementType, OutAddedElements);
}

void FSparseElementsStorage::BatchAddElementInstancesToEntities(TArrayView<const FMassEntityHandle> InEntityHandles, const FStructArrayView& FragmentPayload)
{
	checkf(InEntityHandles.Num() == FragmentPayload.Num(), TEXT("%hs: Payload size is expected to match number of entity handles"), __FUNCTION__);
	const UScriptStruct* ElementType = FragmentPayload.GetScriptStruct();
	CA_ASSUME(ElementType);
	const int32 TypeIndex = FMassElementBitSet::GetTypeIndex(ElementType);

	FTypePool& TypePool = GetOrCreateTypePool(ElementType, TypeIndex);
	for (int32 Index = 0; Index < InEntityHandles.Num(); ++Index)
	{
		TypePool.Add(InEntityHandles[Index].Index, static_cast<uint8*>(FragmentPayload.GetDataAt(Index)));
	}
}

void FSparseElementsStorage::BatchAddElementInstancesToEntities(TArrayView<const FMassEntityHandle> InEntityHandles, const FConstStructView FragmentInstance)
{
	check(FragmentInstance.IsValid());
	const UScriptStruct* ElementType = FragmentInstance.GetScriptStruct();
	CA_ASSUME(ElementType);
	const int32 TypeIndex = FMassElementBitSet::GetTypeIndex(ElementType);

	FTypePool& TypePool = GetOrCreateTypePool(ElementType, TypeIndex);
	for (int32 Index = 0; Index < InEntityHandles.Num(); ++Index)
	{
		TypePool.Add(InEntityHandles[Index].Index, FragmentInstance.GetMemory());
	}
}	

int32 FSparseElementsStorage::BatchRemoveElementFromEntities(TArray<FMassEntityHandle>& InOutEntityHandles, TNotNull<const UScriptStruct*> ElementType)
{
	SanitizeEntityHandleArray(InOutEntityHandles);
	return BatchRemoveElementFromEntities(MakeArrayView(InOutEntityHandles), ElementType);
}

int32 FSparseElementsStorage::BatchRemoveElementFromEntities(TArrayView<const FMassEntityHandle> InEntityHandles, TNotNull<const UScriptStruct*> ElementType)
{
	if (InEntityHandles.IsEmpty())
	{
		return 0;
	}

	int32 TotalRemoved = 0;

	if (FTypePool* TypePool = GetTypePool(ElementType))
	{
		// @todo TypePool.Remove can be optimized by checking for index continuity and whether the following index is in the same chunk.
		for (const FMassEntityHandle& EntityHandle : InEntityHandles)
		{
			TotalRemoved += static_cast<int32>(TypePool->Remove(EntityHandle.Index));
		}
	}
	return TotalRemoved;
}

FStructView FSparseElementsStorage::GetMutableElementDataForEntity(const FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType)
{
	checkSlow(!UE::Mass::IsA<FMassTag>(ElementType));
	if (FTypePool* TypePool = GetTypePool(ElementType))
	{
		return TypePool->GetMutable(EntityHandle.Index);
	}
	return {};
}

FConstStructView FSparseElementsStorage::GetElementDataForEntity(const FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType) const
{
	checkSlow(!UE::Mass::IsA<FMassTag>(ElementType));
	if (const FTypePool* TypePool = GetTypePool(ElementType))
	{
		return TypePool->Get(EntityHandle.Index);
	}
	return {};
}

void FSparseElementsStorage::RemoveEntity(FMassEntityHandle EntityHandle, const FMassElementBitSet& Elements)
{
	for (FMassElementBitSet::FIndexIterator It = Elements.GetIndexIterator(); It; ++It)
	{
		if (TypePools.IsValidIndex(*It))
		{
			TypePools[*It].Remove(EntityHandle.Index);
		}
	}
}

void FSparseElementsStorage::ConfigureType(TNotNull<const UScriptStruct*> ElementType, FTypeConfig& Config)
{
	const int32 TypeIndex = FMassElementBitSet::GetTypeIndex(ElementType);

	if (ensureMsgf(TypePools.IsValidIndex(TypeIndex) == false || TypePools[TypeIndex].GetNumElements() == 0
		, TEXT("%hs: Configuring types after elements have been instantiated is not supported."), __FUNCTION__))
	{
		TypePools.EmplaceAt(TypeIndex, Config);
	}
}


} // namespace UE::Mass
