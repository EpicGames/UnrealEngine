// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSubsystemAccess.h"
#include "MassEntityManager.h"


namespace UE::Mass::Private
{
	template<typename T>
	TSubclassOf<T> ConvertToSubsystemClass(TSubclassOf<USubsystem> SubsystemClass)
	{
		return *(reinterpret_cast<TSubclassOf<T>*>(&SubsystemClass));
	}
} // namespace UE::Mass::Private

//-----------------------------------------------------------------------------
// FMassSubsystemAccess
//-----------------------------------------------------------------------------
FMassSubsystemAccess::FMassSubsystemAccess(FMassEntityManager* InEntityManager)
	: EntityManager(InEntityManager)
{
	Subsystems.AddZeroed(FMassExternalSubsystemBitSet::GetMaxNum());
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMassSubsystemAccess::FMassSubsystemAccess(UWorld* InWorld)
	: EntityManager(nullptr)
{
	ensureMsgf(InWorld == nullptr, TEXT("FMassSubsystemAccess(UWorld*) is deprecated and no longer functional. "
		"Use FMassSubsystemAccess(FMassEntityManager*) instead. Subsystem caching will not work."));
	Subsystems.AddZeroed(FMassExternalSubsystemBitSet::GetMaxNum());
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UWorld* FMassSubsystemAccess::GetWorld() const
{
	return EntityManager ? EntityManager->GetWorld() : nullptr;
}

USubsystem* FMassSubsystemAccess::FetchSubsystemInstance(UWorld* World, TSubclassOf<USubsystem> SubsystemClass)
{
	QUICK_SCOPE_CYCLE_COUNTER(Mass_FetchSubsystemInstance);

	check(SubsystemClass);
	if (SubsystemClass->IsChildOf<UWorldSubsystem>())
	{
		return World
			? World->GetSubsystemBase(UE::Mass::Private::ConvertToSubsystemClass<UWorldSubsystem>(SubsystemClass))
			: nullptr;
	}
	if (SubsystemClass->IsChildOf<UEngineSubsystem>())
	{
		return GEngine->GetEngineSubsystemBase(UE::Mass::Private::ConvertToSubsystemClass<UEngineSubsystem>(SubsystemClass));
	}
	if (SubsystemClass->IsChildOf<UGameInstanceSubsystem>())
	{
		return (World && World->GetGameInstance())
			? World->GetGameInstance()->GetSubsystemBase(UE::Mass::Private::ConvertToSubsystemClass<UGameInstanceSubsystem>(SubsystemClass))
			: nullptr;
	}
	if (SubsystemClass->IsChildOf<ULocalPlayerSubsystem>())
	{
		const ULocalPlayer* LocalPlayer = World ? World->GetFirstLocalPlayerFromController() : nullptr;
		return LocalPlayer
			? LocalPlayer->GetSubsystemBase(UE::Mass::Private::ConvertToSubsystemClass<ULocalPlayerSubsystem>(SubsystemClass))
			: nullptr;
	}
#if WITH_EDITOR
	if (SubsystemClass->IsChildOf<UEditorSubsystem>())
	{
		return GEditor->GetEditorSubsystemBase(UE::Mass::Private::ConvertToSubsystemClass<UEditorSubsystem>(SubsystemClass));
	}
#endif // WITH_EDITOR
	return nullptr;
}

bool FMassSubsystemAccess::CacheSubsystemRequirements(const FMassSubsystemRequirements& SubsystemRequirements)
{
	bool bResult = true;

	if (SubsystemRequirements.IsEmpty() == false)
	{
		for (FMassExternalSubsystemBitSet::FIndexIterator It = SubsystemRequirements.GetRequiredConstSubsystems().GetIndexIterator(); It && bResult; ++It)
		{
			bResult = bResult && CacheSubsystem(*It);
		}

		for (FMassExternalSubsystemBitSet::FIndexIterator It = SubsystemRequirements.GetRequiredMutableSubsystems().GetIndexIterator(); It && bResult; ++It)
		{
			bResult = bResult && CacheSubsystem(*It);
		}
	}

	if (bResult)
	{
		ConstSubsystemsBitSet = SubsystemRequirements.GetRequiredConstSubsystems();
		MutableSubsystemsBitSet = SubsystemRequirements.GetRequiredMutableSubsystems();
	}

	return bResult;
}

bool FMassSubsystemAccess::CacheSubsystem(const uint32 SystemIndex)
{
	if (UNLIKELY(Subsystems.IsValidIndex(SystemIndex) == false))
	{
		Subsystems.AddZeroed(SystemIndex - Subsystems.Num() + 1);
	}

	if (Subsystems[SystemIndex])
	{
		return true;
	}

	const UClass* SubsystemClass = FMassExternalSubsystemBitSet::GetTypeAtIndex(SystemIndex);
	checkSlow(SubsystemClass);

	TSubclassOf<USubsystem> SubsystemSubclass(const_cast<UClass*>(SubsystemClass));
	checkSlow(*SubsystemSubclass);

	if (SubsystemSubclass)
	{
		USubsystem* SystemInstance = FetchSubsystemInstance(GetWorld(), SubsystemSubclass);
		Subsystems[SystemIndex] = SystemInstance;

		if (SystemInstance == nullptr)
		{
			UE::Mass::ErrorReporting::GetOutputDevice().Logf(TEXT("%s "), *SubsystemClass->GetName());
		}

		return SystemInstance != nullptr;
	}

	return false;
}

void FMassSubsystemAccess::SetSubsystemRequirements(const FMassSubsystemRequirements& SubsystemRequirements)
{
	ConstSubsystemsBitSet = SubsystemRequirements.GetRequiredConstSubsystems();
	MutableSubsystemsBitSet = SubsystemRequirements.GetRequiredMutableSubsystems();
}
