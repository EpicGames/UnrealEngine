// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSubsystemBase.h"
#include "HAL/IConsoleManager.h"
#include "Engine/World.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSubsystemBase)

namespace UE::Mass
{
	namespace Subsystems
	{
		void RegisterSubsystemType(TSharedRef<FMassEntityManager> EntityManager, TSubclassOf<USubsystem> SubsystemClass, FSubsystemTypeTraits&& Traits)
		{
			EntityManager->GetTypeManager().RegisterType(SubsystemClass, MoveTemp(Traits));
		}

		void RegisterSubsystemType(FSubsystemCollectionBase& Collection, TSubclassOf<USubsystem> SubsystemClass, FSubsystemTypeTraits&& Traits)
		{
			if (UMassEntitySubsystem* EntitySubsystem = Collection.InitializeDependency<UMassEntitySubsystem>())
			{
				RegisterSubsystemType(EntitySubsystem->GetMutableEntityManager().AsShared(), SubsystemClass, MoveTemp(Traits));
			}
		}
	} // namespace UE::Mass::Subsystem

	namespace Private
	{
		/** 
		 * A helper function calling PostInitialize and OnWorldBeginPlay for the given subsystem, provided the world has already begun play.
		 * @see UMassSubsystemBase::HandleLateCreation for more detail
		 */
		void HandleLateCreation(UWorldSubsystem& MassWorldSubsystem, const UE::Mass::Subsystems::FInitializationState InitializationState)
		{
			// handle late creation
			UWorld* World = MassWorldSubsystem.GetWorld();
			if (World)
			{
				if (World->IsInitialized() == true && InitializationState.bPostInitializeCalled == false)
				{
					MassWorldSubsystem.PostInitialize();
				}
				if (World->HasBegunPlay() == true && InitializationState.bOnWorldBeginPlayCalled == false)
				{
					MassWorldSubsystem.OnWorldBeginPlay(*World);
				}
			}
		}

		bool bRuntimeSubsystemsEnabled = true;

		namespace
		{
			FAutoConsoleVariableRef AnonymousCVars[] =
			{
				{ TEXT("mass.RuntimeSubsystemsEnabled")
				, bRuntimeSubsystemsEnabled
				, TEXT("true by default, setting to false will prevent auto-creation of game-time Mass-related subsystems. Needs to be set before world loading.")
				, ECVF_Default }
			};
		}
	} // namespace UE::Mass::Private
} // namespace UE::Mass

//-----------------------------------------------------------------------------
// UMassSubsystemBase
//-----------------------------------------------------------------------------
bool UMassSubsystemBase::AreRuntimeMassSubsystemsAllowed(UObject* Outer)
{
	return UE::Mass::Private::bRuntimeSubsystemsEnabled;
}

bool UMassSubsystemBase::ShouldCreateSubsystem(UObject* Outer) const 
{
	return UMassSubsystemBase::AreRuntimeMassSubsystemsAllowed(Outer) && Super::ShouldCreateSubsystem(Outer);
}

void UMassSubsystemBase::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// This ensure is here to make sure we handle HandleLateCreation() gracefully, we don't expect it to ever trigger unless users start to manually call the functions
	ensureMsgf(InitializationState.bInitializeCalled == false, TEXT("%hs called multiple times"), __FUNCTION__);
	InitializationState.bInitializeCalled = true;

	// register the given child class with default traits. Child-class can always override the traits data registered here.
	// Note that we're not performing the registration for UMassEntitySubsystem since that's the subsystem
	// we use to get access to the EntityManager instance in the first place. UMassEntitySubsystem has to perform the registration manually
	if (GetClass()->IsChildOf(UMassEntitySubsystem::StaticClass()) == false)
	{
		// register the given child class with default traits. Child-class can always override the traits data registered here.
		UE::Mass::Subsystems::RegisterSubsystemType(Collection, GetClass(), UE::Mass::FSubsystemTypeTraits::Make<UMassSubsystemBase>());
	}
}

void UMassSubsystemBase::PostInitialize()
{
	Super::PostInitialize();

	// This ensure is here to make sure we handle HandleLateCreation() gracefully, we don't expect it to ever trigger unless users start to manually call the functions
	ensureMsgf(InitializationState.bPostInitializeCalled == false, TEXT("%hs called multiple times"), __FUNCTION__);
	InitializationState.bPostInitializeCalled = true;
}

void UMassSubsystemBase::Deinitialize()
{
	InitializationState = UE::Mass::Subsystems::FInitializationState();

	Super::Deinitialize();
}

void UMassSubsystemBase::OnWorldBeginPlay(UWorld& InWorld)
{
	Super::OnWorldBeginPlay(InWorld);

	// This ensure is here to make sure we handle HandleLateCreation() gracefully, we don't expect it to ever trigger unless users start to manually call the functions
	ensureMsgf(InitializationState.bOnWorldBeginPlayCalled == false, TEXT("%hs called multiple times"), __FUNCTION__);
	InitializationState.bOnWorldBeginPlayCalled = true;
}

void UMassSubsystemBase::HandleLateCreation()
{
	UE::Mass::Private::HandleLateCreation(*this, InitializationState);
}

//-----------------------------------------------------------------------------
// UMassTickableSubsystemBase
//-----------------------------------------------------------------------------
bool UMassTickableSubsystemBase::ShouldCreateSubsystem(UObject* Outer) const
{
	return UMassSubsystemBase::AreRuntimeMassSubsystemsAllowed(Outer) && Super::ShouldCreateSubsystem(Outer);
}

void UMassTickableSubsystemBase::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// This ensure is here to make sure we handle HandleLateCreation() gracefully, we don't expect it to ever trigger unless users start to manually call the functions
	ensureMsgf(InitializationState.bInitializeCalled == false, TEXT("%hs called multiple times"), __FUNCTION__);
	InitializationState.bInitializeCalled = true;

	// register the given child class with default traits. Child-class can always override the traits data registered here.
	UE::Mass::Subsystems::RegisterSubsystemType(Collection, GetClass(), UE::Mass::FSubsystemTypeTraits::Make<UMassTickableSubsystemBase>());
}

void UMassTickableSubsystemBase::PostInitialize()
{
	Super::PostInitialize();

	// This ensure is here to make sure we handle HandleLateCreation() gracefully, we don't expect it to ever trigger unless users start to manually call the functions
	ensureMsgf(InitializationState.bPostInitializeCalled == false, TEXT("%hs called multiple times"), __FUNCTION__);
	InitializationState.bPostInitializeCalled = true;
}

void UMassTickableSubsystemBase::Deinitialize()
{
	InitializationState = UE::Mass::Subsystems::FInitializationState();

	Super::Deinitialize();
}

void UMassTickableSubsystemBase::OnWorldBeginPlay(UWorld& InWorld)
{
	Super::OnWorldBeginPlay(InWorld);

	// This ensure is here to make sure we handle HandleLateCreation() gracefully, we don't expect it to ever trigger unless users start to manually call the functions
	ensureMsgf(InitializationState.bOnWorldBeginPlayCalled == false, TEXT("%hs called multiple times"), __FUNCTION__);
	InitializationState.bOnWorldBeginPlayCalled = true;
}

void UMassTickableSubsystemBase::HandleLateCreation()
{
	UE::Mass::Private::HandleLateCreation(*this, InitializationState);
}
