// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassTypeManager.h"
#include "MassEntityManager.h"
#include "Mass/TestableEnsures.h"
#include "Mass/EntityElementTypes.h"
#include "MassEntityRelations.h"
#include "Misc/CoreDelegates.h"
#include "Misc/MTAccessDetector.h"


namespace UE::Mass
{
	namespace Private
	{
		struct FStaticTypeEntry
		{
			FTypeInfo TypeInfo;
			int32 RefCount = 0;
		};

		struct FStaticTypeRegistry
		{
			TMap<TObjectKey<const UStruct>, FStaticTypeEntry> Map;
			UE_MT_DECLARE_RW_ACCESS_DETECTOR(MTAccessDetector);
		};

		/** Static registry of types that should be registered into every FTypeManager instance during RegisterBuiltInTypes(). */
		FStaticTypeRegistry& GetStaticTypeRegistry()
		{
			static FStaticTypeRegistry Registry;
			return Registry;
		}

		TObjectKey<const UStruct> AddToStaticRegistry(TNotNull<const UStruct*> InType, FTypeInfo&& TypeInfo)
		{
			TObjectKey<const UStruct> TypeKey(InType);

			FStaticTypeRegistry& Registry = GetStaticTypeRegistry();
			UE_MT_SCOPED_WRITE_ACCESS(Registry.MTAccessDetector);

			FStaticTypeEntry* Existing = Registry.Map.Find(TypeKey);
			if (Existing)
			{
				++Existing->RefCount;
			}
			else
			{
				Registry.Map.Add(TypeKey, FStaticTypeEntry{ MoveTemp(TypeInfo), 1 });
			}
			return TypeKey;
		}

		void UnregisterStaticTypeInternal(TObjectKey<const UStruct> TypeKey)
		{
			FStaticTypeRegistry& Registry = GetStaticTypeRegistry();
			UE_MT_SCOPED_WRITE_ACCESS(Registry.MTAccessDetector);

			FStaticTypeEntry* Entry = Registry.Map.Find(TypeKey);
			if (Entry && --Entry->RefCount <= 0)
			{
				Registry.Map.Remove(TypeKey);
			}
		}
	} // namespace Private

	//-----------------------------------------------------------------------------
	// FTypeHandle
	//-----------------------------------------------------------------------------
	FTypeHandle::FTypeHandle(TObjectKey<const UStruct> InTypeKey)
		: TypeKey(InTypeKey)
	{
		
	}

	//-----------------------------------------------------------------------------
	// FTypeManager
	//-----------------------------------------------------------------------------
	FTypeManager::FOnRegisterBuiltInTypes FTypeManager::OnRegisterBuiltInTypes;

	FTypeManager::FTypeManager(FMassEntityManager& InEntityManager)
		: OuterEntityManager(InEntityManager)
	{
	}

	void FTypeManager::RegisterBuiltInTypes()
	{
		if (!ensureMsgf(!bBuiltInTypesRegistered, TEXT("%hs: built-in types already registered!"), __FUNCTION__))
		{
			return;
		}

		// Drain statically registered types first, so delegate handlers can see/override them.
		// We copy rather than move because the static registry is shared across all instances.
		UE::Mass::Private::FStaticTypeRegistry& Registry = UE::Mass::Private::GetStaticTypeRegistry();
		UE_MT_SCOPED_READ_ACCESS(Registry.MTAccessDetector);

		for (TPair<TObjectKey<const UStruct>, UE::Mass::Private::FStaticTypeEntry>& Pair : Registry.Map)
		{
			const UStruct* ResolvedType = Pair.Key.ResolveObjectPtr();
			if (ensureMsgf(ResolvedType, TEXT("Static type registration references a stale UStruct pointer")))
			{
				FTypeInfo TypeInfoCopy = Pair.Value.TypeInfo;
				RegisterTypeInternal(ResolvedType, MoveTemp(TypeInfoCopy));
			}
		}

		OnRegisterBuiltInTypes.Broadcast(*this);
		bBuiltInTypesRegistered = true;
	}

	FTypeHandle FTypeManager::RegisterTypeInternal(TNotNull<const UStruct*> InType, FTypeInfo&& TypeInfo)
	{
		FTypeHandle TypeHandle(InType);
		FTypeInfo* ExistingData = TypeDataMap.Find(TypeHandle);
		if (LIKELY(ExistingData == nullptr))
		{
			if (TypeInfo.Traits.IsType<FSubsystemTypeTraits>())
			{
				SubsystemTypes.Add(TypeHandle);
			}

			TypeDataMap.Add(TypeHandle, MoveTemp(TypeInfo));
		}
		else
		{
			// We're registering the same type multiple times. There are multiple cases where this can happen:
			//
			// * The most common occurrence of this will be with already registered subsystems' subclasses. The
			//   subclasses can change the data registered on their behalf by the super class, but most of the time that
			//   won't be necessary. We assume the new data is more up-to-date and overwrite the existing data.
			//
			// * A GameFeaturePlugin is activating for a second time (e.g. activate > deactivate > activate). In this
			//   case we always allow the duplicate registration and overwrite the existing data.
			//
			// In theory we should ensure that the new type info matches the old type info (for the second case only),
			// but FRelationTypeTraits contains a TFunction and does not support operator==.

			bool bWasSubsystem = ExistingData->Traits.IsType<FSubsystemTypeTraits>();
			bool bIsSubsystem = TypeInfo.Traits.IsType<FSubsystemTypeTraits>();
			ensureAlways(bWasSubsystem == bIsSubsystem);

			UE_LOGF(LogMass, Verbose, "Re-registering type: %ls", *InType->GetName());
			*ExistingData = MoveTemp(TypeInfo);
		}

		return TypeHandle;
	}

	FTypeHandle FTypeManager::RegisterType(TNotNull<const UStruct*> InType, FSubsystemTypeTraits&& TypeTraits)
	{
		FTypeInfo TypeInfo;
		TypeInfo.TypeName = InType->GetFName();
		TypeInfo.Traits.Set<FSubsystemTypeTraits>(MoveTemp(TypeTraits));
		return RegisterTypeInternal(InType, MoveTemp(TypeInfo));
	}

	FTypeHandle FTypeManager::RegisterType(TNotNull<const UStruct*> InType, FSharedFragmentTypeTraits&& TypeTraits)
	{
		testableCheckfReturn(UE::Mass::IsA<FMassSharedFragment>(InType), {}
			, TEXT("%s is not a valid shared fragment type"), *InType->GetName());

		FTypeInfo TypeInfo;
		TypeInfo.TypeName = InType->GetFName();
		TypeInfo.Traits.Set<FSharedFragmentTypeTraits>(MoveTemp(TypeTraits));
		return RegisterTypeInternal(InType, MoveTemp(TypeInfo));
	}

	FTypeHandle FTypeManager::RegisterType(FRelationTypeTraits&& TypeTraits)
	{
		TNotNull<const UScriptStruct*> InType = TypeTraits.GetRelationTagType();
		testableCheckfReturn(UE::Mass::IsA<FMassRelation>(InType), return {}
			, TEXT("%s is not a valid relation type"), *InType->GetName());

		testableCheckfReturn(TypeTraits.RelationFragmentType->IsChildOf(FMassRelationFragment::StaticStruct()), return {}
			, TEXT("%s is not a valid TypeTraits.RelationFragmentType, needs to derive from FMassRelationFragment")
			, *TypeTraits.RelationFragmentType->GetName());

		if (TypeTraits.RelationName.IsNone())
		{
			TypeTraits.RelationName = InType->GetFName();
			TypeTraits.SetDebugInFix(TypeTraits.RelationName.ToString());
		}

		FTypeInfo* ExistingData = TypeDataMap.Find(FTypeHandle(InType));
		
		if (!testableEnsureMsgf(bBuiltInTypesRegistered == false || ExistingData == nullptr
			, TEXT("Modifying relationship after registration done is not supported")))
		{
			return {};
		}

		FTypeInfo TypeInfo;
		TypeInfo.TypeName = InType->GetFName();
		if (TypeTraits.RegisteredGroupType.IsValid() == false)
		{
			TypeTraits.RegisteredGroupType = OuterEntityManager.FindOrAddArchetypeGroupType(TypeTraits.RelationName);
		}
		TypeInfo.Traits.Set<FRelationTypeTraits>(MoveTemp(TypeTraits));

		const FTypeHandle RegisteredTypeHandle = RegisterTypeInternal(InType, MoveTemp(TypeInfo));
		if (RegisteredTypeHandle.IsValid())
		{
			if (bBuiltInTypesRegistered)
			{
				// if the built-in types are already registered we need to notify the entity manager that there's a new
				// relation type, that might require additional handling (like creation of appropriate entity destruction observers)
				// Note that we don't call this during built-in types registration to give project-specific code a chance
				// to override type traits before the entity manager handles the registered types.
				OuterEntityManager.OnNewTypeRegistered(RegisteredTypeHandle);
			}
		}

		return RegisteredTypeHandle;
	}

	FTypeHandle FTypeManager::GetRelationTypeHandle(const TNotNull<const UScriptStruct*> RelationOrElementType) const
	{
		const bool bValidRelationshipType = UE::Mass::IsA<FMassRelation>(RelationOrElementType);
		ensureMsgf(bValidRelationshipType, TEXT("%s is not a valid relationship type"), *RelationOrElementType->GetName());

		return UE::Mass::IsA<FMassRelation>(RelationOrElementType) 
			? FTypeHandle(RelationOrElementType)
			: FTypeHandle();
	}

	bool FTypeManager::IsValidRelationType(const TNotNull<const UScriptStruct*> RelationOrElementType) const
	{
		return UE::Mass::IsA<FMassRelation>(RelationOrElementType) && TypeDataMap.Contains(FTypeHandle(RelationOrElementType));
	}

	//-----------------------------------------------------------------------------
	// FStaticTypeRegistrationHandle
	//-----------------------------------------------------------------------------
	FTypeManager::FStaticTypeRegistrationHandle::~FStaticTypeRegistrationHandle()
	{
		if (IsValid())
		{
			UE::Mass::Private::UnregisterStaticTypeInternal(this->RegisteredType);
			this->RegisteredType = TObjectKey<const UStruct>();
		}
	}

	FTypeManager::FStaticTypeRegistrationHandle::FStaticTypeRegistrationHandle(FStaticTypeRegistrationHandle&& InOther)
		: RegisteredType(InOther.RegisteredType)
	{
		InOther.RegisteredType = TObjectKey<const UStruct>();
	}

	FTypeManager::FStaticTypeRegistrationHandle& FTypeManager::FStaticTypeRegistrationHandle::operator=(FStaticTypeRegistrationHandle&& InOther)
	{
		if (this != &InOther)
		{
			if (IsValid())
			{
				UE::Mass::Private::UnregisterStaticTypeInternal(this->RegisteredType);
			}
			this->RegisteredType = InOther.RegisteredType;
			InOther.RegisteredType = TObjectKey<const UStruct>();
		}
		return *this;
	}

	//-----------------------------------------------------------------------------
	// Static type registration (handle-based)
	//-----------------------------------------------------------------------------
	FTypeManager::FStaticTypeRegistrationHandle FTypeManager::RegisterStaticType(TNotNull<const UStruct*> InType, FSubsystemTypeTraits&& TypeTraits)
	{
		FTypeInfo TypeInfo;
		TypeInfo.TypeName = InType->GetFName();
		TypeInfo.Traits.Set<FSubsystemTypeTraits>(MoveTemp(TypeTraits));

		FStaticTypeRegistrationHandle Handle;
		Handle.RegisteredType = UE::Mass::Private::AddToStaticRegistry(InType, MoveTemp(TypeInfo));
		return Handle;
	}

	FTypeManager::FStaticTypeRegistrationHandle FTypeManager::RegisterStaticType(TNotNull<const UStruct*> InType, FSharedFragmentTypeTraits&& TypeTraits)
	{
		if (!ensureMsgf(UE::Mass::IsA<FMassSharedFragment>(InType), TEXT("%s is not a valid shared fragment type"), *InType->GetName()))
		{
			return FStaticTypeRegistrationHandle();
		}

		FTypeInfo TypeInfo;
		TypeInfo.TypeName = InType->GetFName();
		TypeInfo.Traits.Set<FSharedFragmentTypeTraits>(MoveTemp(TypeTraits));

		FStaticTypeRegistrationHandle Handle;
		Handle.RegisteredType = UE::Mass::Private::AddToStaticRegistry(InType, MoveTemp(TypeInfo));
		return Handle;
	}

} // namespace UE::Mass
