// Copyright Epic Games, Inc. All Rights Reserved.

#include "Relations/MassChildOf.h"
#include "MassTypeManager.h"
#include "MassEntityManager.h"
#include "Misc/CoreDelegates.h"
#include "MassExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassChildOf)

namespace UE::Mass
{
	namespace Relations
	{
		FTypeHandle ChildOfHandle;

		struct FChildOfRelationTypeInitializer
		{
			FChildOfRelationTypeInitializer()
				: RegisterBuiltInTypesHandle(FTypeManager::OnRegisterBuiltInTypes.AddStatic(&FChildOfRelationTypeInitializer::RegisterType))
			{
				UE::Mass::Relations::ChildOfHandle = FTypeManager::MakeTypeHandle(FMassChildOfRelation::StaticStruct());
			}

			~FChildOfRelationTypeInitializer()
			{
				FTypeManager::OnRegisterBuiltInTypes.Remove(RegisterBuiltInTypesHandle);
			}

			static void RegisterType(FTypeManager& InOutTypeManager)
			{
				FRelationTypeTraits ChildOfRelation(FMassChildOfRelation::StaticStruct());
				ChildOfRelation.RoleTraits[static_cast<int32>(ERelationRole::Object)].bExclusive = true;
				ChildOfRelation.RoleTraits[static_cast<int32>(ERelationRole::Object)].DestructionPolicy = ERemovalPolicy::Destroy;

				ChildOfRelation.RoleTraits[static_cast<int32>(ERelationRole::Subject)].bExclusive = false;
				ChildOfRelation.RoleTraits[static_cast<int32>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::CleanUp;
				// @todo unused for now.
				ChildOfRelation.RoleTraits[static_cast<int32>(ERelationRole::Subject)].Element = FMassChildOfFragment::StaticStruct();

				ChildOfRelation.bHierarchical = true;
				ChildOfRelation.RelationName = TEXT("ChildOf");
				
				ChildOfRelation.RegisteredGroupType = InOutTypeManager.GetEntityManager().FindOrAddArchetypeGroupType(ChildOfRelation.RelationName);

				ChildOfRelation.RelationEntityCreationObserverClass = UMassChildOfRelationEntityCreation::StaticClass();

				InOutTypeManager.RegisterType(MoveTemp(ChildOfRelation));
			}

			const FDelegateHandle RegisterBuiltInTypesHandle;
		};

		void RegisterChildOfRelation()
		{
			static FChildOfRelationTypeInitializer BuildInRelationHandlesInitializer;
		}
	} // namespace Relations
} // namespace UE::Mass

//-----------------------------------------------------------------------------
// UMassChildOfRelationEntityCreation
//-----------------------------------------------------------------------------
void UMassChildOfRelationEntityCreation::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	struct FRelationInstanceRegistration
	{
		FMassEntityHandle ChildHandle;
		FMassEntityHandle ParentHandle;
		FMassEntityHandle RelationEntityHandle;
	};

	TArray<FRelationInstanceRegistration> RelationInstances;
	EntityQuery.ForEachEntityChunk(Context, [&RelationInstances, ObservedType = GetSingleObservedTypeChecked()](FMassExecutionContext& ExecutionContext)
	{
		TConstArrayView<FMassRelationFragment> RelationFragments = ExecutionContext.GetFragmentView<FMassRelationFragment>(ObservedType);
		for (FMassExecutionContext::FEntityIterator EntityIt = ExecutionContext.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			RelationInstances.Add({
				RelationFragments[EntityIt].Subject
				, RelationFragments[EntityIt].Object
				, ExecutionContext.GetEntity(EntityIt)
			});
		}
	});
	// @todo if we don't do mapping how do we "remove relation", how do we find the relation entity?
	// @todo, again, we need "external requirement" meaning "things we touch on other entities, so needs
	// to be accounted for in dependencies, but we're not binding or expecting them to be in the very archetypes we process"
	for (const FRelationInstanceRegistration& RelationInstance : RelationInstances)
	{
		// using GetFragmentDataStruct to support this observer processor being used by relations that extend the ChildOf
		// relation by using a fragment deriving from FMassChildOfFragment
		// @todo use the fragment type from traits, not hardcoded FMassChildOfFragment
		FStructView FragmentView = EntityManager.GetFragmentDataStruct(RelationInstance.ChildHandle, FMassChildOfFragment::StaticStruct());
		FMassChildOfFragment* ChildOfFragment = FragmentView.GetPtr<FMassChildOfFragment>();
		if (ensure(ChildOfFragment))
		{
			ChildOfFragment->Parent = RelationInstance.ParentHandle;
		}
	}
}
