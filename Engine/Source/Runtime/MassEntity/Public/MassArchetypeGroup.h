// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
// HEADER_UNIT_SKIP - Deprecated
#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.8, "MassArchetypeGroup.h has moved to the MassCore module and been renamed to Mass/ArchetypeGroup.h. "
    "Include Mass/ArchetypeGroup.h from MassCore and add a MassCore dependency to your .Build.cs.")
#include "Mass/ArchetypeGroup.h"
