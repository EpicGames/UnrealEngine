// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/ArchetypeGroup.h"
#include "MassCommands.h"

#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
#	define DEBUG_NAME(Name) , FName(TEXT(Name))
#	define DEBUG_NAME_PARAM(Name) , const FName InDebugName = TEXT(Name)
#	define FORWARD_DEBUG_NAME_PARAM , InDebugName
#else
#	define DEBUG_NAME(Name)
#	define DEBUG_NAME_PARAM(Name)
#	define FORWARD_DEBUG_NAME_PARAM
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

namespace UE::Mass::Command
{
	//-----------------------------------------------------------------------------
	// FGroupEntities
	//-----------------------------------------------------------------------------
	struct FGroupEntities : public FMassBatchedEntityCommand
	{
		using Super = FMassBatchedEntityCommand;
		FGroupEntities()
			: Super(EMassCommandOperationType::ChangeComposition DEBUG_NAME("GroupEntities"))
		{}

		void Add(FArchetypeGroupHandle GroupHandle, TArray<FMassEntityHandle>&& Entities)
		{
			Groups.FindOrAdd(GroupHandle).Append(MoveTemp(Entities));
		}

	protected:
		virtual void Run(FMassEntityManager& EntityManager) override
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandGroupEntities_Execute);
			for (auto Group : Groups)
			{
				TArray<FMassArchetypeEntityCollection> CollectionsAtLevel;
				UE::Mass::Utils::CreateEntityCollections(EntityManager, Group.Value
					, FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates
					, CollectionsAtLevel);

				EntityManager.BatchGroupEntities(Group.Key, CollectionsAtLevel);
			}
		}

		TMap<FArchetypeGroupHandle, TArray<FMassEntityHandle>> Groups;
	};

} // namespace UE::Mass::Command
