// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "UObject/Class.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Containers/ArrayView.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Containers/StridedView.h"
#include "Containers/UnrealString.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"

#define UE_API MASSENTITY_API

struct FMassEntityManager;
struct FMassArchetypeData;
struct FMassExecutionContext;
struct FMassFragment;
struct FMassArchetypeChunkIterator;
struct FMassEntityHandle;
struct FMassEntityQuery;
struct FMassArchetypeEntityCollection;
struct FMassEntityView;
struct FMassDebugger;
struct FMassArchetypeHelper;
struct FMassArchetypeVersionedHandle;
struct FMassChunkProcessingQueueParams;

using FMassEntityExecuteFunction = TFunction< void(FMassExecutionContext& /*ExecutionContext*/, int32 /*EntityIndex*/) >;
using FMassExecuteFunction = TFunction< void(FMassExecutionContext& /*ExecutionContext*/) >;
using FMassChunkConditionFunction = TFunction< bool(const FMassExecutionContext& /*ExecutionContext*/) >;
using FMassQueueChunkFunction = TFunction< void(FMassChunkProcessingQueueParams&&) >;

//-----------------------------------------------------------------------------
// FMassArchetypeHandle
//-----------------------------------------------------------------------------
/** An opaque handle to an archetype */
struct FMassArchetypeHandle final
{
	FMassArchetypeHandle() = default;
	bool IsValid() const;

	bool operator==(const FMassArchetypeHandle& Other) const;
	bool operator!=(const FMassArchetypeHandle& Other) const;

	MASSENTITY_API friend uint32 GetTypeHash(const FMassArchetypeHandle& Instance);

	void Reset();

	/**
	 * Returns the composition bitset for the archetype, asserts if archetype is invalid
	 */
	MASSENTITY_API const FMassElementBitSet& GetCompositionBitSetChecked() const;

private:
	FMassArchetypeHandle(const TSharedPtr<FMassArchetypeData>& InDataPtr);

	TSharedPtr<FMassArchetypeData> DataPtr;

	friend FMassArchetypeHelper;
	friend FMassEntityManager;
	friend FMassArchetypeVersionedHandle;
};

struct FMassArchetypeVersionedHandle final
{
	FMassArchetypeVersionedHandle() = default;
	MASSENTITY_API FMassArchetypeVersionedHandle(const FMassArchetypeHandle& InHandle);
	FMassArchetypeVersionedHandle(FMassArchetypeHandle&& InHandle);

	bool IsValid() const;
	bool operator==(const FMassArchetypeVersionedHandle& Other) const;
	bool operator!=(const FMassArchetypeVersionedHandle& Other) const;
	MASSENTITY_API bool IsUpToDate() const;

	operator FMassArchetypeHandle() const;

	MASSENTITY_API friend uint32 GetTypeHash(const FMassArchetypeHandle& Instance);
private:
	FMassArchetypeHandle ArchetypeHandle;
	/**
	 * This value indicates whether the target archetype had its entities moved around since the handle creations.
	 * The information is useful in a couple of scenarios (like making sure an entity collection is up to date),
	 * but in most cases the users should not concern themselves with this value.
	 * Note that the value is not used as part of hash calculation, it's effectively transient. 
	 */
	uint32 HandleVersion = 0;
};

//-----------------------------------------------------------------------------
// FMassArchetypeEntityCollection
//-----------------------------------------------------------------------------
/** A struct that converts an arbitrary array of entities of given Archetype into a sequence of continuous
 *  entity chunks. The goal is to have the user create an instance of this struct once and run through a bunch of
 *  systems. The runtime code usually uses FMassArchetypeChunkIterator to iterate on the chunk collection.
 */
struct FMassArchetypeEntityCollection 
{
public:
	struct FArchetypeEntityRange
	{
		int32 ChunkIndex = INDEX_NONE;
		 /** 
		  * The index of the first entity within the specified chunk that starts this subchunk.
		  */
		int32 SubchunkStart = 0;
		/** 
		 * The number of entities in this subchunk.
		 * If Length is 0 or negative, it indicates that the range covers all remaining entities 
		 * in the chunk starting from SubchunkStart.
		 */
		int32 Length = 0;

		FArchetypeEntityRange() = default;
		explicit FArchetypeEntityRange(const int32 InChunkIndex, const int32 InSubchunkStart = 0, const int32 InLength = 0)
			: ChunkIndex(InChunkIndex), SubchunkStart(InSubchunkStart), Length(InLength)
		{
		}
		/** Note that we consider invalid-length chunks valid as long as ChunkIndex and SubchunkStart are valid */
		bool IsSet() const
		{
			return ChunkIndex != INDEX_NONE && SubchunkStart >= 0;
		}

		/** Checks if given InRange comes right after this instance */
		bool IsAdjacentAfter(const FArchetypeEntityRange& Other) const
		{
			return ChunkIndex == Other.ChunkIndex && SubchunkStart + Length == Other.SubchunkStart;
		}

		bool IsOverlapping(const FArchetypeEntityRange& Other) const
		{
			return ChunkIndex == Other.ChunkIndex
				&& (*this < Other
					// note that Length == 0 means "all the entities starting from SubchunkStart
					? (SubchunkStart + Length > Other.SubchunkStart || Length == 0)
					: (Other.SubchunkStart + Other.Length > SubchunkStart || Other.Length == 0)
				);
		}

		bool operator==(const FArchetypeEntityRange& Other) const
		{
			return ChunkIndex == Other.ChunkIndex && SubchunkStart == Other.SubchunkStart && Length == Other.Length;
		}
		bool operator!=(const FArchetypeEntityRange& Other) const
		{
			return !(*this == Other);
		}

		bool operator<(const FArchetypeEntityRange& Other) const
		{
			return (ChunkIndex != Other.ChunkIndex)
				? ChunkIndex < Other.ChunkIndex
				: (SubchunkStart != Other.SubchunkStart
					? SubchunkStart < Other.SubchunkStart
					: Length < Other.Length);
		}
	};

	enum EDuplicatesHandling
	{
		NoDuplicates,	// indicates that the caller guarantees there are no duplicates in the input Entities collection
						// note that in no-shipping builds a `check` will fail if duplicates are present.
		FoldDuplicates,	// indicates that it's possible that Entities contains duplicates. The input Entities collection 
						// will be processed and duplicates will be removed.
	};

	enum EInitializationType
	{
		GatherAll,	// default behavior, makes given FMassArchetypeEntityCollection instance represent all entities of the given archetype
		DoNothing,	// meant for procedural population by external code (like child classes)
	};

	using FEntityRangeArray = TArray<FArchetypeEntityRange>;
	using FConstEntityRangeArrayView = TConstArrayView<FArchetypeEntityRange>;

private:
	FEntityRangeArray Ranges;
	/** entity indices indicated by EntityRanges are only valid with given Archetype */
	FMassArchetypeVersionedHandle Archetype;

public:
	FMassArchetypeEntityCollection() = default;
	UE_API FMassArchetypeEntityCollection(const FMassArchetypeHandle& InArchetype, TConstArrayView<FMassEntityHandle> InEntities, EDuplicatesHandling DuplicatesHandling);
	/** optimized, special case for a single-entity */
	UE_API FMassArchetypeEntityCollection(const FMassArchetypeHandle& InArchetype, const FMassEntityHandle EntityHandle);
	UE_API FMassArchetypeEntityCollection(FMassArchetypeHandle&& InArchetype, const FMassEntityHandle EntityHandle);
	UE_API explicit FMassArchetypeEntityCollection(const FMassArchetypeHandle& InArchetypeHandle, const EInitializationType Initialization = EInitializationType::GatherAll);
	UE_API explicit FMassArchetypeEntityCollection(TSharedPtr<FMassArchetypeData>& InArchetype, const EInitializationType Initialization = EInitializationType::GatherAll);
	FMassArchetypeEntityCollection(const FMassArchetypeHandle& InArchetypeHandle, FEntityRangeArray&& InEntityRanges)
		: Ranges(Forward<FEntityRangeArray>(InEntityRanges))
		, Archetype(InArchetypeHandle)
	{}

	FConstEntityRangeArrayView GetRanges() const;
	FMassArchetypeHandle GetArchetype() const;
	bool IsEmpty() const;
	bool IsUpToDate() const;

	UE_DEPRECATED(5.6, "This function is deprecated. Use !IsEmpty() instead.")
	bool IsSet() const;

	void Reset() 
	{ 
		Archetype = FMassArchetypeVersionedHandle();
		Ranges.Reset();
	}

	/** The comparison function that checks if Other is identical to this. Intended for diagnostics/debugging. */
	UE_API bool IsSame(const FMassArchetypeEntityCollection& Other) const;
	bool IsSameArchetype(const FMassArchetypeEntityCollection& Other) const;

	/**
	 * Appends ranges of the given FMassArchetypeEntityCollection instance. Note that it can be safely done only
	 * when both collections host entities of the same archetype, and both were created with the same version
	 * of said archetype.
	 * Additionally, we don't expect the operation to produce overlapping entity ranges and this assumption is
	 * only verified in debug builds (i.e. use it only when you're certain no range overlaps are possible).
	 */
	template<typename T>
	requires std::is_same_v<typename TDecay<T>::Type, FMassArchetypeEntityCollection>
	void Append(T&& Other);

	/**
	 * Converts stored entity ranges to FMassEntityHandles and appends them to InOutHandles.
	 * Note that the operation is only supported for already created entities (i.e. not "reserved")
	 * @return whether any entity handles have been actually exported
	 */
	UE_API bool ExportEntityHandles(TArray<FMassEntityHandle>& InOutHandles) const;

	static UE_API bool DoesContainOverlappingRanges(FConstEntityRangeArrayView Ranges);

#if WITH_MASSENTITY_DEBUG
	UE_API int32 DebugCountEntities() const;
#endif // WITH_MASSENTITY_DEBUG

protected:
	friend struct FMassArchetypeEntityCollectionWithPayload;
	UE_API void BuildEntityRanges(TStridedView<const int32> TrueIndices);
	static UE_API FArchetypeEntityRange CreateRangeForEntity(const FMassArchetypeHandle& InArchetype, const FMassEntityHandle EntityHandle);

private:
	void GatherChunksFromArchetype();
};

struct FMassArchetypeEntityCollectionWithPayload
{
	explicit FMassArchetypeEntityCollectionWithPayload(const FMassArchetypeEntityCollection& InEntityCollection)
		: Entities(InEntityCollection)
	{
	}

	explicit FMassArchetypeEntityCollectionWithPayload(FMassArchetypeEntityCollection&& InEntityCollection)
		: Entities(MoveTempIfPossible(InEntityCollection))
	{
	}

	static UE_API void CreateEntityRangesWithPayload(const FMassEntityManager& EntityManager, const TConstArrayView<FMassEntityHandle> Entities
		, const FMassArchetypeEntityCollection::EDuplicatesHandling DuplicatesHandling, FMassGenericPayloadView Payload
		, TArray<FMassArchetypeEntityCollectionWithPayload>& OutEntityCollections);

	const FMassArchetypeEntityCollection& GetEntityCollection() const
	{
		return Entities;
	}
	const FMassGenericPayloadViewSlice& GetPayload() const
	{
		return PayloadSlice;
	}

private:
	FMassArchetypeEntityCollectionWithPayload(const FMassArchetypeHandle& InArchetype, TStridedView<const int32> TrueIndices, FMassGenericPayloadViewSlice&& Payload);

	FMassArchetypeEntityCollection Entities;
	FMassGenericPayloadViewSlice PayloadSlice;
};

//-----------------------------------------------------------------------------
// FMassArchetypeChunkIterator
//-----------------------------------------------------------------------------
/**
 *  The type used to iterate over given archetype's chunks, be it full, continuous chunks or sparse subchunks. It hides
 *  this details from the rest of the system.
 */
struct FMassArchetypeChunkIterator
{
private:
	FMassArchetypeEntityCollection::FConstEntityRangeArrayView EntityRanges;
	int32 CurrentChunkIndex = 0;

public:
	explicit FMassArchetypeChunkIterator(const FMassArchetypeEntityCollection::FConstEntityRangeArrayView& InEntityRanges)
		: EntityRanges(InEntityRanges), CurrentChunkIndex(0)
	{
	}

	operator bool() const
	{
		return EntityRanges.IsValidIndex(CurrentChunkIndex) && EntityRanges[CurrentChunkIndex].IsSet();
	}
	FMassArchetypeChunkIterator& operator++()
	{
		++CurrentChunkIndex;
		return *this;
	}

	const FMassArchetypeEntityCollection::FArchetypeEntityRange* operator->() const
	{
		check(bool(*this));
		return &EntityRanges[CurrentChunkIndex];
	}
	const FMassArchetypeEntityCollection::FArchetypeEntityRange& operator*() const
	{
		check(bool(*this));
		return EntityRanges[CurrentChunkIndex];
	}
};

//-----------------------------------------------------------------------------
// FMassRawEntityInChunkData
//-----------------------------------------------------------------------------
struct FMassRawEntityInChunkData 
{
	FMassRawEntityInChunkData() = default;
	FMassRawEntityInChunkData(uint8* InChunkRawMemory, const int32 InIndexWithinChunk);

	bool IsSet() const;
	bool operator==(const FMassRawEntityInChunkData& Other) const;

	uint8* const ChunkRawMemory = nullptr;
	const int32 IndexWithinChunk = INDEX_NONE;
};

//-----------------------------------------------------------------------------
// FMassEntityInChunkDataHandle
//-----------------------------------------------------------------------------
/**
 * This is an extension of FMassRawEntityInChunkData that provides additional safety features.
 * It can be used to detect that the underlying data has changed. 
 */
struct FMassEntityInChunkDataHandle : FMassRawEntityInChunkData 
{
	FMassEntityInChunkDataHandle() = default;
	FMassEntityInChunkDataHandle(FMassEntityInChunkDataHandle&&) = default;
	FMassEntityInChunkDataHandle(const FMassEntityInChunkDataHandle&) = default;
	FMassEntityInChunkDataHandle& operator=(const FMassEntityInChunkDataHandle&);
	FMassEntityInChunkDataHandle& operator=(FMassEntityInChunkDataHandle&&);
	FMassEntityInChunkDataHandle(uint8* InChunkRawMemory, const int32 InIndexWithinChunk, const int32 InChunkIndex, const int32 InChunkSerialNumber);
	FMassEntityInChunkDataHandle(const FMassEntityInChunkDataHandle& Source, const int32 InIndexWithinChunk);

	MASSENTITY_API bool IsValid(const FMassArchetypeData* ArchetypeData) const;
	MASSENTITY_API bool IsValid(const FMassArchetypeHandle& ArchetypeHandle) const;
	bool operator==(const FMassEntityInChunkDataHandle& Other) const;

	int32 GetAbsoluteIndex(const int32 NumEntitiesPerChunk) const;

	const int32 ChunkIndex = INDEX_NONE;
	const int32 ChunkSerialNumber = INDEX_NONE;
};

//-----------------------------------------------------------------------------
// FMassQueryRequirementIndicesMapping
//-----------------------------------------------------------------------------
using FMassFragmentIndicesMapping = TArray<int32, TInlineAllocator<16>>;

struct FMassQueryRequirementIndicesMapping
{
	FMassQueryRequirementIndicesMapping() = default;

	FMassFragmentIndicesMapping EntityFragments;
	FMassFragmentIndicesMapping ChunkFragments;
	FMassFragmentIndicesMapping ConstSharedFragments;
	FMassFragmentIndicesMapping SharedFragments;
	inline bool IsEmpty() const
	{
		return EntityFragments.Num() == 0 || ChunkFragments.Num() == 0;
	}
};

struct FMassChunkProcessingQueueParams
{
	FMassExecutionContext* SourceContext = nullptr;
	FMassEntityQuery* SourceQuery = nullptr;
	FMassArchetypeData* Archetype = nullptr;
	const FMassArchetypeEntityCollection::FArchetypeEntityRange EntityRange;
	const FMassExecuteFunction* ExecuteFunction = nullptr;
	const FMassChunkConditionFunction* ChunkCondition = nullptr;
	const FMassQueryRequirementIndicesMapping* RequirementMapping = nullptr;
	bool bParallelExecution = false;
};

//-----------------------------------------------------------------------------
// INLINES
//-----------------------------------------------------------------------------
inline bool FMassArchetypeVersionedHandle::IsValid() const
{
	return ArchetypeHandle.IsValid();
}

inline bool FMassArchetypeVersionedHandle::operator==(const FMassArchetypeVersionedHandle& Other) const 
{ 
	return ArchetypeHandle == Other.ArchetypeHandle && HandleVersion == Other.HandleVersion; 
}

inline bool FMassArchetypeVersionedHandle::operator!=(const FMassArchetypeVersionedHandle& Other) const 
{ 
	return !(*this == Other);
}

inline FMassArchetypeVersionedHandle::operator FMassArchetypeHandle() const
{
	return ArchetypeHandle;
}

inline bool FMassArchetypeHandle::IsValid() const
{
	return DataPtr.IsValid();
}

inline bool FMassArchetypeHandle::operator==(const FMassArchetypeHandle& Other) const
{
	return DataPtr == Other.DataPtr;
}

inline bool FMassArchetypeHandle::operator!=(const FMassArchetypeHandle& Other) const
{
	return DataPtr != Other.DataPtr;
}

inline void FMassArchetypeHandle::Reset()
{
	DataPtr.Reset();
}

inline FMassArchetypeHandle::FMassArchetypeHandle(const TSharedPtr<FMassArchetypeData>& InDataPtr)
	: DataPtr(InDataPtr)
{
	
}

inline FMassArchetypeEntityCollection::FConstEntityRangeArrayView FMassArchetypeEntityCollection::GetRanges() const
{
	return Ranges;
}

inline FMassArchetypeHandle FMassArchetypeEntityCollection::GetArchetype() const
{
	return Archetype;
}

inline bool FMassArchetypeEntityCollection::IsEmpty() const
{
	return Ranges.Num() == 0 && Archetype.IsValid() == false;
}

inline bool FMassArchetypeEntityCollection::IsUpToDate() const
{
	return IsEmpty() || Archetype.IsUpToDate();
}

inline bool FMassArchetypeEntityCollection::IsSet() const
{
	return !IsEmpty();
}

inline bool FMassArchetypeEntityCollection::IsSameArchetype(const FMassArchetypeEntityCollection& Other) const
{
	return Archetype == Other.Archetype;
}

template<typename T>
requires std::is_same_v<typename TDecay<T>::Type, FMassArchetypeEntityCollection>
void FMassArchetypeEntityCollection::Append(T&& Other)
{
	const bool bWasEmpty = Ranges.IsEmpty();
	checkf(IsSameArchetype(Other), TEXT("Unable to merge two entity collections representing different archetypes"));

	Ranges.Append(Forward<T>(Other).Ranges);

	if (bWasEmpty == false)
	{
		Ranges.Sort();
		checkfSlow(DoesContainOverlappingRanges(Ranges) == false
			, TEXT("Entity collection ranges overlap as a result of %hs"), __FUNCTION__);
	}
}

inline FMassRawEntityInChunkData::FMassRawEntityInChunkData(uint8* InChunkRawMemory, const int32 InIndexWithinChunk)
	: ChunkRawMemory(InChunkRawMemory), IndexWithinChunk(InIndexWithinChunk)
{}

inline bool FMassRawEntityInChunkData::IsSet() const
{
	return ChunkRawMemory != nullptr && IndexWithinChunk != INDEX_NONE;
}

inline bool FMassRawEntityInChunkData::operator==(const FMassRawEntityInChunkData & Other) const
{
	return ChunkRawMemory == Other.ChunkRawMemory && IndexWithinChunk == Other.IndexWithinChunk;
}

inline FMassEntityInChunkDataHandle& FMassEntityInChunkDataHandle::operator=(const FMassEntityInChunkDataHandle& Other)
{
	new (this) FMassEntityInChunkDataHandle(Other);
	return *this;
}

inline FMassEntityInChunkDataHandle& FMassEntityInChunkDataHandle::operator=(FMassEntityInChunkDataHandle&& Other)
{
	new (this) FMassEntityInChunkDataHandle(Other);
	return *this;
}

inline FMassEntityInChunkDataHandle::FMassEntityInChunkDataHandle(uint8* InChunkRawMemory, const int32 InIndexWithinChunk, const int32 InChunkIndex, const int32 InChunkSerialNumber)
		: FMassRawEntityInChunkData(InChunkRawMemory, InIndexWithinChunk)
		, ChunkIndex(InChunkIndex), ChunkSerialNumber(InChunkSerialNumber)
{
}

inline FMassEntityInChunkDataHandle::FMassEntityInChunkDataHandle(const FMassEntityInChunkDataHandle& Source, const int32 InIndexWithinChunk)
	: FMassRawEntityInChunkData(Source.ChunkRawMemory, InIndexWithinChunk)
		, ChunkIndex(Source.ChunkIndex), ChunkSerialNumber(Source.ChunkSerialNumber)
{
}

inline bool FMassEntityInChunkDataHandle::operator==(const FMassEntityInChunkDataHandle& Other) const
{
	return FMassRawEntityInChunkData::operator==(Other)
		&& ChunkIndex == Other.ChunkIndex
		&& ChunkSerialNumber == Other.ChunkSerialNumber;
}

inline int32 FMassEntityInChunkDataHandle::GetAbsoluteIndex(const int32 NumEntitiesPerChunk) const
{
	return ChunkIndex * NumEntitiesPerChunk + IndexWithinChunk;
}

#undef UE_API 