// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructUtilsTypes.h"
#include "TypeBitSetBuilder.h"
#include "Mass/EntityElementTypes.h"
#include "MassEntityConcepts.h"

/** Specialization of TTypeBitSetTraits for FMassTag */
template <>
struct TTypeBitSetTraits<FMassTag> final
{
	/** Compile-time check to ensure that TTestedType is a valid Mass Tag */
	template<typename TTestedType>
	static constexpr bool IsValidType = UE::Mass::CTag<TTestedType>;

	/** Indicates that a base type is required for inheritance checks */
	static constexpr bool RequiresBaseType = true;
};

/** Specialization of TTypeBitSetTraits for FMassFragment */
template <>
struct TTypeBitSetTraits<FMassFragment> final
{
	/** Compile-time check to ensure that TTestedType is a valid Mass Fragment */
	template<typename TTestedType>
	static constexpr bool IsValidType = UE::Mass::CFragment<TTestedType>;

	/** Indicates that a base type is required for inheritance checks */
	static constexpr bool RequiresBaseType = true;
};

namespace UE::Mass
{
	namespace Private
	{
		/**
		 * Template class for registering and managing bitsets for Mass types (e.g., Fragments and Tags).
		 * Provides functionality to create builders for constructing bitsets.
		 * The type hosts a FStructTracker instance that stores information on all the types used to 
		 * build bitsets via it, and only those types - as opposed to TStructTypeBitSet, which is using
		 * the same FStructTracker throughout the engine's instance lifetime.
		 *
		 * @param T the base Mass type (e.g., FMassFragment or FMassTag).
		 * @param TUStructType the Unreal Engine struct type, default is UScriptStruct.
		 */
		template<typename T, typename TUStructType = UScriptStruct>
		struct TBitTypeRegistry
		{
			/** Alias for the bitset builder specific to the type T */
			using FBitSetBuilder = TTypeBitSetBuilder<T, TUStructType, /*bTestInheritanceAtRuntime=*/TTypeBitSetTraits<T>::RequiresBaseType>;

			/** The type representing the runtime-used bitset. Const by design. */
			using FBitSet = typename FBitSetBuilder::FConstBitSet;

			/**
			 * Factory type for creating and initializing bitsets.
			 * Inherits from FBitSetBuilder to provide building capabilities.
			 * Use this type when you want to build a bitset from scratch (i.e. when you 
			 * don't have a FBitSet instance you want to modify).
			 */
			struct FBitSetFactory : FBitSetBuilder
			{
				/**
				 * Constructor that initializes the builder with a new bitset instance.
				 * @param InStructTracker the struct tracker to use for managing types.
				 */
				FBitSetFactory(FStructTracker& InStructTracker)
					: FBitSetBuilder(InStructTracker, BitSetInstance)
				{
				}

				/** The bitset instance being built */
				FBitSet BitSetInstance; 
			};

			/**
			 * Constructor that initializes the struct tracker.
			 * Uses a lambda to retrieve the UStruct representing the base type T.
			 */
			TBitTypeRegistry()
				: StructTracker([](){ return StructUtils::GetAsUStruct<T>(); })
			{
			}

			/**
			 * Creates a bitset builder for an existing bitset.
			 * @param BitSet the bitset instance to modify
			 * @return A new FBitSetBuilder instance
			 */
			[[nodiscard]] inline FBitSetBuilder MakeBuilder(FBitSet& BitSet) const
			{
				return FBitSetBuilder(StructTracker, BitSet);
			}

			/**
			 * Creates a factory for building new bitsets, essentially a FBitSetBuilder-BitSet combo.
			 * @return A new FBitSetFactory instance.
			 */
			[[nodiscard]] inline FBitSetFactory MakeBuilder() const
			{
				return FBitSetFactory(StructTracker);
			}

			/**
			 * Registers a type with the struct tracker.
			 * @param Type the UScriptStruct representing the type to register.
			 * @return The index assigned to the registered type.
			 */
			inline int32 RegisterType(TNotNull<const UScriptStruct*> Type)
			{
				return StructTracker.Register(Type);
			}

			/**
			 * Template method to register a type with the struct tracker.
			 * @param TType the type to register.
			 * @return The index assigned to the registered type.
			 */
			template<typename TType>
			inline int32 RegisterType()
			{
				return RegisterType(TType::StaticStrict());
			}

			/**
			 * Specialized struct tracker for bitsets, disabling serialization.
			 * Inherits from FStructTracker.
			 */
			struct FBitSetStructTracker : public FStructTracker
			{
				/**
				 * Constructor that initializes the base type and optional type validation function.
				 * @param InBaseType the base UStruct type.
				 * @param InTypeValidation Optional function for type validation.
				 */
				FBitSetStructTracker(const UStruct* InBaseType, const FTypeValidation& InTypeValidation = FTypeValidation())
					: FStructTracker(InBaseType, InTypeValidation)
				{
					// Disable serialization, temporarily, until serialization is implemented for the new bitset type
					bIsSerializable = false;
				}
			};

			/** Struct tracker for managing types */
			mutable FBitSetStructTracker StructTracker; 
		};
	} // namespace Private

	/** Alias for the fragment bit registry */
	using FFragmentBitRegistry = Private::TBitTypeRegistry<FMassFragment>;
	/** Alias for the fragment bitset builder */
	using FFragmentBitSetBuilder = FFragmentBitRegistry::FBitSetBuilder;
	/** Alias for a read-only fragment bitset builder */
	using FFragmentBitSetReader = const FFragmentBitSetBuilder;
	/** Alias for the fragment bitset factory */
	using FFragmentBitSetFactory = FFragmentBitRegistry::FBitSetFactory;

	/** Alias for the tag bit registry */
	using FTagBitRegistry = Private::TBitTypeRegistry<FMassTag>;
	/** Alias for the tag bitset builder */
	using FTagBitSetBuilder = FTagBitRegistry::FBitSetBuilder;
	/** Alias for a read-only tag bitset builder */
	using FTagBitSetReader = const FTagBitSetBuilder;
	/** Alias for the tag bitset factory */
	using FTagBitSetFactory = FTagBitRegistry::FBitSetFactory;

	/** Explicit template instantiation declarations for the registries */
	template<>
	MASSENTITY_API FFragmentBitRegistry::TBitTypeRegistry();

	template<>
	MASSENTITY_API FTagBitRegistry::TBitTypeRegistry();

} // namespace UE::Mass

using FMassFragmentBitSet_WIP = UE::Mass::FFragmentBitRegistry::FBitSet;
using FMassTagBitSet_WIP = UE::Mass::FTagBitRegistry::FBitSet;
