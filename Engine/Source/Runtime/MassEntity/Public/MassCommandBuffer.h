// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityManager.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "MassEntityUtils.h"
#include "MassDebuggerBreakpoints.h"
#include "MassCommands.h"
// required for UE::Mass::Deprecation
#include <type_traits>
#include "Concepts/ConvertibleTo.h"

struct FMassEntityManager;

//@TODO: Consider debug information in case there is an assert when replaying the command buffer
// (e.g., which system added the command, or even file/line number in development builds for the specific call via a macro)

#define COMMAND_PUSHING_CHECK() \
checkf(IsFlushing() == false, TEXT("Trying to push commands is not supported while the given buffer is being flushed")); \
checkf(OwnerThreadId == FPlatformTLS::GetCurrentThreadId(), TEXT("Commands can be pushed only in the same thread where the command buffer was created."))

// The following is a temporary construct to help users update their code after FMassBatchedCommand::Execute deprecation
// @todo remove by 5.9
namespace UE::Mass::Deprecation
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	template <typename TCommand>
	constexpr bool IsExecuteOverridden()
	{
	    using TResolved = decltype(&TCommand::Execute);
	    using TBase = decltype(&FMassBatchedCommand::Execute);
	    return !std::is_same_v<TResolved, TBase>;
	}

	template <typename TCommand>
	requires std::is_base_of_v<FMassBatchedCommand, TCommand>
	struct TOverridesExecute : std::bool_constant<IsExecuteOverridden<TCommand>()>
	{
	};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
} // namespace UE::Mass::Deprecation

#define ASSERT_EXECUTE_DEPRECATION(CommandType) \
		static_assert(!UE::Mass::Deprecation::TOverridesExecute<CommandType>::value, "Mass Commands: CONST Execute function is deprecated in 5.7 and will be removed by 5.9. Use Run instead.");

namespace UE::Mass::Debug
{

	template<typename TCommand, typename... Args>
	concept HasCheckBreakpoints =
		requires(Args&&... args) {
			{ TCommand::CheckBreakpoints(Forward<Args>(args)...) }
			-> CConvertibleTo<bool>;
	};

	template<typename TCommand, typename... Args>
	concept HasCheckBreakpointsWithEntity =
		requires(const FMassEntityHandle Entity, Args&&... args) {
			{ TCommand::CheckBreakpoints(Entity, Forward<Args>(args)...) }
			-> CConvertibleTo<bool>;
	};

	template<typename TCommand, typename... TArgs>
	void CallCheckBreakpoints(TArgs&&... InArgs)
	{
#if WITH_MASSENTITY_DEBUG
		if constexpr (HasCheckBreakpoints<TCommand, TArgs...>)
		{
			if (TCommand::CheckBreakpoints(Forward<TArgs>(InArgs)...))
			{
				UE::Mass::Debug::FBreakpoint::DebugBreak();
			}
		}
#endif //WITH_MASSENTITY_DEBUG
	}

	template<typename TCommand, typename... TArgs >
	void CallCheckBreakpointsByInstance(TArgs&&... InArgs)
	{
#if WITH_MASSENTITY_DEBUG
		if constexpr (HasCheckBreakpointsWithEntity<TCommand, TArgs...>)
		{
			if (TCommand::CheckBreakpointsByInstance(Forward<TArgs>(InArgs)...))
			{
				UE::Mass::Debug::FBreakpoint::DebugBreak();
			}
		}
#endif //WITH_MASSENTITY_DEBUG
	}
} // namespace UE::Mass::Debug


struct FMassCommandBuffer
{
	MASSENTITY_API FMassCommandBuffer();
	MASSENTITY_API ~FMassCommandBuffer();

	/** Adds a new entry to a given TCommand batch command instance */
	template< template<typename... TArgs> typename TCommand, typename... TArgs >
	void PushCommand(const FMassEntityHandle Entity, TArgs&&... InArgs)
	{
		COMMAND_PUSHING_CHECK();
		UE::Mass::Debug::CallCheckBreakpointsByInstance<TCommand<TArgs...>>(Entity, Forward<TArgs>(InArgs)...);

		LLM_SCOPE_BYNAME(TEXT("Mass/PushCommand"));
		TCommand<TArgs...>& Instance = CreateOrAddCommand<TCommand<TArgs...>>();
		Instance.Add(Entity, Forward<TArgs>(InArgs)...);
		++ActiveCommandsCounter;
	}

	template<typename TCommand, typename... TArgs>
	void PushCommand(TArgs&&... InArgs)
	{
		COMMAND_PUSHING_CHECK();
		UE::Mass::Debug::CallCheckBreakpointsByInstance<TCommand>(Forward<TArgs>(InArgs)...);

		LLM_SCOPE_BYNAME(TEXT("Mass/PushCommand"));
		TCommand& Instance = CreateOrAddCommand<TCommand>();
		Instance.Add(Forward<TArgs>(InArgs)...);
		++ActiveCommandsCounter;
	}

	/** Adds a new entry to a given TCommand batch command instance */
	template< typename TCommand>
	void PushCommand(const FMassEntityHandle Entity)
	{
		COMMAND_PUSHING_CHECK();
		UE::Mass::Debug::CallCheckBreakpoints<TCommand>(Entity);

		LLM_SCOPE_BYNAME(TEXT("Mass/PushCommand"));
		CreateOrAddCommand<TCommand>().Add(Entity);
		++ActiveCommandsCounter;
	}

	template< typename TCommand>
	void PushCommand(TConstArrayView<FMassEntityHandle> Entities)
	{
		COMMAND_PUSHING_CHECK();
		UE::Mass::Debug::CallCheckBreakpoints<TCommand>(Entities);

		LLM_SCOPE_BYNAME(TEXT("Mass/PushCommand"));
		CreateOrAddCommand<TCommand>().Add(Entities);
		++ActiveCommandsCounter;
	}

	/**
	 * Ordinary PushCommand calls try to reuse existing command instances (as stored in CommandInstances)
	 * based on their type.
	 * This command lets callers add a manually configured command instance, that 
	 * might not be distinguishable from other commands based solely on its type.
	 * For example, when you implement a command type that has member properties that 
	 * control how the given command works or what exactly it does.
	 */
	FMassBatchedCommand& PushUniqueCommand(TUniquePtr<FMassBatchedCommand>&& CommandInstance)
	{
		COMMAND_PUSHING_CHECK();

		LLM_SCOPE_BYNAME(TEXT("Mass/PushCommand"));
		UE::TScopeLock Lock(AppendingCommandsCS);
		UE_MT_SCOPED_WRITE_ACCESS(PendingBatchCommandsDetector);
		AppendedCommandInstances.Add(MoveTemp(CommandInstance));

		++ActiveCommandsCounter;

		return *AppendedCommandInstances.Last().Get();
	}

	template<typename TCommand, typename... ArgsType>
	requires TIsDerivedFrom<UE::Mass::Clean<TCommand>, FMassBatchedCommand>::Value
		&& (UE::Mass::Command::TCommandTraits<TCommand>::RequiresUniqueHandling == true)
	TCommand& PushUniqueCommand(ArgsType&&... Args)
	{
		TUniquePtr<TCommand> CommandInstance = MakeUnique<TCommand>(Forward<ArgsType>(Args)...);
		return static_cast<TCommand&>(PushUniqueCommand(MoveTemp(CommandInstance)));
	}

	template<typename T>
	void AddFragment(FMassEntityHandle Entity)
	{
		MASS_STATIC_CHECK_FRAGMENT(T);
		PushCommand<FMassCommandAddElements<T>>(Entity);
	}

	template<typename T>
	UE_DEPRECATED(5.8, "Use AddFragment instead")
	void AddFragment_RuntimeCheck(FMassEntityHandle Entity)
	{
		checkf(UE::Mass::IsA<FMassFragment>(T::StaticStruct()), TEXT(MASS_INVALID_FRAGMENT_MSG_F), *T::StaticStruct()->GetName());
		PushCommand<FMassCommandAddElements<T>>(Entity);
	}

	template<typename T>
	void RemoveFragment(FMassEntityHandle Entity)
	{
		MASS_STATIC_CHECK_FRAGMENT(T);
		PushCommand<FMassCommandRemoveElements<T>>(Entity);
	}

	template<typename T>
	UE_DEPRECATED(5.8, "Use RemoveFragment instead")
	void RemoveFragment_RuntimeCheck(FMassEntityHandle Entity)
	{
		checkf(UE::Mass::IsA<FMassFragment>(T::StaticStruct()), TEXT(MASS_INVALID_FRAGMENT_MSG_F), *T::StaticStruct()->GetName());
		PushCommand<FMassCommandRemoveElements<T>>(Entity);
	}

	/** the convenience function equivalent to calling PushCommand<FMassCommandAddTag<T>>(Entity) */
	template<typename T>
	void AddTag(FMassEntityHandle Entity)
	{
		static_assert(UE::Mass::CTag<T>, "Given struct type is not a valid tag type.");
		PushCommand<FMassCommandAddElements<T>>(Entity);
	}

	template<typename T>
	void AddTag(TConstArrayView<FMassEntityHandle> Entities)
	{
		static_assert(UE::Mass::CTag<T>, "Given struct type is not a valid tag type.");
		PushCommand<FMassCommandAddElements<T>>(Entities);
	}

	template<typename T>
	UE_DEPRECATED(5.8, "Use AddTag instead")
	void AddTag_RuntimeCheck(FMassEntityHandle Entity)
	{
		checkf(UE::Mass::IsA<FMassTag>(T::StaticStruct()), TEXT("Given struct type is not a valid tag type."));
		PushCommand<FMassCommandAddElements<T>>(Entity);
	}

	/** the convenience function equivalent to calling PushCommand<FMassCommandRemoveTag<T>>(Entity) */
	template<typename T>
	void RemoveTag(FMassEntityHandle Entity)
	{
		static_assert(UE::Mass::CTag<T>, "Given struct type is not a valid tag type.");
		PushCommand<FMassCommandRemoveElements<T>>(Entity);
	}

	template<typename T>
	void RemoveTag(TConstArrayView<FMassEntityHandle> Entities)
	{
		static_assert(UE::Mass::CTag<T>, "Given struct type is not a valid tag type.");
		PushCommand<FMassCommandRemoveElements<T>>(Entities);
	}

	template<typename T>
	UE_DEPRECATED(5.8, "Use RemoveTag instead")
	void RemoveTag_RuntimeCheck(FMassEntityHandle Entity)
	{
		checkf(UE::Mass::IsA<FMassTag>(T::StaticStruct()), TEXT("Given struct type is not a valid tag type."));
		PushCommand<FMassCommandRemoveElements<T>>(Entity);
	}

	/** the convenience function equivalent to calling PushCommand<FMassCommandSwapTags<TOld, TNew>>(Entity)  */
	template<typename TOld, typename TNew>
	void SwapTags(FMassEntityHandle Entity)
	{
		static_assert(UE::Mass::CTag<TOld>, "Given struct type is not a valid tag type.");
		static_assert(UE::Mass::CTag<TNew>, "Given struct type is not a valid tag type.");
		PushCommand<FMassCommandSwapTagsInternal<EMassCommandCheckTime::CompileTimeCheck, TOld, TNew>>(Entity);
	}

	template<typename TOld, typename TNew>
	UE_DEPRECATED(5.8, "Use SwapTags instead")
	void SwapTags_RuntimeCheck(FMassEntityHandle Entity)
	{
		checkf(UE::Mass::IsA<FMassTag>(TOld::StaticStruct()), TEXT("Given struct type is not a valid tag type."));
		checkf(UE::Mass::IsA<FMassTag>(TNew::StaticStruct()), TEXT("Given struct type is not a valid tag type."));
		PushCommand<FMassCommandSwapTagsInternal<EMassCommandCheckTime::RuntimeCheck, TOld, TNew>>(Entity);
	}

	/** Adds any mix of non-shared element types (fragments, tags, sparse) to an entity. Batches via PushCommand. */
	template<typename... TTypes>
	void AddElements(FMassEntityHandle Entity)
	{
		PushCommand<FMassCommandAddElements<TTypes...>>(Entity);
	}

	/** Adds any mix of non-shared element types (fragments, tags, sparse) to entities. Batches via PushCommand. */
	template<typename... TTypes>
	void AddElements(TConstArrayView<FMassEntityHandle> Entities)
	{
		PushCommand<FMassCommandAddElements<TTypes...>>(Entities);
	}

	/** Adds non-shared element types AND shared fragment values to an entity. Per-hash grouping for batching. */
	template<typename... TTypes>
	void AddElementsWithSharedFragments(FMassEntityHandle Entity, FMassArchetypeSharedFragmentValues&& SharedValues)
	{
		PushCommand<FMassCommandAddElementsWithSharedFragments<TTypes...>>(Entity, MoveTemp(SharedValues));
	}

	/** Adds element types with per-entity values AND shared fragment values to an entity in a single move. Per-hash grouping.
	 *  TElements accepts FMassFragment types (per-entity values required) and FMassTag types including sparse
	 *  (tags participate in composition but are excluded from the value payload due to zero-size storage). */
	template<typename... TElements>
	void AddFragmentInstancesWithSharedFragments(FMassEntityHandle Entity, FMassArchetypeSharedFragmentValues&& SharedValues, TElements&&... InElements)
	{
		PushCommand<FMassCommandAddFragmentInstancesWithSharedFragments<FMassArchetypeSharedFragmentValues, UE::Mass::Clean<TElements>...>>(Entity, MoveTemp(SharedValues), Forward<TElements>(InElements)...);
	}

	/** Adds any mix of non-shared element types to an entity. Types provided at runtime. Uses PushUniqueCommand. */
	void AddElements(FMassEntityHandle Entity, TConstArrayView<const UScriptStruct*> ElementTypes)
	{
		PushUniqueCommand<FMassCommandAddElementList>(ElementTypes).Add(Entity);
	}

	/** Adds any mix of non-shared element types to entities. Types provided at runtime. Uses PushUniqueCommand. */
	void AddElements(TConstArrayView<FMassEntityHandle> Entities, TConstArrayView<const UScriptStruct*> ElementTypes)
	{
		PushUniqueCommand<FMassCommandAddElementList>(ElementTypes).Add(Entities);
	}

	/** Removes any mix of element types (fragments, tags, shared, const shared, sparse) from an entity. Batches via PushCommand. */
	template<typename... TTypes>
	void RemoveElements(FMassEntityHandle Entity)
	{
		PushCommand<FMassCommandRemoveElements<TTypes...>>(Entity);
	}

	/** Removes any mix of element types (fragments, tags, shared, const shared, sparse) from entities. Batches via PushCommand. */
	template<typename... TTypes>
	void RemoveElements(TConstArrayView<FMassEntityHandle> Entities)
	{
		PushCommand<FMassCommandRemoveElements<TTypes...>>(Entities);
	}

	/** Removes any mix of element types from an entity. Types provided at runtime. Uses PushUniqueCommand. */
	void RemoveElements(FMassEntityHandle Entity, TConstArrayView<const UScriptStruct*> ElementTypes)
	{
		PushUniqueCommand<FMassCommandRemoveElementList>(ElementTypes).Add(Entity);
	}

	/** Removes any mix of element types from entities. Types provided at runtime. Uses PushUniqueCommand. */
	void RemoveElements(TConstArrayView<FMassEntityHandle> Entities, TConstArrayView<const UScriptStruct*> ElementTypes)
	{
		PushUniqueCommand<FMassCommandRemoveElementList>(ElementTypes).Add(Entities);
	}

	void DestroyEntity(FMassEntityHandle Entity)
	{
		PushCommand<FMassCommandDestroyEntities>(Entity);
	}

	void DestroyEntities(TConstArrayView<FMassEntityHandle> InEntitiesToDestroy)
	{
		PushCommand<FMassCommandDestroyEntities>(InEntitiesToDestroy);
	}

	void DestroyEntities(TArray<FMassEntityHandle>&& InEntitiesToDestroy)
	{
		PushCommand<FMassCommandDestroyEntities>(MoveTemp(InEntitiesToDestroy));
	}

	MASSENTITY_API SIZE_T GetAllocatedSize() const;

	/** 
	 * Appends the commands from the passed buffer into this one
	 * @param InOutOther the source buffer to copy the commands from. Note that after the call the InOutOther will be 
	 *	emptied due to the function using Move semantics
	 */
	MASSENTITY_API void MoveAppend(FMassCommandBuffer& InOutOther);

	bool HasPendingCommands() const 
	{
		return ActiveCommandsCounter > 0;
	}
	bool IsFlushing() const
	{
		return bIsFlushing;
	}

	/**
	 * Removes any pending command instances
	 * This could be required for CommandBuffers that are queued to
	 * flush their commands on the game thread but the EntityManager is no longer available.
	 * In such scenario we need to cancel commands to avoid an ensure for unprocessed commands
	 * when the buffer gets destroyed.
	 */
	void CancelCommands()
	{
		CleanUp();
	}

	bool IsInOwnerThread() const
	{
		return OwnerThreadId == FPlatformTLS::GetCurrentThreadId();
	}

	/**
	 * Updates the OwnerThreadId which indicates that the given command buffer instance is being
	 * used in a different thread now. Use this with extreme caution, it's not a tool to be used
	 * every time we get "Commands can be pushed only in the same thread where the command buffer was created."
	 * error. It's meant to be used when there's a possibility the code owning the buffer has been
	 * moved to another thread (like in ParallelFor).
	 */
	void ForceUpdateCurrentThreadID();

private:
	friend FMassEntityManager;

	template<typename T>
	T& CreateOrAddCommand()
	{
		ASSERT_EXECUTE_DEPRECATION(T);

		static_assert(!UE::Mass::Command::TCommandTraits<T>::RequiresUniqueHandling, "This command type needs to be added via PushUniqueCommand");
		const int32 Index = FMassBatchedCommand::GetCommandIndex<T>();

		if (CommandInstances.IsValidIndex(Index) == false)
		{
			CommandInstances.AddZeroed(Index - CommandInstances.Num() + 1);
		}
		else if (CommandInstances[Index])
		{
			return static_cast<T&>(*CommandInstances[Index].Get());
		}

		CommandInstances[Index] = MakeUnique<T>();
		return static_cast<T&>(*CommandInstances[Index].Get());
	}

	/** 
	 * Executes all accumulated commands. 
	 * @return whether any commands have actually been executed
	 */
	bool Flush(FMassEntityManager& EntityManager);
	MASSENTITY_API void CleanUp();

	FTransactionallySafeCriticalSection AppendingCommandsCS;

	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(PendingBatchCommandsDetector);
	/** 
	 * Commands created for this specific command buffer. All commands in the array are unique (by type) and reusable 
	 * with subsequent PushCommand calls
	 */
	TArray<TUniquePtr<FMassBatchedCommand>> CommandInstances;
	/** 
	 * Commands appended to this command buffer (via FMassCommandBuffer::MoveAppend). These commands are just naive list
	 * of commands, potentially containing duplicates with multiple MoveAppend calls. Once appended these commands are 
	 * not being reused and consumed, destructively, during flushing
	 */
	TArray<TUniquePtr<FMassBatchedCommand>> AppendedCommandInstances;

	int32 ActiveCommandsCounter = 0;

	/** Indicates that this specific MassCommandBuffer is currently flushing its contents */
	bool bIsFlushing = false;

	/** 
	 * Identifies the thread where given FMassCommandBuffer instance was created. Adding commands from other
	 * threads is not supported and we use this value to check that.
	 * Note that it could be const since we set it in the constructor, but we need to recache on server forking.
	 */
	uint32 OwnerThreadId;
};

#undef COMMAND_PUSHING_CHECK
