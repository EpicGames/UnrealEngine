// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/CsvProfilerConfig.h"
#include "AutoRTFM.h"
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "MassEntityTypes.h"
#include "MassEntityUtils.h"
#include "MassEntityManager.h"
#include "MassDebuggerBreakpoints.h"
#include "MassCommands.generated.h"

/**
 * =====================================================================================================================
 * Mass Commands Quick Reference
 * =====================================================================================================================
 *
 * ADDING ELEMENTS (composition-only, no initial values):
 * -------------------------------------------------------
 * | Need                       | Command                                                    | Batching    | BP  |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Add fragments/tags/sparse  | FMassCommandAddElements<T...>                              | Cross-chunk | Yes |
 * | (compile-time types)       | Convenience: AddElements<T...>()                           |             |     |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Add fragments/tags/sparse  | FMassCommandAddElementList                                 | Per-inst.   | No  |
 * | (runtime types)            | Convenience: AddElements(E, T)                             |             |     |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Add single type (runtime)  | FMassCommandAddElement                                     | Per-inst.   | No  |
 *
 * ADDING FRAGMENTS WITH VALUES (+ optional tags/sparse):
 * --------------------------------------------------------
 * | Need                       | Command                                                    | Batching    | BP  |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Add fragments with values  | FMassCommandAddFragmentInstances<T...>                     | Cross-chunk | Yes |
 * | + optional tags/sparse     | T... accepts FMassFragment, FMassTag, and sparse subtypes  |             |     |
 * |                            | Tags/sparse tags: composition only. Sparse frags: in-place |             |     |
 *
 * ADDING ELEMENTS WITH SHARED FRAGMENT VALUES (+ optional tags/sparse):
 * ----------------------------------------------------------------------
 * | Need                       | Command                                                    | Batching    | BP  |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Add fragments/tags/sparse  | FMassCommandAddElementsWithSharedFragments<T...>           | Per-hash    | No  |
 * | + shared fragment values   | Convenience: AddElementsWithSharedFragments<T...>(E, V)    | group       |     |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Add fragments with values  | FMassCommandAddFragmentInstancesWithSharedFragments        | Per-hash    | Yes |
 * | + shared values + tags     | <TShared, T...> T... accepts fragments, tags, sparse       | group       |     |
 *
 * REMOVING ELEMENTS:
 * -------------------
 * | Need                       | Command                                                    | Batching    | BP  |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Remove any element types   | FMassCommandRemoveElements<T...>                           | Cross-chunk | Yes |
 * | (compile-time types)       | Convenience: RemoveElements<T...>()                        |             |     |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Remove any element types   | FMassCommandRemoveElementList                              | Per-inst.   | No  |
 * | (runtime types)            | Convenience: RemoveElements(E, T)                          |             |     |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Remove single (runtime)    | FMassCommandRemoveElement                                  | Per-inst.   | No  |
 *
 * CHANGING COMPOSITION:
 * ----------------------
 * | Need                       | Command                                                    | Batching    | BP  |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Swap one tag for another   | FMassCommandSwapTags<TOld, TNew>                           | Cross-chunk | No  |
 * | (compile-time types)       | Convenience: SwapTags<TOld, TNew>()                        |             |     |
 *
 * CREATING ENTITIES (+ optional tags/sparse):
 * ----------------------------------------------
 * | Need                       | Command                                                    | Batching    | BP  |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Create entity with values  | FMassCommandBuildEntity<T...>                              | Cross-chunk | Yes |
 * | + optional tags/sparse     | T... accepts FMassFragment, FMassTag, and sparse subtypes  |             |     |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Create with shared values  | FMassCommandBuildEntityWithSharedFragments<TShared, T...>  | Per-hash    | Yes |
 * | + optional tags/sparse     | T... accepts fragments, tags, sparse                       | group       |     |
 *
 * DESTROYING ENTITIES:
 * ---------------------
 * | Need                       | Command                                                    | Batching    | BP  |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Destroy entities           | FMassCommandDestroyEntities                                | Cross-chunk | Yes |
 * |                            | Convenience: DestroyEntity/Entities()                      |             |     |
 *
 * DEFERRED (LAMBDA):
 * -------------------
 * | Need                       | Command                                                    | Batching    | BP  |
 * |----------------------------|------------------------------------------------------------|-------------|-----|
 * | Arbitrary logic at flush   | FMassDeferredCreateCommand                                 | Cross-chunk | No  |
 * | (type depends on op)       | FMassDeferredAddCommand                                    |             |     |
 * |                            | FMassDeferredRemoveCommand                                 |             |     |
 * |                            | FMassDeferredChangeCompositionCommand                      |             |     |
 * |                            | FMassDeferredSetCommand                                    |             |     |
 * |                            | FMassDeferredDestroyCommand                                |             |     |
 *
 * NOTES:
 * - Cross-chunk: PushCommand reuses one instance across all chunks; one Run() for all entities.
 * - Per-inst.: PushUniqueCommand creates a new instance per call. Entities batch via Add().
 * - Per-hash group: entities grouped by shared fragment value hash; one move per group.
 * - Adding shared/const shared fragments requires values; use the WithSharedFragments variant.
 * - Removing shared/const shared fragments is supported by RemoveElements and RemoveElement.
 * - Sparse elements are handled in-place (no archetype move). Non-sparse types: single entity move.
 * - BP = Breakpoint support (Mass debugger breakpoints triggered at PushCommand time).
 * - Deferred commands store a TFunction<void(FMassEntityManager&)>; no type safety, full flexibility.
 * ========================================================================================================
 */

/**
 * Enum used by MassBatchCommands to declare their "type". This data is later used to group commands so that command
 * effects are applied in a controllable fashion
 * Important: if changed make sure to update FMassCommandBuffer::Flush.CommandTypeOrder as well
 */
UENUM()
enum class EMassCommandOperationType : uint8
{
	None,				// default value. Commands marked this way will always be executed last. Programmers are encouraged to instead use one of the meaningful values below.
	Create,				// signifies commands performing entity creation
	Add,				// signifies commands adding fragments or tags to entities
	Remove,				// signifies commands removing fragments or tags from entities
	ChangeComposition,	// signifies commands both adding and removing fragments and/or tags from entities
	Set,				// signifies commands setting values to pre-existing fragments. The fragments might be added if missing,
						// depending on specific command, so this group will always be executed after the Add group
	Destroy,			// signifies commands removing entities
	MAX
};

enum class EMassCommandCheckTime : bool
{
	RuntimeCheck = true,
	CompileTimeCheck = false
};

#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
#	define DEBUG_NAME(Name) , FName(TEXT(Name))
#	define DEBUG_NAME_PARAM(Name) , const FName InDebugName = TEXT(Name)
#	define FORWARD_DEBUG_NAME_PARAM , InDebugName
#else
#	define DEBUG_NAME(Name)
#	define DEBUG_NAME_PARAM(Name)
#	define FORWARD_DEBUG_NAME_PARAM
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

namespace UE::Mass::Utils
{
	template<typename BitSetType, EMassCommandCheckTime CheckTime, typename... TTypes>
	BitSetType ConstructBitSet()
	{
		if constexpr (CheckTime == EMassCommandCheckTime::RuntimeCheck)
		{
			return BitSetType({ TTypes::StaticStruct()... });
		}
		else
		{
			BitSetType Result;
			UE::Mass::TMultiTypeList<TTypes...>::PopulateBitSet(Result);
			return Result;
		}
	}

	template<EMassCommandCheckTime CheckTime, typename... TTypes>
	FMassFragmentBitSet ConstructFragmentBitSet()
	{
		return ConstructBitSet<FMassFragmentBitSet, CheckTime, TTypes...>();
	}

	template<EMassCommandCheckTime CheckTime, typename... TTypes>
	FMassTagBitSet ConstructTagBitSet()
	{
		return ConstructBitSet<FMassTagBitSet, CheckTime, TTypes...>();
	}
} // namespace UE::Mass::Utils

namespace UE::Mass::Command
{
	template<typename T>
	struct TCommandTraits final
	{
		enum
		{
			RequiresUniqueHandling = false
		};
	};
} // namespace UE::Mass::Command

struct FMassBatchedCommand
{
	FMassBatchedCommand() = default;
	explicit FMassBatchedCommand(EMassCommandOperationType OperationType)
		: OperationType(OperationType)
	{}
#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
	FMassBatchedCommand(EMassCommandOperationType OperationType, FName DebugName)
		: OperationType(OperationType)
		, DebugName(DebugName)
	{}
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
	virtual ~FMassBatchedCommand()
	{
		Reset();
	}

	UE_DEPRECATED(5.7, "Mass Commands: CONST Execute function is deprecated in 5.7 and will be removed by 5.9. Use Run instead.")
	virtual void Execute(FMassEntityManager& EntityManager) const
	{
		ensureMsgf(false, TEXT("FMassBatchedCommand::Execute is DEPRECATED, override Run function instead."));
	}

	virtual void Run(FMassEntityManager& EntityManager)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Execute(EntityManager);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	virtual void Reset()
	{
		bHasWork = false;
	}

	bool HasWork() const
	{
		return bHasWork;
	}
	EMassCommandOperationType GetOperationType() const
	{
		return OperationType;
	}
	
	template<typename T>
	UE_AUTORTFM_ALWAYS_OPEN
	static uint32 GetCommandIndex()
	{
		static const uint32 ThisTypesStaticIndex = CommandsCounter++;
		return ThisTypesStaticIndex;
	}

	virtual SIZE_T GetAllocatedSize() const = 0;

#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
	virtual int32 GetNumOperationsStat() const = 0;
	FName GetFName() const
	{
		return DebugName;
	}
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

protected:
	bool bHasWork = false;
	EMassCommandOperationType OperationType = EMassCommandOperationType::None;

#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
	FName DebugName;
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

private:
	static MASSENTITY_API std::atomic<uint32> CommandsCounter;
};

struct FMassBatchedEntityCommand : public FMassBatchedCommand
{
	using Super = FMassBatchedCommand;

	FMassBatchedEntityCommand() = default;
	explicit FMassBatchedEntityCommand(EMassCommandOperationType OperationType DEBUG_NAME_PARAM("BatchedEntityCommand"))
		: Super(OperationType FORWARD_DEBUG_NAME_PARAM)
	{}

	void Add(FMassEntityHandle Entity)
	{
		UE_MT_SCOPED_WRITE_ACCESS(EntitiesAccessDetector);
		TargetEntities.Add(Entity);
		bHasWork = true;
	}

	void Add(TConstArrayView<FMassEntityHandle> Entities)
	{
		UE_MT_SCOPED_WRITE_ACCESS(EntitiesAccessDetector);
		TargetEntities.Append(Entities.GetData(), Entities.Num());
		bHasWork = true;
	}

	void Add(TArray<FMassEntityHandle>&& Entities)
	{
		UE_MT_SCOPED_WRITE_ACCESS(EntitiesAccessDetector);
		TargetEntities.Append(Forward<TArray<FMassEntityHandle>>(Entities));
		bHasWork = true;
	}

protected:
	virtual SIZE_T GetAllocatedSize() const
	{
		return TargetEntities.GetAllocatedSize();
	}

	virtual void Reset() override
	{
		TargetEntities.Reset();
		Super::Reset();
	}

#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
	virtual int32 GetNumOperationsStat() const override
	{
		return TargetEntities.Num();
	}
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(EntitiesAccessDetector); 
	TArray<FMassEntityHandle> TargetEntities;
};

//-----------------------------------------------------------------------------
// Entity destruction
//-----------------------------------------------------------------------------
/**
 * Destroys entities. Uses PushCommand for cross-chunk batching.
 *
 * Example:
 *   // Single entity:
 *   Context.Defer().DestroyEntity(EntityHandle);
 *   // All entities in chunk:
 *   Context.Defer().DestroyEntities(Context.GetEntities());
 *
 * Breakpoint support: Yes — CheckDestroyEntityBreakpoints called at PushCommand time.
 */
struct FMassCommandDestroyEntities : public FMassBatchedEntityCommand
{
	using Super = FMassBatchedEntityCommand;

	FMassCommandDestroyEntities()
		: Super(EMassCommandOperationType::Destroy DEBUG_NAME("DestroyEntities"))
	{
	}

#if WITH_MASSENTITY_DEBUG
	template<typename T>
	static bool CheckBreakpoints(T Entity)
	{
		return UE::Mass::Debug::FBreakpoint::CheckDestroyEntityBreakpoints(Entity);
	}
#endif // WITH_MASSENTITY_DEBUG

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandDestroyEntities_Execute);

		TArray<FMassArchetypeEntityCollection> EntityCollectionsToDestroy;
		UE::Mass::Utils::CreateEntityCollections(EntityManager, TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates, EntityCollectionsToDestroy);
		EntityManager.BatchDestroyEntityChunks(EntityCollectionsToDestroy);
	}
};

//-----------------------------------------------------------------------------
// Mixed element types
//-----------------------------------------------------------------------------
/**
 * Adds a single element type (fragment, tag, or sparse) to entities at runtime.
 * Requires PushUniqueCommand — each call creates a separate command instance (no cross-call batching).
 * Shared/const shared fragments are rejected at runtime (they require values).
 *
 * When to use: when the element type is only known at runtime and you need to add a single type.
 * Prefer FMassCommandAddElements<T...> when types are known at compile time (better batching).
 *
 * Example:
 *   // Single entity:
 *   FMassCommandAddElement& Cmd = Context.Defer().PushUniqueCommand<FMassCommandAddElement>(FMyFragment::StaticStruct());
 *   Cmd.Add(EntityHandle);
 *   // All entities in chunk:
 *   Cmd.Add(Context.GetEntities());
 *
 * Supported types: FMassFragment, FMassTag, FMassSparseFragment, FMassSparseTag.
 * Rejected types: FMassSharedFragment, FMassConstSharedFragment, FMassChunkFragment.
 * Breakpoint support: No (runtime types — use FMassCommandAddElements<T...> for breakpoints).
 */
struct FMassCommandAddElement : public FMassBatchedEntityCommand
{
	using Super = FMassBatchedEntityCommand;

	FMassCommandAddElement(const TNotNull<const UScriptStruct*> InElementType)
		: Super(EMassCommandOperationType::Add DEBUG_NAME("AddElement"))
		, ElementType(InElementType)
	{}

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandAddElement_Execute);
		EntityManager.BatchAddElementToEntities(TargetEntities, ElementType);
	}

	TNotNull<const UScriptStruct*> ElementType;
};

template<>
struct UE::Mass::Command::TCommandTraits<FMassCommandAddElement> final
{
	enum
	{
		RequiresUniqueHandling = true
	};
};

/**
 * Removes a single element type from entities at runtime.
 * Requires PushUniqueCommand — each call creates a separate command instance (no cross-call batching).
 * Supports all element types including shared and const shared fragments.
 *
 * When to use: when the element type is only known at runtime and you need to remove a single type.
 * Prefer FMassCommandRemoveElements<T...> when types are known at compile time (better batching, single entity move).
 *
 * Example:
 *   // Single entity:
 *   FMassCommandRemoveElement& Cmd = Context.Defer().PushUniqueCommand<FMassCommandRemoveElement>(FMyFragment::StaticStruct());
 *   Cmd.Add(EntityHandle);
 *   // All entities in chunk:
 *   Cmd.Add(Context.GetEntities());
 *
 * Supported types: all FMassElement subtypes (fragments, tags, shared, const shared, sparse).
 * Breakpoint support: No (runtime types — use FMassCommandRemoveElements<T...> for breakpoints).
 */
struct FMassCommandRemoveElement : public FMassBatchedEntityCommand
{
	using Super = FMassBatchedEntityCommand;

	FMassCommandRemoveElement(const TNotNull<const UScriptStruct*> InElementType)
		: Super(EMassCommandOperationType::Remove DEBUG_NAME("RemoveElement"))
		, ElementType(InElementType)
	{}

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandRemoveElement_Execute);
		EntityManager.BatchRemoveElementFromEntities(TargetEntities, ElementType);
	}

	TNotNull<const UScriptStruct*> ElementType;
};

template<>
struct UE::Mass::Command::TCommandTraits<FMassCommandRemoveElement> final
{
	enum
	{
		RequiresUniqueHandling = true
	};
};

/**
 * Removes any mix of element types from entities at runtime.
 * Element types are provided as UScriptStruct* at construction time.
 * Requires PushUniqueCommand — each call creates a separate command instance
 * (no cross-call batching, but entities batch within a single instance via Add()).
 * Performs a single entity move for all non-sparse types + in-place sparse removal.
 *
 * When to use: when element types are only known at runtime and you need to remove multiple types at once.
 * Prefer FMassCommandRemoveElements<T...> when types are known at compile time (cross-chunk batching).
 *
 * Example:
 *   const UScriptStruct* Types[] = { FMyFragment::StaticStruct(), FMyTag::StaticStruct() };
 *   // Single entity:
 *   FMassCommandRemoveElementList& Cmd = Context.Defer().PushUniqueCommand<FMassCommandRemoveElementList>(Types);
 *   Cmd.Add(EntityHandle);
 *   // All entities in chunk via convenience:
 *   Context.Defer().RemoveElements(Context.GetEntities(), Types);
 *
 * Supported types: all FMassElement subtypes (fragments, tags, shared, const shared, sparse).
 * Breakpoint support: No (runtime types).
 */
struct FMassCommandRemoveElementList : public FMassBatchedEntityCommand
{
	using Super = FMassBatchedEntityCommand;

	FMassCommandRemoveElementList(TConstArrayView<const UScriptStruct*> InElementTypes)
		: Super(EMassCommandOperationType::Remove DEBUG_NAME("RemoveElementList"))
	{
		for (const UScriptStruct* ElementType : InElementTypes)
		{
			if (UE::Mass::IsSparse(ElementType))
			{
				SparseTypes.Add(ElementType);
			}
			else
			{
				NonSparseTypes.Add(ElementType);
			}
		}
	}

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandRemoveElementList_Execute);

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(EntityManager, TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates, EntityCollections);

		// Sparse element modification is always in-place (no entity move)
		// Process first before changing the composition of the entities that will invalidate the entity collections
		for (const UScriptStruct* SparseType : SparseTypes)
		{
			EntityManager.BatchRemoveSparseElementFromEntities(EntityCollections, SparseType);
		}

		if (!NonSparseTypes.IsEmpty())
		{
			EntityManager.BatchChangeCompositionForEntities(EntityCollections, FMassElementBitSet(), NonSparseTypes);
		}
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return Super::GetAllocatedSize() + NonSparseTypes.GetAllocatedSize() + SparseTypes.GetAllocatedSize();
	}

#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
	virtual int32 GetNumOperationsStat() const override
	{
		return TargetEntities.Num();
	}
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

	FMassElementBitSet NonSparseTypes;
	TArray<const UScriptStruct*> SparseTypes;
};

template<>
struct UE::Mass::Command::TCommandTraits<FMassCommandRemoveElementList> final
{
	enum
	{
		RequiresUniqueHandling = true
	};
};

/**
 * Removes any mix of element types from entities. This is the preferred command for element removal.
 * Types are template parameters — uses PushCommand for efficient cross-chunk batching
 * (one Run() for all entities across all chunks that push into this command).
 * Performs a single entity move for all non-sparse types.
 *
 * When to use: whenever you need to remove fragments, tags, shared fragments, and/or sparse elements from
 * entities and the types are known at compile time. Replaces FMassCommandRemoveFragments and FMassCommandRemoveTags.
 *
 * Example:
 *   // All entities in chunk:
 *   Context.Defer().PushCommand<FMassCommandRemoveElements<FMyFragment, FMyTag, FMyConstShared>>(Context.GetEntities());
 *   // Single entity:
 *   Context.Defer().PushCommand<FMassCommandRemoveElements<FMyFragment, FMyTag>>(EntityHandle);
 *   // Via convenience:
 *   Context.Defer().RemoveElements<FMyFragment, FMyTag, FMyConstShared>(Context.GetEntities());
 *
 * Supported types: all FMassElement subtypes (fragments, tags, shared, const shared, sparse).
 * Breakpoint support: Yes — CheckFragmentRemoveBreakpoints called per entity at PushCommand time.
 * Entity moves: 1 move for all non-sparse types combined. Sparse removal is in-place (no move).
 */
template<typename... TTypes>
struct FMassCommandRemoveElements : public FMassBatchedEntityCommand
{
	using Super = FMassBatchedEntityCommand;

	FMassCommandRemoveElements()
		: Super(EMassCommandOperationType::Remove DEBUG_NAME("RemoveElements"))
	{
		(CategorizeElementType<TTypes>(), ...);
	}

#if WITH_MASSENTITY_DEBUG
	// Function pointer forces the non-template overload of CheckFragmentRemoveBreakpoints.
	// Without it, MSVC deduces TFragments=UScriptStruct* from StaticStruct() return type,
	// matching the variadic template overload which then calls UScriptStruct*::StaticStruct() and fails.
	static bool CheckBreakpoints(const FMassEntityHandle Entity)
	{
		using FCheckFn = bool(*)(const FMassEntityHandle&, const UScriptStruct*);
		const FCheckFn Check = &UE::Mass::Debug::FBreakpoint::CheckFragmentRemoveBreakpoints;
		return (Check(Entity, UE::Mass::Clean<TTypes>::StaticStruct()) || ...);
	}
	static bool CheckBreakpoints(TConstArrayView<FMassEntityHandle> Entities)
	{
		using FCheckFn = bool(*)(const FMassEntityHandle&, const UScriptStruct*);
		const FCheckFn Check = &UE::Mass::Debug::FBreakpoint::CheckFragmentRemoveBreakpoints;
		for (const FMassEntityHandle& Entity : Entities)
		{
			if ((Check(Entity, UE::Mass::Clean<TTypes>::StaticStruct()) || ...))
			{
				return true;
			}
		}
		return false;
	}
#endif // WITH_MASSENTITY_DEBUG

private:
	template<typename T>
	void CategorizeElementType()
	{
		using FClean = UE::Mass::Clean<T>;
		if constexpr (UE::Mass::CSparse<FClean>)
		{
			SparseTypes.Add(FClean::StaticStruct());
		}
		else
		{
			NonSparseTypes.Add(FClean::StaticStruct());
		}
	}

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandRemoveElements_Execute);

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(EntityManager, TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates, EntityCollections);

		// Sparse element modification is always in-place (no entity move)
		// Process first before changing the composition of the entities that will invalidate the entity collections
		for (const UScriptStruct* SparseType : SparseTypes)
		{
			EntityManager.BatchRemoveSparseElementFromEntities(EntityCollections, SparseType);
		}

		// Single entity move for all non-sparse element types
		if (!NonSparseTypes.IsEmpty())
		{
			EntityManager.BatchChangeCompositionForEntities(EntityCollections, FMassElementBitSet(), NonSparseTypes);
		}
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return Super::GetAllocatedSize() + NonSparseTypes.GetAllocatedSize() + SparseTypes.GetAllocatedSize();
	}

	FMassElementBitSet NonSparseTypes;
	TArray<const UScriptStruct*> SparseTypes;
};

/**
 * Adds any mix of non-shared element types to entities. This is the preferred command for element addition.
 * Types are template parameters — uses PushCommand for efficient cross-chunk batching (one Run() for all entities
 * across all chunks that push into this command).
 * Performs a single entity move for all non-sparse types.
 * Shared/const shared fragments cannot be added — they require values (use AddSharedFragmentToEntity).
 *
 * When to use: whenever you need to add fragments, tags, and/or sparse elements to entities and the types
 * are known at compile time. Replaces FMassCommandAddFragments and FMassCommandAddTags.
 *
 * Example:
 *   // All entities in chunk:
 *   Context.Defer().PushCommand<FMassCommandAddElements<FMyFragment, FMyTag>>(Context.GetEntities());
 *   // Single entity:
 *   Context.Defer().PushCommand<FMassCommandAddElements<FMyFragment, FMyTag>>(EntityHandle);
 *   // Via convenience:
 *   Context.Defer().AddElements<FMyFragment, FMyTag>(Context.GetEntities());
 *
 * Supported types: FMassFragment, FMassTag, FMassSparseFragment, FMassSparseTag.
 * Rejected types: FMassSharedFragment, FMassConstSharedFragment (compile-time static_assert), FMassChunkFragment (runtime).
 * Breakpoint support: Yes — CheckFragmentAddBreakpoints called per entity at PushCommand time.
 * Entity moves: 1 move for all non-sparse types combined. Sparse addition is in-place (no move).
 */
template<typename... TTypes>
struct FMassCommandAddElements : public FMassBatchedEntityCommand
{
	static_assert((!TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassSharedFragment>::Value && ...), "Shared fragments cannot be added via FMassCommandAddElements — they require a value.");
	static_assert((!TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassConstSharedFragment>::Value && ...), "Const shared fragments cannot be added via FMassCommandAddElements — they require a value.");

	using Super = FMassBatchedEntityCommand;

	FMassCommandAddElements()
		: Super(EMassCommandOperationType::Add DEBUG_NAME("AddElements"))
	{
		(CategorizeElementType<TTypes>(), ...);
	}

#if WITH_MASSENTITY_DEBUG
	// Function pointer forces the non-template overload of CheckFragmentAddBreakpoints.
	// Without it, MSVC deduces TFragments=UScriptStruct* from StaticStruct() return type,
	// matching the variadic template overload which then calls UScriptStruct*::StaticStruct() and fails.
	static bool CheckBreakpoints(const FMassEntityHandle Entity)
	{
		using FCheckFn = bool(*)(const FMassEntityHandle&, const UScriptStruct*);
		const FCheckFn Check = &UE::Mass::Debug::FBreakpoint::CheckFragmentAddBreakpoints;
		return (Check(Entity, UE::Mass::Clean<TTypes>::StaticStruct()) || ...);
	}
	static bool CheckBreakpoints(TConstArrayView<FMassEntityHandle> Entities)
	{
		using FCheckFn = bool(*)(const FMassEntityHandle&, const UScriptStruct*);
		const FCheckFn Check = &UE::Mass::Debug::FBreakpoint::CheckFragmentAddBreakpoints;
		for (const FMassEntityHandle& Entity : Entities)
		{
			if ((Check(Entity, UE::Mass::Clean<TTypes>::StaticStruct()) || ...))
			{
				return true;
			}
		}
		return false;
	}
#endif // WITH_MASSENTITY_DEBUG

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandAddElements_Execute);

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(EntityManager, TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates, EntityCollections);

		// Sparse element modification is always in-place (no entity move)
		// Process first before changing the composition of the entities that will invalidate the entity collections
		for (const UScriptStruct* SparseType : SparseTypes)
		{
			EntityManager.BatchAddSparseElementToEntities(EntityCollections, SparseType);
		}

		if (!NonSparseTypes.IsEmpty())
		{
			EntityManager.BatchChangeCompositionForEntities(EntityCollections, NonSparseTypes, FMassElementBitSet());
		}
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return Super::GetAllocatedSize() + NonSparseTypes.GetAllocatedSize() + SparseTypes.GetAllocatedSize();
	}

private:
	template<typename T>
	void CategorizeElementType()
	{
		using FClean = UE::Mass::Clean<T>;
		if constexpr (UE::Mass::CSparse<FClean>)
		{
			SparseTypes.Add(FClean::StaticStruct());
		}
		else
		{
			NonSparseTypes.Add(FClean::StaticStruct());
		}
	}

	FMassElementBitSet NonSparseTypes;
	TArray<const UScriptStruct*> SparseTypes;
};

/**
 * Adds any mix of non-shared element types to entities at runtime.
 * Element types are provided as UScriptStruct* at construction time.
 * Requires PushUniqueCommand — each call creates a separate command instance
 * (no cross-call batching, but entities batch within a single instance via Add()).
 * Performs a single entity move for all non-sparse types + in-place sparse addition.
 *
 * When to use: when element types are only known at runtime and you need to add multiple types at once.
 * Prefer FMassCommandAddElements<T...> when types are known at compile time (cross-chunk batching).
 *
 * Example:
 *   const UScriptStruct* Types[] = { FMyFragment::StaticStruct(), FMyTag::StaticStruct() };
 *   // Single entity:
 *   FMassCommandAddElementList& Cmd = Context.Defer().PushUniqueCommand<FMassCommandAddElementList>(Types);
 *   Cmd.Add(EntityHandle);
 *   // All entities in chunk via convenience:
 *   Context.Defer().AddElements(Context.GetEntities(), Types);
 *
 * Supported types: FMassFragment, FMassTag, FMassSparseFragment, FMassSparseTag.
 * Rejected types: shared/const shared and chunk fragments are rejected at runtime by BatchChangeCompositionForEntities (no compile-time check).
 * Breakpoint support: No (runtime types).
 */
struct FMassCommandAddElementList : public FMassBatchedEntityCommand
{
	using Super = FMassBatchedEntityCommand;

	FMassCommandAddElementList(TConstArrayView<const UScriptStruct*> InElementTypes)
		: Super(EMassCommandOperationType::Add DEBUG_NAME("AddElementList"))
	{
		for (const UScriptStruct* ElementType : InElementTypes)
		{
			if (UE::Mass::IsSparse(ElementType))
			{
				SparseTypes.Add(ElementType);
			}
			else
			{
				NonSparseTypes.Add(ElementType);
			}
		}
	}

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandAddElementList_Execute);

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(EntityManager, TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates, EntityCollections);

		// Sparse element modification is always in-place (no entity move)
		// Process first before changing the composition of the entities that will invalidate the entity collections
		for (const UScriptStruct* SparseType : SparseTypes)
		{
			EntityManager.BatchAddSparseElementToEntities(EntityCollections, SparseType);
		}

		if (!NonSparseTypes.IsEmpty())
		{
			EntityManager.BatchChangeCompositionForEntities(EntityCollections, NonSparseTypes, FMassElementBitSet());
		}
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return Super::GetAllocatedSize() + NonSparseTypes.GetAllocatedSize() + SparseTypes.GetAllocatedSize();
	}

#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
	virtual int32 GetNumOperationsStat() const override
	{
		return TargetEntities.Num();
	}
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

	FMassElementBitSet NonSparseTypes;
	TArray<const UScriptStruct*> SparseTypes;
};

template<>
struct UE::Mass::Command::TCommandTraits<FMassCommandAddElementList> final
{
	enum
	{
		RequiresUniqueHandling = true
	};
};

/**
 * Adds any mix of non-shared element types AND shared fragment values to existing entities.
 * Types are template parameters for non-shared elements.
 * Shared fragment values are provided at runtime.
 * Entities are grouped by shared fragment value hash for efficient per-group archetype changes.
 * Performs a single entity move per group (non-sparse composition + shared values combined).
 *
 * When to use: when you need to add fragments/tags AND shared fragment values to existing entities
 * in a single operation. Replaces separate AddElements + BatchAddSharedFragmentsForEntities calls.
 *
 * Example:
 *   FMassArchetypeSharedFragmentValues SharedValues;
 *   SharedValues.Add(EntityManager->GetOrCreateConstSharedFragment(FMySharedFragment(42)));
 *   Context.Defer().PushCommand<FMassCommandAddElementsWithSharedFragments<FMyFragment, FMyTag>>(
 *       EntityHandle, MoveTemp(SharedValues));
 *
 * Supported types (TTypes): FMassFragment + FMassTag (including sparse).
 * Rejected types (TTypes): FMassSharedFragment, FMassConstSharedFragment (use shared values param instead), FMassChunkFragment.
 * Breakpoint support: No (composition-only command).
 * Entity moves: 1 move per shared-value group. Sparse addition is in-place.
 */
template<typename... TTypes>
struct FMassCommandAddElementsWithSharedFragments : public FMassBatchedCommand
{
	static_assert((!TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassSharedFragment>::Value && ...),
		"Shared fragments cannot be passed as TTypes -- provide them via FMassArchetypeSharedFragmentValues instead.");
	static_assert((!TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassConstSharedFragment>::Value && ...),
		"Const shared fragments cannot be passed as TTypes -- provide them via FMassArchetypeSharedFragmentValues instead.");
	static_assert((!TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassChunkFragment>::Value && ...),
		"Chunk fragments cannot be passed to FMassCommandAddElementsWithSharedFragments.");

	using Super = FMassBatchedCommand;

	FMassCommandAddElementsWithSharedFragments()
		: Super(EMassCommandOperationType::Add DEBUG_NAME("AddElementsWithSharedFragments"))
	{
		(CategorizeElementType<TTypes>(), ...);
	}

	/** @note Callers must not Add() the same entity with different shared values — it would end up in two hash groups,
	 *  and the second group's archetype move would overwrite the first group's without merging fragment values. */
	void Add(FMassEntityHandle Entity, FMassArchetypeSharedFragmentValues&& InSharedFragments)
	{
		InSharedFragments.Sort();
		const uint32 Hash = GetTypeHash(InSharedFragments);
		FPerSharedFragmentsHashData& Instance = Data.FindOrAdd(Hash, MoveTemp(InSharedFragments));
		Instance.TargetEntities.Add(Entity);
		bHasWork = true;
	}

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandAddElementsWithSharedFragments_Execute);

		// @todo: add inline arrays support in CreateEntityCollections
		TArray<FMassArchetypeEntityCollection> EntityCollections;

		for (auto& It : Data)
		{
			UE::Mass::Utils::CreateEntityCollections(EntityManager, It.Value.TargetEntities,
				FMassArchetypeEntityCollection::FoldDuplicates, EntityCollections);

			// Sparse element modification is always in-place (no entity move)
			// Process first before changing the composition of the entities that will invalidate the entity collections
			for (const UScriptStruct* SparseType : SparseTypes)
			{
				EntityManager.BatchAddSparseElementToEntities(EntityCollections, SparseType);
			}

			// Combine non-sparse element types + shared fragment bits from values
			FMassElementBitSet AllToAdd = NonSparseTypes;
			AllToAdd += It.Value.SharedFragmentValues.GetBitSet();

			EntityManager.BatchChangeCompositionForEntities(
				EntityCollections, AllToAdd, /*ElementsToRemove*/FMassElementBitSet(), It.Value.SharedFragmentValues);

			EntityCollections.Reset();
		}
	}

	virtual void Reset() override
	{
		Data.Reset();
		Super::Reset();
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		SIZE_T TotalSize = NonSparseTypes.GetAllocatedSize() + SparseTypes.GetAllocatedSize();
		for (const auto& KeyValue : Data)
		{
			TotalSize += KeyValue.Value.TargetEntities.GetAllocatedSize()
				+ KeyValue.Value.SharedFragmentValues.GetAllocatedSize();
		}
		TotalSize += Data.GetAllocatedSize();
		return TotalSize;
	}

#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
	virtual int32 GetNumOperationsStat() const override
	{
		int32 TotalCount = 0;
		for (const auto& KeyValue : Data)
		{
			TotalCount += KeyValue.Value.TargetEntities.Num();
		}
		return TotalCount;
	}
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

private:
	template<typename T>
	void CategorizeElementType()
	{
		using FClean = UE::Mass::Clean<T>;
		if constexpr (UE::Mass::CSparse<FClean>)
		{
			SparseTypes.Add(FClean::StaticStruct());
		}
		else
		{
			NonSparseTypes.Add(FClean::StaticStruct());
		}
	}

	struct FPerSharedFragmentsHashData
	{
		FPerSharedFragmentsHashData(FMassArchetypeSharedFragmentValues&& InSharedFragmentValues)
			: SharedFragmentValues(MoveTemp(InSharedFragmentValues))
		{}

		SIZE_T GetAllocatedSize() const
		{
			return TargetEntities.GetAllocatedSize() + SharedFragmentValues.GetAllocatedSize();
		}

		TArray<FMassEntityHandle> TargetEntities;
		FMassArchetypeSharedFragmentValues SharedFragmentValues;
	};

	FMassElementBitSet NonSparseTypes;
	TArray<const UScriptStruct*> SparseTypes;
	TMap<uint32, FPerSharedFragmentsHashData> Data;
};

/**
 * Adds fragment types with per-entity values AND shared fragment values to existing entities in a single entity move.
 * Extends FMassCommandAddFragmentInstances with shared fragment value support; entities are partitioned by shared
 * fragment value hash at Add() time and dispatched in one BatchAddFragmentInstancesForEntities call per group.
 * When no shared fragment values are needed, prefer FMassCommandAddFragmentInstances — its flat single-batch
 * storage avoids the per-Add hash computation and TMap overhead.
 * Uses PushCommand for cross-chunk batching.
 *
 * Note: TSharedFragmentValues is always FMassArchetypeSharedFragmentValues but is a template param
 * to maintain uniform PushCommand interface (all params in one typename... list).
 *
 * When to use: when you need to add fragments with initial values AND shared fragment values to existing entities in one operation.
 * Replaces separate AddFragmentInstances + deferred lambda AddConstSharedFragmentToEntity calls.
 *
 * Example:
 *   FMassArchetypeSharedFragmentValues SharedValues;
 *   SharedValues.Add(EntityManager->GetOrCreateConstSharedFragment(FMySharedFragment(42)));
 *   Context.Defer().PushCommand<FMassCommandAddFragmentInstancesWithSharedFragments>(
 *       EntityHandle, MoveTemp(SharedValues), FMyFragmentA(1), FMyFragmentB(2.0f));
 *
 * Supported types (TTypes): FMassFragment + FMassTag (including sparse).
 * Tags are included in composition but excluded from value payload (zero-size storage).
 * Shared fragments via FMassArchetypeSharedFragmentValues.
 * Breakpoint support: Yes -- CheckFragmentAddBreakpoints called at PushCommand time.
 * Entity moves: 1 move per shared-value group.
 * Fragment values set in-place after move.
 */
template<typename TSharedFragmentValues, typename... TTypes>
struct FMassCommandAddFragmentInstancesWithSharedFragments : public FMassBatchedCommand
{
	static_assert(((TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassFragment>::Value
		|| TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassTag>::Value) && ...),
		"All types passed to FMassCommandAddFragmentInstancesWithSharedFragments must derive from FMassFragment or FMassTag.");
	static_assert((!TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassSharedFragment>::Value && ...),
		"Shared fragments cannot be passed as TTypes -- provide them via FMassArchetypeSharedFragmentValues instead.");
	static_assert((!TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassConstSharedFragment>::Value && ...),
		"Const shared fragments cannot be passed as TTypes -- provide them via FMassArchetypeSharedFragmentValues instead.");
	static_assert((!TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassChunkFragment>::Value && ...),
		"Chunk fragments cannot be passed to FMassCommandAddFragmentInstancesWithSharedFragments.");

	using Super = FMassBatchedCommand;

	FMassCommandAddFragmentInstancesWithSharedFragments(EMassCommandOperationType OperationType = EMassCommandOperationType::Set DEBUG_NAME_PARAM("AddFragmentInstancesWithSharedFragments"))
		: Super(OperationType FORWARD_DEBUG_NAME_PARAM)
	{
		(CategorizeElementType<TTypes>(), ...);
	}

	/** @note Callers must not Add() the same entity with different shared values — it would end up in two hash groups,
	 *  and the second group's archetype move would overwrite the first group's without merging fragment values. */
	void Add(FMassEntityHandle Entity, FMassArchetypeSharedFragmentValues&& InSharedFragments, TTypes... InFragments)
	{
		InSharedFragments.Sort();

		// Compute hash before adding to the map since evaluation order is not guaranteed
		// and MoveTemp will invalidate InSharedFragments
		const uint32 Hash = GetTypeHash(InSharedFragments);

		FPerSharedFragmentsHashData& Instance = Data.FindOrAdd(Hash, MoveTemp(InSharedFragments));
		Instance.Fragments.Add(InFragments...);
		Instance.TargetEntities.Add(Entity);

		bHasWork = true;
	}

#if WITH_MASSENTITY_DEBUG
	static bool CheckBreakpoints(const FMassEntityHandle Entity, TTypes... InFragments)
	{
		// Function pointer to force non-template overload resolution.
		// Without this, MSVC deduces TFragments=UScriptStruct* from StaticStruct() return type,
		// matching the variadic template overload instead of the (FMassEntityHandle, const UScriptStruct*) overload.
		using FCheckFn = bool(*)(const FMassEntityHandle&, const UScriptStruct*);
		const FCheckFn Check = &UE::Mass::Debug::FBreakpoint::CheckFragmentAddBreakpoints;
		return (Check(Entity, UE::Mass::Clean<TTypes>::StaticStruct()) || ...);
	}
#endif // WITH_MASSENTITY_DEBUG

protected:
	virtual SIZE_T GetAllocatedSize() const override
	{
		SIZE_T TotalSize = FragmentsAffected.GetAllocatedSize() + TagsAffected.GetAllocatedSize() + SparseTypes.GetAllocatedSize();
		for (const auto& KeyValue : Data)
		{
			TotalSize += KeyValue.Value.GetAllocatedSize();
		}
		TotalSize += Data.GetAllocatedSize();
		return TotalSize;
	}

	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandAddFragmentInstancesWithSharedFragments_Execute);

		constexpr int FragmentTypesCount = UE::Mass::TMultiTypeList<TTypes...>::Ordinal + 1;
		TArray<FStructArrayView, TInlineAllocator<8>> GenericMultiArray;
		GenericMultiArray.Reserve(FragmentTypesCount);
		TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
		TArray<FMassArchetypeEntityCollection, TInlineAllocator<16>> NonPayloadCollections;

		for (auto& It : Data)
		{
			// Exclude tag types from the payload — tags have zero-size storage
			It.Value.Fragments.GetAsNonTagGenericMultiArray(GenericMultiArray);

			FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(EntityManager, It.Value.TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates
				, FMassGenericPayloadView(GenericMultiArray), EntityCollections);

			// Sparse element modification is always in-place (no entity move).
			// Process before the composition change that would invalidate entity collections.
			if (!SparseTypes.IsEmpty())
			{
				NonPayloadCollections.Reserve(EntityCollections.Num());
				for (const FMassArchetypeEntityCollectionWithPayload& WithPayload : EntityCollections)
				{
					NonPayloadCollections.Add(WithPayload.GetEntityCollection());
				}
				for (const UScriptStruct* SparseType : SparseTypes)
				{
					EntityManager.BatchAddSparseElementToEntities(NonPayloadCollections, SparseType);
				}
				NonPayloadCollections.Reset();
			}

			EntityManager.BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected, TagsAffected, It.Value.SharedFragmentValues);

			GenericMultiArray.Reset();
			EntityCollections.Reset();
		}
	}

	virtual void Reset() override
	{
		Data.Reset();
		Super::Reset();
	}

#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
	virtual int32 GetNumOperationsStat() const override
	{
		int32 TotalCount = 0;
		for (const auto& KeyValue : Data)
		{
			TotalCount += KeyValue.Value.TargetEntities.Num();
		}
		return TotalCount;
	}
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

	FMassFragmentBitSet FragmentsAffected;
	FMassTagBitSet TagsAffected;
	TArray<const UScriptStruct*> SparseTypes;

	struct FPerSharedFragmentsHashData
	{
		FPerSharedFragmentsHashData(FMassArchetypeSharedFragmentValues&& InSharedFragmentValues)
			: SharedFragmentValues(MoveTemp(InSharedFragmentValues))
		{
		}

		SIZE_T GetAllocatedSize() const
		{
			return TargetEntities.GetAllocatedSize() + Fragments.GetAllocatedSize() + SharedFragmentValues.GetAllocatedSize();
		}

		TArray<FMassEntityHandle> TargetEntities;
		mutable UE::Mass::TMultiArray<TTypes...> Fragments;
		FMassArchetypeSharedFragmentValues SharedFragmentValues;
	};

	TMap<uint32, FPerSharedFragmentsHashData> Data;

private:
	template<typename T>
	void CategorizeElementType()
	{
		using FClean = UE::Mass::Clean<T>;
		if constexpr (UE::Mass::CSparse<FClean>)
		{
			SparseTypes.Add(FClean::StaticStruct());
		}
		else if constexpr (TIsDerivedFrom<FClean, FMassTag>::Value)
		{
			TagsAffected.Add(FClean::StaticStruct());
		}
		else
		{
			FragmentsAffected.Add(FClean::StaticStruct());
		}
	}
};

//-----------------------------------------------------------------------------
// Simple fragment composition change
//-----------------------------------------------------------------------------
template<EMassCommandCheckTime CheckTime, typename... TTypes>
struct UE_DEPRECATED(5.8, "Use FMassCommandAddElements instead — supports any mix of element types in a single operation.") FMassCommandAddFragmentsInternal : public FMassBatchedEntityCommand
{
	static_assert((TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassFragment>::Value && ...), "All types passed to FMassCommandAddFragments must derive from FMassFragment.");
	static_assert((!UE::Mass::CSparse<UE::Mass::Clean<TTypes>> && ...), "Sparse elements cannot be used with FMassCommandAddFragments. Use FMassCommandAddElements instead.");

	using Super = FMassBatchedEntityCommand;
	FMassCommandAddFragmentsInternal()
		: Super(EMassCommandOperationType::Add DEBUG_NAME("AddFragments"))
		, FragmentsAffected(UE::Mass::Utils::ConstructFragmentBitSet<CheckTime, TTypes...>())
	{}

#if WITH_MASSENTITY_DEBUG
	template<typename T>
	static bool CheckBreakpoints(T Entity, TTypes... InFragments)
	{
		return UE::Mass::Debug::FBreakpoint::CheckFragmentAddBreakpoints(Entity, Forward<TTypes>(InFragments)...);
	}
#endif // WITH_MASSENTITY_DEBUG

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandAddFragments_Execute);
		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(EntityManager, TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates, EntityCollections);
		EntityManager.BatchChangeFragmentCompositionForEntities(EntityCollections, FragmentsAffected, FMassFragmentBitSet());
	}
	FMassFragmentBitSet FragmentsAffected;
};

template<typename... TTypes>
using FMassCommandAddFragments = FMassCommandAddElements<TTypes...>;

template<EMassCommandCheckTime CheckTime, typename... TTypes>
struct UE_DEPRECATED(5.8, "Use FMassCommandRemoveElements instead — supports any mix of element types and performs a single entity move.") FMassCommandRemoveFragmentsInternal : public FMassBatchedEntityCommand
{
	static_assert((TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassFragment>::Value && ...), "All types passed to FMassCommandRemoveFragments must derive from FMassFragment.");

	using Super = FMassBatchedEntityCommand;
	FMassCommandRemoveFragmentsInternal()
		: Super(EMassCommandOperationType::Remove DEBUG_NAME("RemoveFragments"))
		, FragmentsAffected(UE::Mass::Utils::ConstructFragmentBitSet<CheckTime, TTypes...>())
	{}

#if WITH_MASSENTITY_DEBUG
	template<typename T>
	static bool CheckBreakpoints(T Entity, TTypes... InFragments)
	{
		return UE::Mass::Debug::FBreakpoint::CheckFragmentRemoveBreakpoints(Entity, Forward<TTypes>(InFragments)...);
	}
#endif // WITH_MASSENTITY_DEBUG

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandRemoveFragments_Execute);
		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(EntityManager, TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates, EntityCollections);
		EntityManager.BatchChangeFragmentCompositionForEntities(EntityCollections, FMassFragmentBitSet(), FragmentsAffected);
	}
	FMassFragmentBitSet FragmentsAffected;
};

template<typename... TTypes>
using FMassCommandRemoveFragments = FMassCommandRemoveElements<TTypes...>;

//-----------------------------------------------------------------------------
// Simple tag composition change
//-----------------------------------------------------------------------------
struct FMassCommandChangeTags : public FMassBatchedEntityCommand
{
	using Super = FMassBatchedEntityCommand;
	FMassCommandChangeTags()
		: Super(EMassCommandOperationType::ChangeComposition DEBUG_NAME("ChangeTags"))
	{}

	FMassCommandChangeTags(EMassCommandOperationType OperationType, FMassTagBitSet TagsToAdd, FMassTagBitSet TagsToRemove DEBUG_NAME_PARAM("ChangeTags"))
		: Super(OperationType FORWARD_DEBUG_NAME_PARAM)
		, TagsToAdd(TagsToAdd)
		, TagsToRemove(TagsToRemove)
	{}

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandChangeTags_Execute);
		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(EntityManager, TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates, EntityCollections);

		EntityManager.BatchChangeTagsForEntities(EntityCollections, TagsToAdd, TagsToRemove);
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return TagsToAdd.GetAllocatedSize() + TagsToRemove.GetAllocatedSize() + Super::GetAllocatedSize();
	}

	FMassTagBitSet TagsToAdd;
	FMassTagBitSet TagsToRemove;
};

template<EMassCommandCheckTime CheckTime, typename... TTypes>
struct UE_DEPRECATED(5.8, "Use FMassCommandAddElements instead — supports any mix of element types in a single operation.") FMassCommandAddTagsInternal : public FMassCommandChangeTags
{
	static_assert((TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassTag>::Value && ...), "All types passed to FMassCommandAddTags must derive from FMassTag.");
	static_assert((!UE::Mass::CSparse<UE::Mass::Clean<TTypes>> && ...), "Sparse elements cannot be used with FMassCommandAddTags. Use FMassCommandAddElements instead.");

	using Super = FMassCommandChangeTags;
	FMassCommandAddTagsInternal()
		: Super(
			EMassCommandOperationType::Add,
			UE::Mass::Utils::ConstructTagBitSet<CheckTime, TTypes...>(),
			{}
			DEBUG_NAME("AddTags"))
	{}
};

template<typename T>
using FMassCommandAddTag = FMassCommandAddElements<T>;

template<typename... TTypes>
using FMassCommandAddTags = FMassCommandAddElements<TTypes...>;

template<EMassCommandCheckTime CheckTime, typename... TTypes>
struct UE_DEPRECATED(5.8, "Use FMassCommandRemoveElements instead — supports any mix of element types and performs a single entity move.") FMassCommandRemoveTagsInternal : public FMassCommandChangeTags
{
	static_assert((TIsDerivedFrom<UE::Mass::Clean<TTypes>, FMassTag>::Value && ...), "All types passed to FMassCommandRemoveTags must derive from FMassTag.");

	using Super = FMassCommandChangeTags;
	FMassCommandRemoveTagsInternal()
		: Super(
			EMassCommandOperationType::Remove,
			{},
			UE::Mass::Utils::ConstructTagBitSet<CheckTime, TTypes...>()
			DEBUG_NAME("RemoveTags"))
	{}

#if WITH_MASSENTITY_DEBUG
	template<typename T>
	static bool CheckBreakpoints(T Entity, TTypes... InFragments)
	{
		return UE::Mass::Debug::FBreakpoint::CheckFragmentRemoveBreakpoints(Entity, Forward<TTypes>(InFragments)...);
	}
#endif // WITH_MASSENTITY_DEBUG
};

template<typename T>
using FMassCommandRemoveTag = FMassCommandRemoveElements<T>;

template<typename... TTypes>
using FMassCommandRemoveTags = FMassCommandRemoveElements<TTypes...>;

/**
 * Atomically swaps one tag for another on entities.
 * Removes TOld and adds TNew in a single composition change.
 * Uses PushCommand for cross-chunk batching.
 *
 * Example:
 *   // All entities in chunk:
 *   Context.Defer().SwapTags<FOldTag, FNewTag>(Context.GetEntities());
 *   // Single entity:
 *   Context.Defer().SwapTags<FOldTag, FNewTag>(EntityHandle);
 *
 * Supported types: FMassTag subtypes only (compile-time static_assert).
 * Breakpoint support: No.
 * Entity moves: 1 move (both tag changes applied together).
 */
template<EMassCommandCheckTime CheckTime, typename TOld, typename TNew>
struct FMassCommandSwapTagsInternal : public FMassCommandChangeTags
{
	static_assert(TIsDerivedFrom<UE::Mass::Clean<TOld>, FMassTag>::Value && TIsDerivedFrom<UE::Mass::Clean<TNew>, FMassTag>::Value, "Both types passed to FMassCommandSwapTags must derive from FMassTag.");

	using Super = FMassCommandChangeTags;
	FMassCommandSwapTagsInternal()
		: Super(
			EMassCommandOperationType::ChangeComposition,
			UE::Mass::Utils::ConstructTagBitSet<CheckTime, TNew>(),
			UE::Mass::Utils::ConstructTagBitSet<CheckTime, TOld>()
			DEBUG_NAME("SwapTags"))
	{}
};

template<typename TOld, typename TNew>
using FMassCommandSwapTags = FMassCommandSwapTagsInternal<EMassCommandCheckTime::CompileTimeCheck, TOld, TNew>;

//-----------------------------------------------------------------------------
// Struct Instances adding and setting
//-----------------------------------------------------------------------------
/**
 * Adds fragments with initial values to entities. Each entity receives typed fragment data.
 * Tags (FMassTag, FMassSparseTag) are also accepted — they participate in the composition change but are excluded from the value payload (tags have zero-size storage).
 * Uses PushCommand for cross-chunk batching. Sparse fragments are supported (handled internally).
 * All entities are accumulated in a single flat batch and dispatched in one BatchAddFragmentInstancesForEntities call.
 *
 * When to use: when you need to add fragments AND set their initial values in one operation, optionally adding tags in the same entity move.
 * For composition-only changes (no values), prefer FMassCommandAddElements<T...>.
 * For adding shared fragment values in the same move, use FMassCommandAddFragmentInstancesWithSharedFragments.
 *
 * Example:
 *   // Add fragments with values + tags in a single entity move:
 *   Context.Defer().PushCommand<FMassCommandAddFragmentInstances>(EntityHandle, FMyFragmentA(42), FMyTag());
 *
 * Supported types: FMassFragment + FMassTag (including sparse).
 * Tag instances are accepted by Add() for uniform syntax but their values are ignored.
 * Breakpoint support: Yes — CheckFragmentAddBreakpoints called at PushCommand time.
 */
template<typename... TOthers>
struct FMassCommandAddFragmentInstances : public FMassBatchedEntityCommand
{
	static_assert(((TIsDerivedFrom<UE::Mass::Clean<TOthers>, FMassFragment>::Value
		|| TIsDerivedFrom<UE::Mass::Clean<TOthers>, FMassTag>::Value) && ...),
		"All types passed to FMassCommandAddFragmentInstances must derive from FMassFragment or FMassTag.");
	static_assert((!TIsDerivedFrom<UE::Mass::Clean<TOthers>, FMassSharedFragment>::Value && ...),
		"Shared fragments cannot be passed to FMassCommandAddFragmentInstances -- use FMassCommandAddFragmentInstancesWithSharedFragments instead.");
	static_assert((!TIsDerivedFrom<UE::Mass::Clean<TOthers>, FMassConstSharedFragment>::Value && ...),
		"Const shared fragments cannot be passed to FMassCommandAddFragmentInstances -- use FMassCommandAddFragmentInstancesWithSharedFragments instead.");
	static_assert((!TIsDerivedFrom<UE::Mass::Clean<TOthers>, FMassChunkFragment>::Value && ...),
		"Chunk fragments cannot be passed to FMassCommandAddFragmentInstances.");

	// Note: sparse fragments and sparse tags are intentionally allowed in TOthers.
	// CategorizeElementType routes them to SparseTypes for in-place handling at the command level
	// before the composition change. They do not appear in FragmentsAffected or TagsAffected.

	using Super = FMassBatchedEntityCommand;

	FMassCommandAddFragmentInstances(EMassCommandOperationType OperationType = EMassCommandOperationType::Set DEBUG_NAME_PARAM("AddFragmentInstanceList"))
		: Super(OperationType FORWARD_DEBUG_NAME_PARAM)
	{
		(CategorizeElementType<TOthers>(), ...);
	}

	template<typename... TInstances>
	void Add(FMassEntityHandle Entity, TInstances... InFragments)
	{
		Super::Add(Entity);
		Fragments.Add(Forward<TInstances>(InFragments)...);
	}

#if WITH_MASSENTITY_DEBUG
	static bool CheckBreakpoints(const FMassEntityHandle Entity, TOthers... InFragments)
	{
		// Function pointer to force non-template overload resolution.
		// Without this, MSVC deduces TFragments=UScriptStruct* from StaticStruct() return type,
		// matching the variadic template overload instead of the (FMassEntityHandle, const UScriptStruct*) overload.
		using FCheckFn = bool(*)(const FMassEntityHandle&, const UScriptStruct*);
		const FCheckFn Check = &UE::Mass::Debug::FBreakpoint::CheckFragmentAddBreakpoints;
		return (Check(Entity, UE::Mass::Clean<TOthers>::StaticStruct()) || ...);
	}
#endif // WITH_MASSENTITY_DEBUG

protected:
	virtual void Reset() override
	{
		Fragments.Reset();
		Super::Reset();
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return Super::GetAllocatedSize() + Fragments.GetAllocatedSize() + FragmentsAffected.GetAllocatedSize() + TagsAffected.GetAllocatedSize() + SparseTypes.GetAllocatedSize();
	}

	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandAddFragmentInstances_Execute);

		TArray<FStructArrayView, TInlineAllocator<8>> GenericMultiArray;
		GenericMultiArray.Reserve(Fragments.GetNumArrays());
		// Use GetAsNonTagGenericMultiArray to exclude tag types from the payload —
		// tags have zero-size storage and BatchSetFragmentValues would crash on them.
		Fragments.GetAsNonTagGenericMultiArray(GenericMultiArray);

		TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
		FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(EntityManager, TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates
			, FMassGenericPayloadView(GenericMultiArray), EntityCollections);

		// Sparse element modification is always in-place (no entity move).
		// Process before the composition change that would invalidate entity collections.
		if (!SparseTypes.IsEmpty())
		{
			TArray<FMassArchetypeEntityCollection, TInlineAllocator<16>> NonPayloadCollections;
			NonPayloadCollections.Reserve(EntityCollections.Num());
			for (const FMassArchetypeEntityCollectionWithPayload& WithPayload : EntityCollections)
			{
				NonPayloadCollections.Add(WithPayload.GetEntityCollection());
			}
			for (const UScriptStruct* SparseType : SparseTypes)
			{
				EntityManager.BatchAddSparseElementToEntities(NonPayloadCollections, SparseType);
			}
		}

		EntityManager.BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected, TagsAffected);
	}

	mutable UE::Mass::TMultiArray<TOthers...> Fragments;
	FMassFragmentBitSet FragmentsAffected;
	FMassTagBitSet TagsAffected;
	TArray<const UScriptStruct*> SparseTypes;

private:
	template<typename T>
	void CategorizeElementType()
	{
		using FClean = UE::Mass::Clean<T>;
		if constexpr (UE::Mass::CSparse<FClean>)
		{
			SparseTypes.Add(FClean::StaticStruct());
		}
		else if constexpr (TIsDerivedFrom<FClean, FMassTag>::Value)
		{
			TagsAffected.Add(FClean::StaticStruct());
		}
		else
		{
			FragmentsAffected.Add(FClean::StaticStruct());
		}
	}
};

/**
 * Builds reserved entities with fragment values (and optionally tags).
 * Entities must be pre-reserved via EntityManager->ReserveEntity().
 * Uses PushCommand for cross-chunk batching.
 *
 * Example:
 *   FMassEntityHandle Entity = EntityManager->ReserveEntity();
 *   Context.Defer().PushCommand<FMassCommandBuildEntity>(Entity, FMyFragmentA(42), FMyTag());
 *
 * Supported types: FMassFragment + FMassTag (including sparse).
 * Breakpoint support: Yes — CheckCreateEntityBreakpoints called at PushCommand time.
 */
template<typename... TOthers>
struct FMassCommandBuildEntity : public FMassCommandAddFragmentInstances<TOthers...>
{
	using Super = FMassCommandAddFragmentInstances<TOthers...>;

	FMassCommandBuildEntity()
		: Super(EMassCommandOperationType::Create DEBUG_NAME("BuildEntity"))
	{
	}

#if WITH_MASSENTITY_DEBUG
	static bool CheckBreakpoints(TOthers... InFragments)
	{
		return UE::Mass::Debug::FBreakpoint::CheckCreateEntityBreakpoints(Forward<TOthers>(InFragments)...);
	}
#endif // WITH_MASSENTITY_DEBUG

protected:

	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandBuildEntity_Execute);

		TArray<FStructArrayView, TInlineAllocator<8>> GenericMultiArray;
		GenericMultiArray.Reserve(Super::Fragments.GetNumArrays());
		Super::Fragments.GetAsNonTagGenericMultiArray(GenericMultiArray);

		TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
		FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(EntityManager, Super::TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates
			, FMassGenericPayloadView(GenericMultiArray), EntityCollections);

		check(EntityCollections.Num() <= 1);
		if (EntityCollections.Num())
		{
			if (Super::TagsAffected.IsEmpty())
			{
				EntityManager.BatchBuildEntities(EntityCollections[0], Super::FragmentsAffected, FMassArchetypeSharedFragmentValues());
			}
			else
			{
				FMassElementBitSet Composition(Super::FragmentsAffected);
				Composition += FMassElementBitSet(Super::TagsAffected);
				EntityManager.BatchBuildEntities(EntityCollections[0], FMassArchetypeCompositionDescriptor(MoveTemp(Composition)), FMassArchetypeSharedFragmentValues());
			}

			// Sparse elements are in-place — add after entities are built.
			// Rebuild collections from TargetEntities since the original collections are stale after build.
			if (!Super::SparseTypes.IsEmpty())
			{
				TArray<FMassArchetypeEntityCollection> FreshCollections;
				UE::Mass::Utils::CreateEntityCollections(EntityManager, Super::TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates, FreshCollections);
				for (const UScriptStruct* SparseType : Super::SparseTypes)
				{
					EntityManager.BatchAddSparseElementToEntities(FreshCollections, SparseType);
				}
			}
		}
	}
};

/**
 * Builds reserved entities with fragment values AND shared fragment values.
 * Entities with the same shared fragment values are grouped for efficient archetype creation.
 * Inherits data management from FMassCommandAddFragmentInstancesWithSharedFragments; only Run() differs.
 * Uses PushCommand for cross-chunk batching.
 *
 * Note: TSharedFragmentValues is always FMassArchetypeSharedFragmentValues but is a template param
 * to maintain uniform PushCommand interface (all params in one typename... list).
 *
 * Example:
 *   FMassEntityHandle Entity = EntityManager->ReserveEntity();
 *   FMassArchetypeSharedFragmentValues SharedValues;
 *   SharedValues.Add(EntityManager->GetOrCreateConstSharedFragment(FMySharedFragment(42)));
 *   Context.Defer().PushCommand<FMassCommandBuildEntityWithSharedFragments>(
 *       Entity, MoveTemp(SharedValues), FMyFragmentA(1), FMyFragmentB(2.0f));
 *
 * Supported types: FMassFragment + FMassTag (including sparse).
 * Shared fragments via FMassArchetypeSharedFragmentValues.
 * Breakpoint support: Yes — CheckCreateEntityBreakpoints called at PushCommand time.
 */
template<typename TSharedFragmentValues, typename... TOthers>
struct FMassCommandBuildEntityWithSharedFragments : public FMassCommandAddFragmentInstancesWithSharedFragments<TSharedFragmentValues, TOthers...>
{
	using Super = FMassCommandAddFragmentInstancesWithSharedFragments<TSharedFragmentValues, TOthers...>;

	FMassCommandBuildEntityWithSharedFragments()
		: Super(EMassCommandOperationType::Create DEBUG_NAME("BuildEntityWithSharedFragments"))
	{
	}

#if WITH_MASSENTITY_DEBUG
	static bool CheckBreakpoints(TOthers... InFragments)
	{
		// debugger doesn't currently support shared fragment filtering, so just send the others
		return UE::Mass::Debug::FBreakpoint::CheckCreateEntityBreakpoints(Forward<TOthers>(InFragments)...);
	}
#endif // WITH_MASSENTITY_DEBUG

protected:
	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandBuildEntityWithSharedFragments_Execute);

		constexpr int FragmentTypesCount = UE::Mass::TMultiTypeList<TOthers...>::Ordinal + 1;
		TArray<FStructArrayView, TInlineAllocator<8>> GenericMultiArray;
		GenericMultiArray.Reserve(FragmentTypesCount);
		TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
		TArray<FMassArchetypeEntityCollection> FreshCollections;

		for (auto& It : Super::Data)
		{
			It.Value.Fragments.GetAsNonTagGenericMultiArray(GenericMultiArray);

			FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(EntityManager, It.Value.TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates
				, FMassGenericPayloadView(GenericMultiArray), EntityCollections);
			checkf(EntityCollections.Num() <= 1, TEXT("We expect TargetEntities to only contain archetype-less entities, ones that need to be \'built\'"));

			if (EntityCollections.Num())
			{
				if (Super::TagsAffected.IsEmpty())
				{
					EntityManager.BatchBuildEntities(EntityCollections[0], Super::FragmentsAffected, It.Value.SharedFragmentValues);
				}
				else
				{
					// Build an FMassElementBitSet combining fragments + tags + shared type bits from values
					FMassElementBitSet Composition(Super::FragmentsAffected);
					Composition += FMassElementBitSet(Super::TagsAffected);
					Composition += It.Value.SharedFragmentValues.GetBitSet();
					EntityManager.BatchBuildEntities(EntityCollections[0], FMassArchetypeCompositionDescriptor(MoveTemp(Composition)), It.Value.SharedFragmentValues);
				}

				// Sparse elements are in-place — add after entities are built.
				// Rebuild collections from this group's TargetEntities since originals are stale after build.
				if (!Super::SparseTypes.IsEmpty())
				{
					UE::Mass::Utils::CreateEntityCollections(EntityManager, It.Value.TargetEntities, FMassArchetypeEntityCollection::FoldDuplicates, FreshCollections);
					for (const UScriptStruct* SparseType : Super::SparseTypes)
					{
						EntityManager.BatchAddSparseElementToEntities(FreshCollections, SparseType);
					}
					FreshCollections.Reset();
				}
			}

			GenericMultiArray.Reset();
			EntityCollections.Reset();
		}
	}
};

//-----------------------------------------------------------------------------
// Commands that really can't know the types at compile time
//-----------------------------------------------------------------------------
/**
 * Stores a lambda to execute at flush time.
 * Maximum flexibility, no type safety.
 * The OpType template parameter controls execution ordering relative to other commands.
 * Multiple lambdas of the same OpType batch into a single command instance.
 *
 * When to use: as a last resort when no typed command fits the use case, or for complex
 * multi-step operations that can't be expressed as a single composition change.
 * Prefer typed commands (AddElements, RemoveElements, etc.) when possible.
 *
 * Example:
 *   Context.Defer().PushCommand<FMassDeferredSetCommand>([Entity](FMassEntityManager& Manager) {
 *       FMassEntityView View(Manager, Entity);
 *       View.GetFragmentData<FMyFragment>().Value = 42;
 *   });
 *
 * Variants (by execution order):
 *   FMassDeferredCreateCommand, FMassDeferredAddCommand, FMassDeferredRemoveCommand,
 *   FMassDeferredChangeCompositionCommand, FMassDeferredSetCommand, FMassDeferredDestroyCommand.
 *
 * Breakpoint support: No.
 * No type safety. No validation.
 */
template<EMassCommandOperationType OpType>
struct FMassDeferredCommand : public FMassBatchedCommand
{
	using Super = FMassBatchedCommand;
	using FExecFunction = TFunction<void(FMassEntityManager& EntityManager)>;

	FMassDeferredCommand()
		: Super(OpType DEBUG_NAME("BatchedDeferredCommand"))
	{}

	void Add(FExecFunction&& ExecFunction)
	{
		DeferredFunctions.Add(MoveTemp(ExecFunction));
		bHasWork = true;
	}

	void Add(const FExecFunction& ExecFunction)
	{
		DeferredFunctions.Add(ExecFunction);
		bHasWork = true;
	}

protected:
	virtual SIZE_T GetAllocatedSize() const
	{
		return DeferredFunctions.GetAllocatedSize();
	}

	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassDeferredCommand_Execute);

		for (const FExecFunction& ExecFunction : DeferredFunctions)
		{
			ExecFunction(EntityManager);
		}
	}

	virtual void Reset() override
	{
		DeferredFunctions.Reset();
		Super::Reset();
	}

#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
	virtual int32 GetNumOperationsStat() const override
	{
		return DeferredFunctions.Num();
	}
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

	TArray<FExecFunction> DeferredFunctions;
};

using FMassDeferredCreateCommand = FMassDeferredCommand<EMassCommandOperationType::Create>;
using FMassDeferredAddCommand = FMassDeferredCommand<EMassCommandOperationType::Add>;
using FMassDeferredRemoveCommand = FMassDeferredCommand<EMassCommandOperationType::Remove>;
using FMassDeferredChangeCompositionCommand = FMassDeferredCommand<EMassCommandOperationType::ChangeComposition>;
using FMassDeferredSetCommand = FMassDeferredCommand<EMassCommandOperationType::Set>;
using FMassDeferredDestroyCommand = FMassDeferredCommand<EMassCommandOperationType::Destroy>;

#undef DEBUG_NAME
#undef DEBUG_NAME_PARAM
#undef FORWARD_DEBUG_NAME_PARAM
