// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityElementTypes.h"
#include "MassProcessingTypes.h"
#if WITH_MASSENTITY_DEBUG
#include "MassExecutionContext.h"
#include "Mass/EntityHandle.h"
#include "MassDebugger.h"
#include "VisualLogger/VisualLogger.h"
#endif // WITH_MASSENTITY_DEBUG
#include "MassDebugLogging.generated.h"


struct FMassExecutionContext;
class UObject;

USTRUCT()
struct FMassDebugLogFragment : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY()
	TWeakObjectPtr<const UObject> LogOwner = nullptr;
};

namespace UE::Mass::Debug
{
	struct FLoggingContext
	{
#if WITH_MASSENTITY_DEBUG
		explicit FLoggingContext(const FMassExecutionContext& InContext, bool bInLogEverythingWhenRecording = true)
			: DebugFragmentsView(InContext.GetFragmentView<FMassDebugLogFragment>())
			, EntityListView(InContext.GetEntities())
			, bLogEverythingWhenRecording(bInLogEverythingWhenRecording)
		{	
		}

		inline bool ShouldLogEntity(int32 EntityIndex, FColor* OutEntityColor = nullptr) const
		{
#if ENABLE_VISUAL_LOG
			if (bLogEverythingWhenRecording
				&& DebugFragmentsView.IsValidIndex(EntityIndex)
				&& DebugFragmentsView[EntityIndex].LogOwner != nullptr
				&& FVisualLogger::IsRecording())
			{
				return true;
			}
#endif // ENABLE_VISUAL_LOG
			// If no actor owner or the visual logger is not recording, base it on the mass debugger
			return EntityListView.IsValidIndex(EntityIndex)
				&& IsDebuggingEntity(EntityListView[EntityIndex], OutEntityColor);
		}

		inline const UObject* GetLogOwner(int32 EntityIndex, const UObject* FallbackOwner = nullptr) const
		{
			return DebugFragmentsView.IsValidIndex(EntityIndex)
				? DebugFragmentsView[EntityIndex].LogOwner.Get()
				: FallbackOwner;
		}

	private:
		const TConstArrayView<FMassDebugLogFragment> DebugFragmentsView;
		const TConstArrayView<FMassEntityHandle> EntityListView;

		/** If true, ShouldLogEntity will return true when the visual logger is recording
		 *  If false, ShouldLogEntity will rely only on the MassDebugger
		 */
		const bool bLogEverythingWhenRecording = true;

#else

		explicit FLoggingContext(const FMassExecutionContext& Context, bool bInCheckVisualLoggerForRecording = true)
		{	
		}

		bool ShouldLogEntity(int32 EntityIndex, FColor* OutEntityColor = nullptr) const
		{
			return false;
		}

		inline UObject* GetLogOwner(int32 EntityIndex, UObject* FallbackOwner = nullptr) const
		{
			return FallbackOwner;
		}
#endif // WITH_MASSENTITY_DEBUG
	};
} // namespace UE::Mass::Debug
