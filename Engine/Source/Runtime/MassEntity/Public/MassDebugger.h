// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessingTypes.h"
#include "MassEntityTrace.h"
#if WITH_MASSENTITY_DEBUG
#include "Containers/ContainersFwd.h"
#include "MassDebuggerBreakpoints.h"
#include "MassEntityQuery.h"
#include "MassProcessor.h"
#include "Async/TransactionallySafeMutex.h"
#include "Logging/TokenizedMessage.h"
#include "UObject/ObjectKey.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "StructUtils/InstancedStruct.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

class FOutputDevice;
class FStructOnScope;
class UMassProcessor;
class UWorld;
struct FMassEntityQuery;
struct FMassEntityManager;
struct FMassArchetypeHandle;
struct FMassArchetypeChunk;
struct FMassFragmentRequirements;
struct FMassFragmentRequirementDescription;
struct FMassExecutionContext;
enum class EMassFragmentAccess : uint8;
enum class EMassFragmentPresence : uint8;
#endif // WITH_MASSENTITY_DEBUG
#include "MassDebugger.generated.h"

namespace UE::Mass::Debug
{
	struct FArchetypeStats
	{
		/** Number of active entities of the archetype. */
		int32 EntitiesCount = 0;
		/** Number of entities that fit per chunk. */
		int32 EntitiesCountPerChunk = 0;
		/** Number of allocated chunks. */
		int32 ChunksCount = 0;
		/** Total amount of memory taken by this archetype. Includes SparseElementsBookkeepingSize. */
		SIZE_T AllocatedSize = 0;
		/** How much memory allocated for entities is being unused */
		SIZE_T WastedEntityMemory = 0;
		/** Total amount of memory needed by a single entity */
		SIZE_T BytesPerEntity = 0;
		/**
		 * Memory used by per-chunk sparse element bookkeeping (FChunkSparseElements). Reported as a subset
		 * of AllocatedSize so it can be displayed separately. Will be zero for archetypes whose chunks
		 * carry no sparse elements.
		 */
		SIZE_T SparseElementsBookkeepingSize = 0;
	};

	using FProcessorProviderFunction = TFunction<void(TArray<const UMassProcessor*>&)>;
} // namespace UE::Mass::Debug

USTRUCT()
struct FMassGenericDebugEvent
{
	GENERATED_BODY()
	explicit FMassGenericDebugEvent(const UObject* InContext = nullptr)
#if WITH_EDITORONLY_DATA
		: Context(InContext)
#endif // WITH_EDITORONLY_DATA
	{
	}

#if WITH_EDITORONLY_DATA
	// note that it's not a uproperty since these events are only intended to be used instantly, never stored
	const UObject* Context = nullptr;
#endif // WITH_EDITORONLY_DATA
};

#if WITH_MASSENTITY_DEBUG

namespace UE::Mass::Debug
{
	extern MASSENTITY_API bool bAllowProceduralDebuggedEntitySelection;
	extern MASSENTITY_API bool bAllowBreakOnDebuggedEntity;
	extern MASSENTITY_API bool bTestSelectedEntityAgainstProcessorQueries;
	using FArchetypeFunction = TFunction<void(FMassArchetypeHandle)>;
} // namespace UE::Mass::Debug

#define MASS_IF_ENTITY_DEBUGGED(Manager, EntityHandle) (FMassDebugger::GetSelectedEntity(Manager) == EntityHandle)
#define MASS_BREAK_IF_ENTITY_DEBUGGED(Manager, EntityHandle) { if (UE::Mass::Debug::bAllowBreakOnDebuggedEntity && MASS_IF_ENTITY_DEBUGGED(Manager, EntityHandle)) { PLATFORM_BREAK();} }
#define MASS_BREAK_IF_ENTITY_INDEX(EntityHandle, InIndex) { if (UE::Mass::Debug::bAllowBreakOnDebuggedEntity && EntityHandle.Index == InIndex) { PLATFORM_BREAK();} }
#define MASS_SET_ENTITY_DEBUGGED(Manager, EntityHandle) { if (UE::Mass::Debug::bAllowProceduralDebuggedEntitySelection) {FMassDebugger::SelectEntity(Manager, EntityHandle); }}

enum class EMassDebugMessageSeverity : uint8
{
	Error,
	Warning,
	Info,
	// the following two need to remain last
	Default,
	MAX = Default
};

namespace UE::Mass::Debug
{
	struct FQueryRequirementsView
	{
		TConstArrayView<FMassFragmentRequirementDescription> FragmentRequirements;
		TConstArrayView<FMassFragmentRequirementDescription> ChunkRequirements;
		TConstArrayView<FMassFragmentRequirementDescription> ConstSharedRequirements;
		TConstArrayView<FMassFragmentRequirementDescription> SharedRequirements;
		const FMassTagBitSet& RequiredAllTags;
		const FMassTagBitSet& RequiredAnyTags;
		const FMassTagBitSet& RequiredNoneTags;
		const FMassTagBitSet& RequiredOptionalTags;
		const FMassExternalSubsystemBitSet& RequiredConstSubsystems;
		const FMassExternalSubsystemBitSet& RequiredMutableSubsystems;
	};

	FString DebugGetFragmentAccessString(EMassFragmentAccess Access);
	MASSENTITY_API extern void DebugOutputDescription(TConstArrayView<UMassProcessor*> Processors, FOutputDevice& Ar);

	MASSENTITY_API extern bool HasDebugEntities();
	MASSENTITY_API extern bool IsDebuggingSingleEntity();

	/**
	 * Populates OutBegin and OutEnd with entity index ranges as set by mass.debug.SetDebugEntityRange or
	 * mass.debug.DebugEntity console commands.
	 * @return whether any range has been configured.
	 */
	MASSENTITY_API extern bool GetDebugEntitiesRange(int32& OutBegin, int32& OutEnd);
	MASSENTITY_API extern bool IsDebuggingEntity(FMassEntityHandle Entity, FColor* OutEntityColor = nullptr);
	MASSENTITY_API extern FColor GetEntityDebugColor(FMassEntityHandle Entity);

	/**
	 * Fetches the right world to draw the debug drawing in, depending on the state of mass.debug.ServerPIEDebugDrawMode.
	 * Makes sense only in PIE, nop for other cases. Will return nullptr if drawing shouldn't be attempted,
	 * for example when mass.debug.ServerPIEDebugDrawMode = 1 and the call is on client.
	 */
	MASSENTITY_API extern UWorld* GetDebugDrawWorld(const FMassExecutionContext& Context);

	inline EMessageSeverity::Type MassSeverityToMessageSeverity(EMessageSeverity::Type OriginalSeverity, EMassDebugMessageSeverity MassSeverity)
	{
		static constexpr EMessageSeverity::Type ConversionMap[static_cast<int32>(EMassDebugMessageSeverity::MAX)] =
		{
			/*EMassDebugMessageSeverity::Error=*/EMessageSeverity::Error,
			/*EMassDebugMessageSeverity::Warning=*/EMessageSeverity::Warning,
			/*EMassDebugMessageSeverity::Info=*/EMessageSeverity::Info
		};
		return MassSeverity == EMassDebugMessageSeverity::Default 
			? OriginalSeverity
			: ConversionMap[static_cast<int32>(MassSeverity)];
	}
} // namespace UE::Mass::Debug

struct FMassDebugger
{
	struct FEnvironment
	{
		TWeakPtr<const FMassEntityManager> EntityManager;
		TWeakPtr<FMassEntityManager> MutableEntityManager;
		TMap<FName, UE::Mass::Debug::FProcessorProviderFunction> ProcessorProviders;
		FMassEntityHandle SelectedEntity;
		FMassEntityHandle HighlightedEntity;

		bool bHasBreakpoint = false;

		TArray<UE::Mass::Debug::FBreakpoint> Breakpoints;

		MASSENTITY_API UE::Mass::Debug::FBreakpoint* FindBreakpointByHandle(UE::Mass::Debug::FBreakpointHandle Handle);

		/** quick lookup to skip processors and fragments with no breakpoints set */
		TSet<TObjectKey<const UMassProcessor>> ProcessorsWithBreakpoints;
		TSet<TObjectKey<const UScriptStruct>> FragmentsWithBreakpoints;

#if UE_MASS_TRACE_ENABLED
		FDelegateHandle TraceStartedDelegateHandle;
#endif

		explicit FEnvironment(FMassEntityManager& InEntityManager);
		~FEnvironment();

		bool IsValid() const
		{
			return EntityManager.IsValid();
		}

		void ClearBreakpoints();
	};

	DECLARE_TS_MULTICAST_DELEGATE(FOnBreakpointsChanged);
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnEntitySelected, const FMassEntityManager&, const FMassEntityHandle);
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnMassEntityManagerEvent, const FMassEntityManager&);
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnEnvironmentEvent, const FEnvironment&);
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnDebugEvent, const FName /*EventName*/, FConstStructView /*Payload*/, const EMassDebugMessageSeverity /*SeverityOverride*/);

	static MASSENTITY_API TConstArrayView<FMassEntityQuery*> GetProcessorQueries(const UMassProcessor& Processor);
	/** fetches all queries registered for given Processor. Note that in order to get up to date information
	 *  FMassEntityQuery::CacheArchetypes will be called on each query */
	static MASSENTITY_API TConstArrayView<FMassEntityQuery*> GetUpToDateProcessorQueries(const FMassEntityManager& EntityManager, UMassProcessor& Processor);

	static MASSENTITY_API UE::Mass::Debug::FQueryRequirementsView GetQueryRequirements(const FMassEntityQuery& Query);
	static MASSENTITY_API void GetQueryExecutionRequirements(const FMassEntityQuery& Query, FMassExecutionRequirements& OutExecutionRequirements);
	static MASSENTITY_API TArray<FMassEntityHandle> GetEntitiesMatchingQuery(const FMassEntityManager& EntityManager, const FMassEntityQuery& Query);

	static MASSENTITY_API void ForEachArchetype(const FMassEntityManager& EntityManager, const UE::Mass::Debug::FArchetypeFunction& Function);
	static MASSENTITY_API TArray<FMassArchetypeHandle> GetAllArchetypes(const FMassEntityManager& EntityManager);
	static MASSENTITY_API FMassArchetypeCompositionDescriptor GetArchetypeComposition(const FMassArchetypeHandle& ArchetypeHandle);
	static MASSENTITY_API FMassElementBitSet GetArchetypeSparseElementsBitset(const FMassArchetypeHandle& ArchetypeHandle);

	static MASSENTITY_API uint64 GetArchetypeTraceID(const FMassArchetypeData& ArchetypeData);
	static MASSENTITY_API uint64 GetArchetypeTraceID(const FMassArchetypeHandle& ArchetypeHandle);
	
	static MASSENTITY_API TConstArrayView<FMassEntityHandle> GetEntitiesViewOfArchetype(
		const FMassArchetypeData& ArchetypeData,
		const FMassArchetypeChunk& Chunk);

	static MASSENTITY_API const FMassArchetypeData* GetArchetypeData(const FMassArchetypeHandle& ArchetypeHandle);
	static MASSENTITY_API void EnumerateChunks(const FMassArchetypeData& Archetype, TFunctionRef<void(const FMassArchetypeChunk&)> Fn);

	static MASSENTITY_API void GetArchetypeEntityStats(const FMassArchetypeHandle& ArchetypeHandle, UE::Mass::Debug::FArchetypeStats& OutStats);
	static MASSENTITY_API const TConstArrayView<FName> GetArchetypeDebugNames(const FMassArchetypeHandle& ArchetypeHandle);
	static MASSENTITY_API TArray<FMassEntityHandle> GetEntitiesOfArchetype(const FMassArchetypeHandle& ArchetypeHandle);

	static MASSENTITY_API TConstArrayView<UMassCompositeProcessor::FDependencyNode> GetProcessingGraph(const UMassCompositeProcessor& GraphOwner);
	static MASSENTITY_API TConstArrayView<TObjectPtr<UMassProcessor>> GetHostedProcessors(const UMassCompositeProcessor& GraphOwner);
	
	static MASSENTITY_API FString GetSingleRequirementDescription(const FMassFragmentRequirementDescription& Requirement);
	static MASSENTITY_API FString GetRequirementsDescription(const FMassFragmentRequirements& Requirements);
	static MASSENTITY_API FString GetArchetypeRequirementCompatibilityDescription(const FMassFragmentRequirements& Requirements, const FMassArchetypeHandle& ArchetypeHandle);
	static MASSENTITY_API FString GetArchetypeRequirementCompatibilityDescription(const FMassFragmentRequirements& Requirements, const FMassElementBitSet& ArchetypeCompositionBitSet);
	static FString GetArchetypeRequirementCompatibilityDescription(const FMassFragmentRequirements& Requirements, const FMassArchetypeCompositionDescriptor& ArchetypeComposition)
	{
		return GetArchetypeRequirementCompatibilityDescription(Requirements, ArchetypeComposition.GetElementsBitSet());
	}

	static MASSENTITY_API void OutputArchetypeDescription(FOutputDevice& Ar, const FMassArchetypeHandle& Archetype);
	static MASSENTITY_API void OutputEntityDescription(FOutputDevice& Ar, const FMassEntityManager& EntityManager, const int32 EntityIndex, const TCHAR* InPrefix = TEXT(""));
	static MASSENTITY_API void OutputEntityDescription(FOutputDevice& Ar, const FMassEntityManager& EntityManager, const FMassEntityHandle Entity, const TCHAR* InPrefix = TEXT(""));

	static MASSENTITY_API void SelectEntity(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle);
	static MASSENTITY_API FMassEntityHandle GetSelectedEntity(const FMassEntityManager& EntityManager);

	static MASSENTITY_API void HighlightEntity(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle);
	static MASSENTITY_API FMassEntityHandle GetHighlightedEntity(const FMassEntityManager& EntityManager);

	static MASSENTITY_API FOnBreakpointsChanged OnBreakpointsChangedDelegate;
	static MASSENTITY_API FOnEntitySelected OnEntitySelectedDelegate;

	static MASSENTITY_API FOnMassEntityManagerEvent OnEntityManagerInitialized;
	static MASSENTITY_API FOnMassEntityManagerEvent OnEntityManagerDeinitialized;
	static MASSENTITY_API FOnEnvironmentEvent OnProcessorProviderRegistered;

	static MASSENTITY_API FOnDebugEvent OnDebugEvent;
	
	static void DebugEvent(const FName EventName, FConstStructView Payload, const EMassDebugMessageSeverity SeverityOverride = EMassDebugMessageSeverity::Default)
	{
		OnDebugEvent.Broadcast(EventName, Payload, SeverityOverride);
	}

	template<typename TMessage, typename... TArgs>
	static void DebugEvent(TArgs&&... InArgs)
	{
		DebugEvent(TMessage::StaticStruct()->GetFName()
			, FConstStructView::Make(TMessage(Forward<TArgs>(InArgs)...)));
	}

	/**
	 * Registers given EntityManager with the debugger, creating a new entry in ActiveEnvironments.
	 * @return the index of newly created environment
	 */
	static MASSENTITY_API int32 RegisterEntityManager(FMassEntityManager& EntityManager);
	static MASSENTITY_API void UnregisterEntityManager(FMassEntityManager& EntityManager);
	
	/**
	 * Confirms that the initialization state of the EntityManager is Initialized.
	 * @return true if EntityManager is initialized, false otherwise
	 */
	static MASSENTITY_API bool IsEntityManagerInitialized(const FMassEntityManager& EntityManager);

	/**
	 * Registers the given ProviderFunction with the existing FEnvironment associated with the provided EntityManager.
	 * If one doesn't exist yet, it will be created (i.e. will automatically call RegisterEntityManager() with the provided EntityManager).
	 * The function will be called during data collection for the given FEnvironment.
	 * NOTE: there's no UnregisterProcessorDataProvider, the registered providers will automatically get removed along with
	 * the rest of the data associated with the relevant EntityManager as part of UnregisterEntityManager call.
	 */
	static MASSENTITY_API void RegisterProcessorDataProvider(FName ProviderName, const TSharedRef<FMassEntityManager>& EntityManager, const UE::Mass::Debug::FProcessorProviderFunction& ProviderFunction);

	static TConstArrayView<FEnvironment> GetEnvironments()
	{
		return ActiveEnvironments;
	}
	static MASSENTITY_API FEnvironment* FindEnvironmentForEntityManager(const FMassEntityManager& EntityManager);

	/**
	 * Determines whether given Archetype matches given Requirements. In case of a mismatch description of failed conditions will be added to OutputDevice.
	 */
	static MASSENTITY_API bool DoesArchetypeMatchRequirements(const FMassArchetypeHandle& ArchetypeHandle, const FMassFragmentRequirements& Requirements, FOutputDevice& OutputDevice);

	/**
	 * Checks if a processor should break on execute for a given entity. Returns the Id for the breakpoint if one is found, or 0.
	 */
	static MASSENTITY_API UE::Mass::Debug::FBreakpointHandle ShouldProcessorBreak(const FMassEntityManager& EntityManager, const UMassProcessor* Processor, FMassEntityHandle Entity);

	/**
	 * Checks if a processor has any breakpoints set for any entity.
	 */
	static MASSENTITY_API bool HasAnyProcessorBreakpoints(const FMassEntityManager& EntityManager, const UMassProcessor* Processor);

	/**
	 * Checks if a break should be triggered for a processor that's about to write a given fragment on an entity. Returns the Id for the breakpoint if one is found (Invalid otherwise).
	 */
	static MASSENTITY_API UE::Mass::Debug::FBreakpointHandle ShouldBreakOnFragmentWrite(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity);

	/**
	 * Find a breakpoint by Handle. Returns the pointer to the Breakpoint, or nullptr if none found.
	 */
	static MASSENTITY_API UE::Mass::Debug::FBreakpoint* FindBreakpoint(const FMassEntityManager& EntityManager, UE::Mass::Debug::FBreakpointHandle Handle);

	/**
	 * Get all the breakpoints for a given environment.
	 */
	static MASSENTITY_API TArray<UE::Mass::Debug::FBreakpoint>& GetBreakpoints(const FMassEntityManager& EntityManager);

	/**
	 * Refresh breakpoint flags after changes to breakpoint instances.
	 */
	static MASSENTITY_API void RefreshBreakpoints();

	/**
	 * Checks if there are any breakpoints set for writing a fragment for any entity
	 * Use FragmentType = nullptr (default) to check for ANY fragment types.
	 */
	static MASSENTITY_API bool HasAnyFragmentWriteBreakpoints(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType = nullptr);

	/**
	 * Create a default-constructed breakpoint
	 */
	static MASSENTITY_API UE::Mass::Debug::FBreakpoint& CreateBreakpoint(const FMassEntityManager& EntityManager);

	/**
	 * Sets a break to be triggered on processor execute for an entity.
	 */
	static MASSENTITY_API void SetProcessorBreakpoint(const FMassEntityManager& EntityManager, TNotNull<const UMassProcessor*>, FMassEntityHandle Entity);

	/**
	 * Sets a break to be triggered for a processor that's about to write a given fragment on an entity.
	 */
	static MASSENTITY_API void SetFragmentWriteBreakpoint(const FMassEntityManager& EntityManager, TNotNull<const UScriptStruct*> FragmentType, FMassEntityHandle Entity);

	/**
	 * Sets or removes a break before a write to a given fragment on whichever entity is selected at the time.
	 */
	static MASSENTITY_API void SetFragmentWriteBreakForSelectedEntity(const FMassEntityManager& EntityManager, TNotNull<const UScriptStruct*> FragmentType);

	/**
	 * Enable or Disable a breakpoint with a given Id.
	 */
	static MASSENTITY_API void SetBreakpointEnabled(UE::Mass::Debug::FBreakpointHandle Handle, bool bEnable);

	/**
	 * Clears a breakpoint triggered on processor execute for an entity.
	 */
	static MASSENTITY_API void ClearProcessorBreakpoint(const FMassEntityManager& EntityManager, const UMassProcessor* Processor, FMassEntityHandle Entity);

	/**
	 * Clears all breakpoints set for a given processor.
	 */
	static MASSENTITY_API void ClearAllProcessorBreakpoints(const FMassEntityManager& EntityManager, const UMassProcessor* Processor);

	/**
	 * Clears a fragment write breakpoint for a given fragment type and entity.
	 */
	static MASSENTITY_API void ClearFragmentWriteBreak(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity);
	
	/**
	 * Clears all write breakpoints set for a given fragment type.
	 */
	static MASSENTITY_API void ClearAllFragmentWriteBreak(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType);

	/**
	 * Clears all breakpoints set for a given entity.
	 */
	static MASSENTITY_API void ClearAllEntityBreakpoints(const FMassEntityManager& EntityManager, FMassEntityHandle Entity);

	/**
	 * Sets a write breakpoint for the specified fragment on the selcected entity
	 * @see SelectEntity
	 */
	static MASSENTITY_API void BreakOnFragmentWriteForSelectedEntity(FName FragmentName);

	/**
	 * Gets the UScriptStruct type for fragment of the specified name.
	 */
	static MASSENTITY_API const UScriptStruct* GetFragmentTypeFromName(FName FragmentName);

	/**
	 * Finds the fragment data of the specified type in the entity data. Returns nullptr if not found.
	 */
	static MASSENTITY_API TSharedPtr<FStructOnScope> GetFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity);

	/**
	 * Finds the fragment data of the specified type in the entity data. Returns false if not found.
	 */
	static MASSENTITY_API bool GetFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity, TSharedPtr<FStructOnScope>& OutStructData);

	/**
	 * Get the shared fragment value container for this entity
	 */
	static MASSENTITY_API const FMassArchetypeSharedFragmentValues& GetSharedFragmentValues(const FMassEntityManager& EntityManager, FMassEntityHandle Entity);
	
	/**
	 * Finds the shared fragment data of the specified type in the entity data. Returns nullptr if not found.
	 */
	static MASSENTITY_API TSharedPtr<FStructOnScope> GetSharedFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity);

	/**
	 * Finds the shared fragment data of the specified type in the entity data. Returns nullptr if not found.
	 */
	static MASSENTITY_API bool GetSharedFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity, TSharedPtr<FStructOnScope>& OutStructData);
	
	/**
	 * Finds the const shared fragment data of the specified type in the entity data. Returns nullptr if not found.
	 */
	static MASSENTITY_API TSharedPtr<FStructOnScope> GetConstSharedFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity);

	/**
	 * Finds the const shared fragment data of the specified type in the entity data. Returns nullptr if not found.
	 */
	static MASSENTITY_API bool GetConstSharedFragmentData(const FMassEntityManager& EntityManager, const UScriptStruct* FragmentType, FMassEntityHandle Entity, TSharedPtr<FStructOnScope>& OutStructData);

	/**
	 * Clears all breakpoints in all environments.
	 */
	static MASSENTITY_API void ClearAllBreakpoints();

	/**
	 * Remove a single breakpoint by Id
	 */
	static MASSENTITY_API void RemoveBreakpoint(UE::Mass::Debug::FBreakpointHandle Handle);

	/**
	 * Destroy all debug environments and shutdown cleanly.
	 */
	static void ShutdownDebugger();
	
private:
	// @todo: contain static data in an instanced struct and setup/teardown in startup/shutdown module
	static MASSENTITY_API TArray<FEnvironment> ActiveEnvironments;
	static MASSENTITY_API UE::FTransactionallySafeMutex EntityManagerRegistrationLock;
	static MASSENTITY_API TMap<FName, const UScriptStruct*> FragmentsByName;
	static MASSENTITY_API TArray<FName> CommandNames;

	static MASSENTITY_API FEnvironment& GetActiveEnvironmentChecked(const FMassEntityManager& EntityManager);
	static MASSENTITY_API FEnvironment* GetActiveEnvironment(const FMassEntityManager& EntityManager);
};

#else

struct FMassArchetypeHandle;
struct FMassFragmentRequirements;
struct FMassFragmentRequirementDescription;
struct FMassArchetypeCompositionDescriptor;
struct FMassExecutionContext;

struct FMassDebugger
{
	static FString GetSingleRequirementDescription(const FMassFragmentRequirementDescription&)
	{
		return TEXT("[no debug information]");
	}
	static FString GetRequirementsDescription(const FMassFragmentRequirements&)
	{
		return TEXT("[no debug information]");
	}
	static FString GetArchetypeRequirementCompatibilityDescription(const FMassFragmentRequirements&, const FMassArchetypeHandle&)
	{
		return TEXT("[no debug information]");
	}
	static FString GetArchetypeRequirementCompatibilityDescription(const FMassFragmentRequirements&, const FMassArchetypeCompositionDescriptor&)
	{
		return TEXT("[no debug information]");
	}
};

namespace UE::Mass::Debug
{
	MASSENTITY_API extern UWorld* GetDebugDrawWorld(const FMassExecutionContext& Context);
} // namespace UE::Mass::Debug

#define MASS_IF_ENTITY_DEBUGGED(a, b) false
#define MASS_BREAK_IF_ENTITY_DEBUGGED(a, b)
#define MASS_BREAK_IF_ENTITY_INDEX(a, b)
#define MASS_SET_ENTITY_DEBUGGED(a, b)

#endif // WITH_MASSENTITY_DEBUG
