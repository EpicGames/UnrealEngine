// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Mass/EntityMacros.h"

#if WITH_MASSENTITY_DEBUG
#define MASS_BREAKPOINT(bShouldBreak)					\
	do {												\
		if (UNLIKELY(bShouldBreak))						\
		{												\
			UE::Mass::Debug::FBreakpoint::DebugBreak();	\
		}												\
	} while (0)

#include "Mass/EntityHandle.h"
#include "Misc/TVariant.h"
#include "UObject/ObjectKey.h"
#include "MassRequirements.h"

struct FMassDebugger;
class UMassProcessor;
class UScriptStruct;

namespace UE::Mass::Debug
{
	struct FBreakpointHandle
	{
		int32 Handle = 0;

		FBreakpointHandle(int32 InHandle)
			: Handle(InHandle)
		{
		}

		FBreakpointHandle() = default;

		friend bool operator==(const FBreakpointHandle& A, const FBreakpointHandle& B)
		{
			return A.Handle == B.Handle;
		}

		bool IsValid() const
		{
			return Handle > 0;
		}

		operator bool() const
		{
			return IsValid();
		}

		static FBreakpointHandle CreateHandle()
		{
			static int32 LastHandle = 0;
			return FBreakpointHandle{ ++LastHandle };
		}

		static FBreakpointHandle Invalid()
		{
			return FBreakpointHandle();
		}
	};

	struct FBreakpoint
	{
		friend FMassDebugger;

		MASSENTITY_API FBreakpoint();

		enum class ETriggerType : uint8
		{
			None,
			ProcessorExecute,
			FragmentWrite,
			EntityCreate,
			EntityDestroy,
			FragmentAdd,
			FragmentRemove,
			TagAdd,
			TagRemove,
			MAX
		};
		static MASSENTITY_API  FString TriggerTypeToString(FBreakpoint::ETriggerType InType);
		static MASSENTITY_API bool StringToTriggerType(const FString& InString, FBreakpoint::ETriggerType& OutType);

		enum class EFilterType : uint8
		{
			None,
			SpecificEntity,
			SelectedEntity,
			Query
		};

		static MASSENTITY_API FString FilterTypeToString(FBreakpoint::EFilterType InType);
		static MASSENTITY_API bool StringToFilterType(const FString& InString, FBreakpoint::EFilterType& OutType);

		using TriggerVariant = TVariant<
			TObjectKey<const UMassProcessor>,
			TObjectKey<const UScriptStruct>
		>;

		using FilterVariant = TVariant<
			FMassEntityHandle,
			FMassFragmentRequirements
		>;

		FBreakpointHandle Handle;
		mutable uint64 HitCount = 0;
		ETriggerType TriggerType = ETriggerType::None;
		EFilterType FilterType = EFilterType::None;
		TriggerVariant Trigger;
		FilterVariant Filter;
		bool bEnabled = false;

		/**
		 * Apply the breakpoint Filter based on a list of fragment types
		 *
		 * @param Fragments	Array of fragment types to evaluate the breakpoint filter against
		 *
		 * @return True if the filter is set to an applicable type it evaluates to true for the provided Fragments
		 */
		MASSENTITY_API bool ApplyEntityFilterByFragments(const TArray<const UScriptStruct*>& Fragments) const;

		/**
		 * Apply the breakpoint Filter based on an FMassArchetypeHandle
		 *
		 * @param ArchetypeHandle	The archetype to evaluate breakpoint filter against
		 *
		 * @return True if the filter is set to an applicable type it evaluates to true for the provided ArchetypeHandle
		 */
		MASSENTITY_API bool ApplyEntityFilterByArchetype(const FMassArchetypeHandle& ArchetypeHandle) const;

		/**
		 * Evaluate the breakpoint filter against a specific entity
		 *
		 * @param EntityManager	The FMassEntityManager that owns the entity
		 * @param Entity	The handle of the entity against which to evaluate the filter
		 *
		 * @return True if the filter is set to an applicable type it evaluates to true for the provided Entity
		 */
		MASSENTITY_API bool ApplyEntityFilter(const FMassEntityManager& EntityManager, const FMassEntityHandle& Entity) const;

		/**
		 * Checks if a fragment add on a given entity should trigger a breakpoint
		 *
		 * @param FMassEntityHandle	The entity being modified
		 * @param FragmentType	The type of fragment being added
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		static MASSENTITY_API bool CheckFragmentAddBreakpoints(const FMassEntityHandle& Handle, const UScriptStruct* FragmentType);

		/**
		 * Checks if a fragment add on a given entity should trigger a breakpoint (templated version)
		 *
		 * @param Handle	The entity being modified
		 * @param InFragments	variadic list of fragment types being added
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		template<typename... TFragments>
		static bool CheckFragmentAddBreakpoints(const FMassEntityHandle& Handle, TFragments... InFragments)
		{
			if (LIKELY(!bHasBreakpoint))
			{
				return false;
			}
			return (... || CheckFragmentAddBreakpoints(Handle, TFragments::StaticStruct()));
		}

		/**
		 * Checks if a fragment add on a group of entities should trigger a breakpoint
		 *
		 * @param Entities	The list of entities being modified
		 * @param FragmentType	The type of fragment being added
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		static MASSENTITY_API bool CheckFragmentAddBreakpoints(TConstArrayView<FMassEntityHandle> Entities, const UScriptStruct* FragmentType);

		/**
		 * Checks if a fragment add on a group of entities should trigger a breakpoint (templated version)
		 *
		 * @param Entities	The list of entities being modified
		 * @param InFragments	variadic list of fragment types being added
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		template<typename... TFragments>
		static bool CheckFragmentAddBreakpoints(TConstArrayView<FMassEntityHandle> Entities, TFragments... InFragments)
		{
			if (LIKELY(!bHasBreakpoint))
			{
				return false;
			}

			return (... || CheckFragmentAddBreakpoints(Entities, TFragments::StaticStruct()));
		}

		/**
		 * Checks if a fragment add on a group of entities should trigger a breakpoint (fragment instance version)
		 *
		 * @param Handle	The entity being modified
		 * @param InFragments	variadic list of fragment instance references being added
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		template<typename... TFragments>
		static bool CheckFragmentAddBreakpointsByInstances(const FMassEntityHandle& Handle, TFragments... InFragments)
		{
			if (LIKELY(!bHasBreakpoint))
			{
				return false;
			}
			return (... || CheckFragmentAddBreakpoints(Handle, std::remove_reference_t<decltype(InFragments)>::StaticStruct()));
		}

		/**
		 * Checks if a fragment add on a group of entities should trigger a breakpoint (fragment instance version)
		 *
		 * @param Entities	The list of entities being modified
		 * @param InFragments	variadic list of fragment instance references being added
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		template<typename... TFragments>
		static bool CheckFragmentAddBreakpointsByInstances(TConstArrayView<FMassEntityHandle> Entities, TFragments... InFragments)
		{
			if (LIKELY(!bHasBreakpoint))
			{
				return false;
			}

			return (... || CheckFragmentAddBreakpoints(Entities, std::remove_reference_t<decltype(InFragments)>::StaticStruct()));
		}

		/**
		 * Checks if a fragment remove on a given entity should trigger a breakpoint
		 *
		 * @param FMassEntityHandle	The entity being modified
		 * @param FragmentType	The type of fragment being removed
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		static MASSENTITY_API bool CheckFragmentRemoveBreakpoints(const FMassEntityHandle& Handle, const UScriptStruct* FragmentType);

		/**
		 * Checks if a fragment remove on a given entity should trigger a breakpoint (templated version)
		 *
		 * @param Handle	The entity being modified
		 * @param InFragments	variadic list of fragment types being removed
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		template<typename... TFragments>
		static bool CheckFragmentRemoveBreakpoints(const FMassEntityHandle& Handle, TFragments... InFragments)
		{
			if (LIKELY(!bHasBreakpoint))
			{
				return false;
			}
			return (... || CheckFragmentRemoveBreakpoints(Handle, std::remove_reference_t<decltype(InFragments)>::StaticStruct()));
		}

		/**
		 * Checks if a fragment remove on a group of entities should trigger a breakpoint
		 *
		 * @param Entities	The list of entities being modified
		 * @param FragmentType	The type of fragment being removed
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		static MASSENTITY_API bool CheckFragmentRemoveBreakpoints(TConstArrayView<FMassEntityHandle> Entities, const UScriptStruct* FragmentType);

		/**
		 * Checks if a fragment remove on a group of entities should trigger a breakpoint (templated version)
		 *
		 * @param Entities	The list of entities being modified
		 * @param InFragments	variadic list of fragment types being removed
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		template<typename... TFragments>
		static bool CheckFragmentRemoveBreakpoints(TConstArrayView<FMassEntityHandle> Entities, TFragments... InFragments)
		{
			if (LIKELY(!bHasBreakpoint))
			{
				return false;
			}
			return (... || CheckFragmentRemoveBreakpoints(Entities, TFragments::StaticStruct()));
		}

		/**
		 * Checks if an entity creation event should trigger a breakpoint
		 *
		 * @param Fragments	array of fragment types that compose the entity being created
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		static bool CheckCreateEntityBreakpoints(TArray<const UScriptStruct*> Fragments);

		/**
		 * Checks if an entity creation event should trigger a breakpoint (templated version)
		 *
		 * @param InFragments	variadic list of fragment types that compose the entity being created
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		template<typename... TFragments>
		static bool CheckCreateEntityBreakpoints(TFragments... InFragments)
		{
			if (LIKELY(!bHasBreakpoint))
			{
				return false;
			}

			return CheckCreateEntityBreakpoints({ TFragments::StaticStruct()... });
		}

		/**
		 * Checks if an entity creation event should trigger a breakpoint (archetype version)
		 *
		 * @param ArchetypeHandle	The archetype of the entity being created
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		static MASSENTITY_API bool CheckCreateEntityBreakpoints(const FMassArchetypeHandle& ArchetypeHandle);

		/**
		 * Checks if an entity destruction event should trigger a breakpoint
		 *
		 * @param Entity	The entity being destroyed
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		static MASSENTITY_API bool CheckDestroyEntityBreakpoints(const FMassEntityHandle& Entity);

		/**
		 * Checks if an entity destruction event should trigger a breakpoint
		 *
		 * @param Entities	The array entities being destroyed
		 *
		 * @return True if any breakpoints are set and enabled that match this event
		 */
		static MASSENTITY_API bool CheckDestroyEntityBreakpoints(TConstArrayView<FMassEntityHandle> Entities);

		/**
		 * Disable the currently-active breakpoint (if any)
		 */
		static MASSENTITY_API void DisableActiveBreakpoint();

		/**
		 * Checks if any breakpoints are currently set
		 *
		 * @return True if there are any breakpoints set in the mass debugger even if they're disabled (so the hit count can still increment).
		 */
		static inline bool HasBreakpoint()
		{
			return bHasBreakpoint;
		}

		/**
		 * Triggers a breakpoint in the attached debugger (if present)
		 */
		static MASSENTITY_API void DebugBreak();

	private:
		static bool bHasBreakpoint;
		static FBreakpointHandle LastBreakpointHandle;
	};
	
} // namespace UE::Mass::Debug

#else // WITH_MASSENTITY_DEBUG
namespace UE::Mass::Debug
{
	struct FBreakpointHandle
	{
	};

	struct FBreakpoint
	{
		enum class ETriggerType : uint8
		{
		};
		enum class EFilterType : uint8
		{
		};
	};
} // namespace UE::Mass::Debug

#define MASS_BREAKPOINT(...)	\
		do { } while (0)

#endif // WITH_MASSENTITY_DEBUG
