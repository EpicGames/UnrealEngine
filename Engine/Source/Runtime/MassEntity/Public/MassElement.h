// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "StructUtils/StructTypeBitSet.h"
#include "Mass/EntityElementTypes.h"
#include "Subsystems/Subsystem.h"

#define UE_API MASSENTITY_API

DECLARE_CLASSTYPEBITSET_EXPORTED(UE_API, FMassExternalSubsystemBitSet, USubsystem);

namespace UE::Mass
{
	DECLARE_STRUCTTYPEBITSET_EXPORTED(UE_API, FElementBitSetBase, FMassElement);

	enum class EElementType : uint8
	{
		Fragment,
		Tag,
		ChunkFragment,
		SharedFragment,
		ConstSharedFragment,
		MAX
	};

	template<typename TElementType>
	struct TElementTraits
	{};

	template<typename TBitSetType>
	struct TBitSetTraits
	{};

	struct FElementBitSet : public FElementBitSetBase
	{
		using FElementBitSetBase::FElementBitSetBase;
		using FElementBitSetBase::operator=;
		using FElementBitSetBase::HasAny;
		using FElementBitSetBase::HasAll;

		FElementBitSet(const FElementBitSetBase& OtherBitSet)
			: FElementBitSetBase(OtherBitSet)
		{
		}

		template<typename TBitSet>
		TBitSet Get() const;

		template<typename TElementType>
		bool HasAny() const;

		template<typename TElementType>
		bool HasAll() const;

		// @todo this is a temporary construct, until shared fragments get removed as a concept
		bool HasAnySharedElements() const;
		
		bool ContainsOnlyTagsAndFragments() const;
		FElementBitSet GetFragmentsAndTags() const;
		FElementBitSet GetSparseElements() const;
		FElementBitSet GetSharedFragments() const;
	
		static EElementType DetermineElementType(const TNotNull<const UStruct*> ElementType)
		{
			if (UE::Mass::IsA<FMassFragment>(ElementType))
			{
				return EElementType::Fragment;
			}
			else if (UE::Mass::IsA<FMassTag>(ElementType))
			{
				return EElementType::Tag;
			}
			else if (UE::Mass::IsA<FMassSharedFragment>(ElementType))
			{
				return EElementType::SharedFragment;
			}
			else if (UE::Mass::IsA<FMassConstSharedFragment>(ElementType))
			{
				return EElementType::ConstSharedFragment;
			}
			else if (UE::Mass::IsA<FMassChunkFragment>(ElementType))
			{
				return EElementType::ChunkFragment;
			}

			checkf(false, TEXT("%hs: Unhandled element type %s"), __FUNCTION__, *ElementType->GetName())
			return EElementType::MAX;
		}

		static void OnTypeRegistered(const TNotNull<const UStruct*> Type, int32 Index);

		static void OnModuleInitialized();

		static void OnModuleShutdown()
		{
			FElementBitSetBaseStructTrackerWrapper::StructTracker.OnTypeRegistered.Remove(OnTypeRegisteredDelegateHandle);
		}

		static const FElementBitSet& GetAllFragmentsAndTags()
		{
			return AllFragmentsAndTagsBitSet;
		}

		static const FElementBitSet& GetAllSharedFragments()
		{
			return AllSharedFragmentsBitSet;
		}

		static const FElementBitSet& GetAllSparseElements()
		{
			return AllSparseElementsBitSet;
		}

	protected:
		UE_API static TStaticArray<FElementBitSetBase, static_cast<uint8>(EElementType::MAX)> ElementTypeBitArrays;
		UE_API static FElementBitSet AllSharedFragmentsBitSet;
		UE_API static FElementBitSet AllFragmentsAndTagsBitSet;
		UE_API static FElementBitSet AllSparseElementsBitSet;
		static FDelegateHandle OnTypeRegisteredDelegateHandle;

	public:
		/**
		 * Fetches FElementBitSetBase containing all known specific element types
		 */
		static FElementBitSetBase& GetTypeBitArray(const EElementType ElementType)
		{	
			return ElementTypeBitArrays[static_cast<uint8>(ElementType)];
		}
	};

	template<typename TBitSet>
	inline TBitSet FElementBitSet::Get() const
	{
		return GetTypeBitArray(TBitSetTraits<TBitSet>::ElementType) & *this;
	}

	template<typename TElementType>
	bool FElementBitSet::HasAny() const
	{
		return HasAny(GetTypeBitArray(TElementTraits<TElementType>::ElementType));
	}

	template<typename TElementType>
	bool FElementBitSet::HasAll() const
	{
		return HasAll(GetTypeBitArray(TElementTraits<TElementType>::ElementType));
	}

	inline bool FElementBitSet::HasAnySharedElements() const
	{
		return FElementBitSetBase::HasAny(GetTypeBitArray(EElementType::SharedFragment)) || FElementBitSetBase::HasAny(GetTypeBitArray(EElementType::ConstSharedFragment));
	}

	inline bool FElementBitSet::ContainsOnlyTagsAndFragments() const
	{
		return GetFragmentsAndTags().IsEquivalent(*this);
	}

	inline FElementBitSet FElementBitSet::GetFragmentsAndTags() const
	{
		return (*this) & GetAllFragmentsAndTags();
	}

	inline FElementBitSet FElementBitSet::GetSparseElements() const
	{
		return (*this) & GetAllSparseElements();
	}

	inline FElementBitSet FElementBitSet::GetSharedFragments() const
	{
		return (*this) & GetAllSharedFragments();
	}

	//-----------------------------------------------------------------------------
	// Helpers to avoid redundant operations when dealing with specialized element bitsets
	//-----------------------------------------------------------------------------
	/** @return whether BitSet indicates only shared fragment types (both const and mutable) */
	inline bool DoesContainOnlySharedFragments(const FElementBitSet& BitSet)
	{
		return FElementBitSet::GetAllSharedFragments().HasAll(BitSet);
	}

	/** @return whether shared fragment type information stored by both bitsets is equivalent */
	inline bool DoContainEquivalentSharedFragments(const FElementBitSet& BitSetA, const FElementBitSet& BitSetB)
	{
		return BitSetA.IsEquivalentFiltered(BitSetB, FElementBitSet::GetAllSharedFragments());
	}


	#define DECLARE_NEWTYPEBITSET(ContainerTypeName, BaseType, ElementTypeEnumValue) struct ContainerTypeName : public FElementBitSetBase \
	{ \
		using FElementType = BaseType; \
		using FElementBitSetBase::FElementBitSetBase; \
		using FElementBitSetBase::operator=; \
		ContainerTypeName(const FElementBitSetBase& OtherBitSet) \
			: FElementBitSetBase(OtherBitSet) \
		{} \
		template<typename T> \
		static const FElementBitSetBase& GetTypeBitSet() \
		{ \
			static_assert(TIsDerivedFrom<typename TRemoveReference<T>::Type, BaseType>::Value, \
				"Type does not match the expected element type for this bitset."); \
			return FElementBitSetBase::GetTypeBitSet<T>(); \
		} \
	}; \
	template<> struct TElementTraits<BaseType> \
	{ \
		static constexpr EElementType ElementType = ElementTypeEnumValue; \
	}; \
	template<> struct TBitSetTraits<ContainerTypeName> \
	{ \
		static constexpr EElementType ElementType = ElementTypeEnumValue; \
	}


	DECLARE_NEWTYPEBITSET(FFragmentBitSet, FMassFragment, EElementType::Fragment);
	DECLARE_NEWTYPEBITSET(FTagBitSet, FMassTag, EElementType::Tag);
	DECLARE_NEWTYPEBITSET(FChunkFragmentBitSet, FMassChunkFragment, EElementType::ChunkFragment);
	DECLARE_NEWTYPEBITSET(FSharedFragmentBitSet, FMassSharedFragment, EElementType::SharedFragment);
	DECLARE_NEWTYPEBITSET(FConstSharedFragmentBitSet, FMassConstSharedFragment, EElementType::ConstSharedFragment);

	#undef DECLARE_NEWTYPEBITSET
} // namespace UE::Mass

using FMassElementBitSet = UE::Mass::FElementBitSet;
using FMassFragmentBitSet = UE::Mass::FFragmentBitSet;
using FMassTagBitSet = UE::Mass::FTagBitSet;
using FMassChunkFragmentBitSet = UE::Mass::FChunkFragmentBitSet;
using FMassSharedFragmentBitSet = UE::Mass::FSharedFragmentBitSet;
using FMassConstSharedFragmentBitSet = UE::Mass::FConstSharedFragmentBitSet;

#undef UE_API
