// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassArchetypeTypes.h"
#include "MassEntityConcepts.h"
#include "Mass/EntityHandle.h"
#include "MassEntityTypes.h"
#include "MassEntityRelations.h"
#include "MassTypeManager.h"

#define UE_API MASSENTITY_API

struct FMassEntityManager;

namespace UE::Mass
{
	struct FTypeHandle;
	namespace Private
	{
		struct FEntityBuilderHelper;
	}

/**
 * FEntityBuilder is a utility struct that provides a convenient way to create and configure entities
 * in the Mass framework. It bridges multiple APIs from FMassEntityManager, MassSpawnerSubsystem,
 * MassEntityTemplates, and other related components, allowing for streamlined entity creation and configuration.
 *
 * Key Features:
 * - Can be seamlessly used in place of FMassEntityHandle, allowing for consistent and intuitive usage.
 * - An entity only gets created once Commit() is called
 * - Copyable, but copied instances represent new entities without carrying over the reserved entity handle.
 *
 * Example Usage:
 * {
 * 		FEntityBuilder Builder(EntityManager);
 * 		Builder.Add<FTransformFragment>(FTransform(FVector(100, 200, 300)))
 * 			.Commit();	// the entity gets reserved and built by this call
 * } 
 *
 * {
 *     FEntityBuilder Builder(EntityManager);
 *     FMassEntityHandle ReservedEntity = Builder; // Entity handle reserved, can be used for commands.
 *     Builder.Add_GetRef<FTransformFragment>().GetMutableTransform().SetTranslation(FVector(100, 200, 300));
 *     Builder.Commit(); // Entity creation is finalized at this point.
 * }
 *
 * // Example of chaining with FMassEntityManager's MakeEntityBuilder() method:
 * FMassEntityHandle NewEntity = EntityManager.MakeEntityBuilder()
 *     .Add<FMassStaticRepresentationTag>()
 *     .Add<FTransformFragment>()
 *     .Add<FAgentRadiusFragment>(FAgentRadiusFragment{ .Radius = 35.f})
 *     .Add<FMassVelocityFragment>()
 *     .Commit();
 *
 * Current Limitations:
 * - Committing entities while Mass's processing is in progress is not yet supported; this functionality will be implemented in the near future.
 * - no support for entity grouping
 */
struct FEntityBuilder
{
	/** Constructs a FEntityBuilder using a reference to a FMassEntityManager. */
	UE_API explicit FEntityBuilder(FMassEntityManager& InEntityManager);

	/** Constructs a FEntityBuilder using a shared reference to a FMassEntityManager. */
	UE_API explicit FEntityBuilder(const TSharedRef<FMassEntityManager>& InEntityManager);

	/** Copy constructor - copies-create a new instance that represents a new entity and does not carry over reserved handle. */
	UE_API FEntityBuilder(const FEntityBuilder& Other);

	/** Assignment operator - copies represent new entities, with no carryover of reserved handle from the original. */
	UE_API FEntityBuilder& operator=(const FEntityBuilder& Other);

	/** Move assignment operator - moves over all the data from Other, including the internal state (like whether the entity handle has already been reserved) */
	UE_API FEntityBuilder& operator=(FEntityBuilder&& Other);

	/** Destructor - automatically commits entity creation if not explicitly aborted or committed beforehand. */
	UE_API ~FEntityBuilder();

	/** Creates an instance of FEntityBuilder and populates it with provided data */
	static UE_API FEntityBuilder Make(const TSharedRef<FMassEntityManager>& InEntityManager
		, const FMassArchetypeCompositionDescriptor& Composition
		, TConstArrayView<FInstancedStruct> InitialFragmentValues = {}
		, TConstArrayView<FConstSharedStruct> ConstSharedFragments = {}
		, TConstArrayView<FSharedStruct> SharedFragments = {});

	/** Creates an instance of FEntityBuilder and populates it with provided data, using move-semantics on said data */
	static UE_API FEntityBuilder Make(const TSharedRef<FMassEntityManager>& InEntityManager
		, const FMassArchetypeCompositionDescriptor& Composition
		, TArray<FInstancedStruct>&& InitialFragmentValues
		, TArray<FConstSharedStruct>&& ConstSharedFragments
		, TArray<FSharedStruct>&& SharedFragments);

	/**
	 * Configured given EntityBuilder instance's commit behavior. Using `true` will result in all Commit calls
	 * resulting in issuing Mass commands, regardless of EntityManager.IsProcessing() value.
	 */
	FEntityBuilder& SetForceDeferredCommit(bool bNewValue);

	/**
	 * Finalizes the creation of the entity with the specified fragments and configurations.
	 * Note that this function needs to be called manually, no automated entity creation will take place upon builder's destruction. 
	 */
	UE_API FMassEntityHandle Commit();

	/**
	 * A wrapper for "Commit" call that, once that's done, prepares the builder for another commit, forgetting the
	 * handle for the entity just created, and reverting the state back to "ReadyToCommit"
	 * @see Commit
	 */
	UE_API FMassEntityHandle CommitAndReprepare();

	/** if the builder is in "Committed" state it will roll back to ReadyToSubmit and reset the stored entity handle */
	UE_API void Reprepare();

	/**
	 * Resets the builder to its initial state, discarding all previous entity configurations.
	 * @param bReleaseEntityHandleIfReserved configures what to do with the reserved entity handle, if it's valid.
	 */
	UE_API void Reset(const bool bReleaseEntityHandleIfReserved = true);

	/**
	 * Stores ReservedEntityHandle as the cached EntityHandle. The ReservedEntityHandle is expected to be valid
	 * and represent a reserved entity. These expectations will be checked via ensures.
	 * If the existing EntityHandle also represents a valid, reserved entity, that handle will be released.
	 * @return whether the ReservedEntityHandle has been stored.
	 */
	UE_API bool SetReservedEntityHandle(const FMassEntityHandle ReservedEntityHandle);

	/**
	 * Appends all element types and values stored by the entity indicated by SourceEntityHandle.
	 * @param SourceEntityHandle valid handle for a fully constructed, built entity.
	 * @return whether the operation was successful
	 */
	UE_API bool AppendDataFromEntity(const FMassEntityHandle SourceEntityHandle, EIncludeSparseElements IncludeSparseElements = EIncludeSparseElements::Yes);

	/**
	 * Copies all element types and values stored by the entity indicated by SourceEntityHandle. Any existing builder data will be overridden
	 * @param SourceEntityHandle valid handle for a fully constructed, built entity.
	 * @return whether the operation was successful
	 */
	UE_API bool CopyDataFromEntity(const FMassEntityHandle SourceEntityHandle, EIncludeSparseElements IncludeSparseElements = EIncludeSparseElements::Yes);

	/**
	 * Adds a fragment of type T to the entity and returns a reference to it, constructing it with the provided arguments.
	 * The function will assert if an element of type T already exists.
	 * @param T - The type of fragment to add.
	 * @param InArgs - Constructor arguments for initializing the fragment.
	 * @return A reference to the added fragment.
	 */
	template<typename T, typename... TArgs>
	T& Add_GetRef(TArgs&&... InArgs) requires (CElement<T> && !(CTag<T> || CChunkFragment<T>));

	/**
	 * Adds a fragment of type T to the entity and returns a reference to it, constructing it with the provided arguments.
	 * If a fragment of the given type already exists then it will be overriden and its reference returned.
	 * @return A reference to the added fragment.
	 */
	template<typename T, typename... TArgs>
	T& GetOrCreate(TArgs&&... InArgs) requires (CElement<T> && !(CTag<T> || CChunkFragment<T>));

	/**
	 * Adds a tag of type T to the entity.
	 * @return Reference to this FEntityBuilder for method chaining.
	 */
	template<CTag T>
	FEntityBuilder& Add();

	/**
	 * Adds a chunk fragment of type T to the entity.
	 * @return Reference to this FEntityBuilder for method chaining.
	 */
	template<CChunkFragment T>
	FEntityBuilder& Add();

	/**
	 * Adds a fragment of type T to the entity, constructing it with the provided arguments.
	 * @return Reference to this FEntityBuilder for method chaining.
	 */
	template<typename T, typename... TArgs>
	FEntityBuilder& Add(TArgs&&... InArgs) requires (CElement<T> && !(CTag<T> || CChunkFragment<T>));

	/**
	 * Adds a fragment instance to the Entity Builder, treating the contents according to its type
	 */
	UE_API FEntityBuilder& Add(const FInstancedStruct& ElementInstance);
	UE_API FEntityBuilder& Add(FInstancedStruct&& ElementInstance);

	/** Adds the ElementType to the target archetype's composition */
	UE_API FEntityBuilder& Add(TNotNull<const UScriptStruct*> ElementType);

	/** type used to store parameters for relations to be created once the target entity is created. */
	struct FPendingRelationParams
	{
		UE::Mass::FTypeHandle RelationTypeHandle;
		FMassEntityHandle OtherEntity;
		Relations::ERelationRole OtherEntityRole;
	};

	/**
	 * Adds information about a specific relation instance to be added once the entity gets created.
	 * The function will simply store the information without any checks for validity or duplication.
	 * If you want to override existing relation data call ForEachRelation.
	 */
	UE_API FEntityBuilder& AddRelation(UE::Mass::FTypeHandle RelationTypeHandle, FMassEntityHandle OtherEntity, Relations::ERelationRole InputEntityRole = Relations::ERelationRole::Object);

	/** templated helper function for calling the other AddRelation function */
	template<UE::Mass::CRelation T>
	FEntityBuilder& AddRelation(FMassEntityHandle OtherEntity, Relations::ERelationRole InputEntityRole = Relations::ERelationRole::Object);

	/**
	 * Calls the provided Operator function for every stored pending relation data instance.
	 * The return value of the Operator is used to determine whether the relation data instance
	 * should be kept (meaning: return `false` for each element you want to remove).
	 * The potential element removal is stable.
	 */
	UE_API void ForEachRelation(const TFunctionRef<bool(FPendingRelationParams&)>& Operator);

	/**
	 * Finds and retrieves a pointer to a fragment of type T if it exists.
	 * @param T - The type of fragment to find.
	 * @return Pointer to the fragment, or nullptr if it does not exist.
	 */
	template<typename T>
	T* Find() requires (CElement<T> && !(CTag<T> || CChunkFragment<T>));

	/**
	 * Advanced functionality. Can be used to provide additional parameters that will be used to
	 * create the entity's target archetype. Note that these parameters will take effect only if
	 * the target archetype doesn't exist yet.
	 */
	void ConfigureArchetypeCreation(const FMassArchetypeCreationParams& InCreationParams);

	/** Converts the builder to a FMassEntityHandle, reserving the entity handle if not already committed. */
	[[nodiscard]] UE_API FMassEntityHandle GetEntityHandle() const;

	[[nodiscard]] UE_API FMassArchetypeHandle GetArchetypeHandle();

	/** Checks whether the builder is in a valid, expected state */
	bool IsValid() const;

	/** @return whether the builder has an entity handle reserved and the data has not been committed yet */
	bool HasReservedEntityHandle() const;

	/** @return whether the builder has already committed the data */
	bool IsCommitted() const;

	/** @return the EntityManager instance this entity builder is working for */
	TSharedRef<FMassEntityManager> GetEntityManager();

	/** Retrieves the default value used to initialize EntityBuilder.bForceDeferred */
	UE_API static bool GetDefaultValueOfForceDeferredCommit();

protected:
	friend Private::FEntityBuilderHelper;

	UE_API void CacheSharedFragmentValue();
	UE_API void CacheArchetypeHandle();
	UE_API void InvalidateCachedData();

	struct FStructInstanceFindingPredicate
	{
		template<typename TInstancedStruct>
		bool operator()(const TInstancedStruct& Instance) const
		{
			return Instance.GetScriptStruct() == SearchedType;
		}
		const UScriptStruct* SearchedType = nullptr;
	};

private:
	template<typename TInstancedStruct>
	FEntityBuilder& AddInternal(TInstancedStruct&& ElementInstance);

	template<typename T>
	bool HandleTypeInstance(const UScriptStruct* Type, TArray<FInstancedStruct>*& OutTargetContainer, bool& bOutAlreadyInComposition);

	TSharedRef<FMassEntityManager> EntityManager;
	mutable FMassEntityHandle EntityHandle;

	FMassArchetypeCompositionDescriptor Composition;

	FMassArchetypeSharedFragmentValues CachedSharedFragmentValues;
	FMassArchetypeHandle CachedArchetypeHandle;

	/** stores optional FMassArchetypeCreationParams, that will be used if the target archetype doesn't exist yet */
	FMassArchetypeCreationParams ArchetypeCreationParams;

	template<CFragment T> 
	TArray<FInstancedStruct>& GetInstancedStructContainerInternal() 
	{ 
		return Fragments; 
	}

	template<CSharedFragment T> 
	TArray<FInstancedStruct>& GetInstancedStructContainerInternal() 
	{
		// Resetting the cached shared values because this function is always called 
		// with the intent to modify the contents of SharedFragments, invalidating the
		// cached data anyway 
		CachedSharedFragmentValues.Reset();
		return SharedFragments; 
	}

	template<CConstSharedFragment T> 
	TArray<FInstancedStruct>& GetInstancedStructContainerInternal() 
	{
		CachedSharedFragmentValues.Reset();
		return ConstSharedFragments; 
	}

	/** Releases reserved handle if it has not been committed yet */
	void ConditionallyReleaseEntityHandle();

	void CacheEntityHandle() const;

	static void CreateEntityImpl(FMassEntityManager& EntityManager, FMassEntityHandle EntityHandle, const FMassArchetypeHandle& ArchetypeHandle
		, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, TConstArrayView<FInstancedStruct> Fragments, TConstArrayView<FPendingRelationParams> RelationsParams
		, const FMassElementBitSet& CompositionBitSet);

	TArray<FInstancedStruct> Fragments;
	TArray<FInstancedStruct> SharedFragments;
	TArray<FInstancedStruct> ConstSharedFragments;
	TArray<FPendingRelationParams> RelationsParams;

	enum class EState : uint8
	{
		Empty,
		ReadyToCommit,
		Committed,
		Invalid,
	};
	EState State = EState::Empty;

	/** if set will make entity creation always performed with Mass commands */
	uint8 bForceDeferred : 1 = false;
};

struct FScopedEntityBuilder : FEntityBuilder
{
	UE_NONCOPYABLE(FScopedEntityBuilder);

	template<typename... TArgs>
	FScopedEntityBuilder(TArgs&&... InArgs)
		: FEntityBuilder(Forward<TArgs>(InArgs)...)
	{	
	}

	~FScopedEntityBuilder()
	{
		Commit();
	}
};

//-----------------------------------------------------------------------------
// Inlines and specializations
//-----------------------------------------------------------------------------
template<CTag T>
FEntityBuilder& FEntityBuilder::Add()
{
	Composition.Add<T>();
	State = EState::ReadyToCommit;
	CachedArchetypeHandle = FMassArchetypeHandle();
	return *this;
}

template<CChunkFragment T>
FEntityBuilder& FEntityBuilder::Add()
{
	Composition.Add<T>();
	State = EState::ReadyToCommit;
	CachedArchetypeHandle = FMassArchetypeHandle();
	return *this;
}

template<typename T, typename... TArgs>
FEntityBuilder& FEntityBuilder::Add(TArgs&&... InArgs) requires (CElement<T> && !(CTag<T> || CChunkFragment<T>))
{
	Add_GetRef<T>(InArgs...);
	return *this;
}

template<typename T, typename... TArgs>
T& FEntityBuilder::Add_GetRef(TArgs&&... InArgs) requires (CElement<T> && !(CTag<T> || CChunkFragment<T>))
{
	State = EState::ReadyToCommit;

	if (ensureMsgf(Composition.GetElementsBitSet().Contains<T>() == false, TEXT("Element of type %s has already been added"), *T::StaticStruct()->GetName()))
	{
		Composition.GetElementsBitSet().Add<T>();
		CachedArchetypeHandle = FMassArchetypeHandle();

		return GetInstancedStructContainerInternal<T>().Add_GetRef(FInstancedStruct::Make<T>(InArgs...)).template GetMutable<T>();
	}

	return GetInstancedStructContainerInternal<T>().FindByPredicate(FStructInstanceFindingPredicate{T::StaticStruct()})
		->template GetMutable<T>();
}

template<typename T, typename... TArgs>
T& FEntityBuilder::GetOrCreate(TArgs&&... InArgs) requires (CElement<T> && !(CTag<T> || CChunkFragment<T>))
{
	State = EState::ReadyToCommit;

	FInstancedStruct* ElementFound = nullptr;
	if (Composition.GetElementsBitSet().Contains<T>())
	{
		// replace
		ElementFound = GetInstancedStructContainerInternal<T>().FindByPredicate(FStructInstanceFindingPredicate{T::StaticStruct()});

		checkf(ElementFound, TEXT("We expect the element to be found since we already tested the Composition"));
		ElementFound->GetMutable<T>() = T(InArgs...);
	}
	else
	{
		Composition.GetElementsBitSet().Add<T>();
		CachedArchetypeHandle = FMassArchetypeHandle();

		ElementFound = &GetInstancedStructContainerInternal<T>().Add_GetRef(FInstancedStruct::Make<T>(InArgs...));
	}

	return ElementFound->GetMutable<T>();
}

template<typename T>
T* FEntityBuilder::Find() requires (CElement<T> && !(CTag<T> || CChunkFragment<T>))
{
	if (Composition.GetElementsBitSet().Contains<T>())
	{
		FInstancedStruct* ElementFound = GetInstancedStructContainerInternal<T>().FindByPredicate(FStructInstanceFindingPredicate{T::StaticStruct()});

		checkf(ElementFound, TEXT("We expect the element to be found since we already tested the Composition"));
		return ElementFound->GetMutablePtr<T>();
	}

	return nullptr;
}

template<UE::Mass::CRelation T>
FEntityBuilder& FEntityBuilder::AddRelation(const FMassEntityHandle OtherEntity, const Relations::ERelationRole InputEntityRole)
{
	return AddRelation(FTypeManager::MakeTypeHandle(T::StaticStruct()), OtherEntity, InputEntityRole);
}

inline bool FEntityBuilder::IsValid() const
{
	return State != EState::Invalid;
}

inline bool FEntityBuilder::HasReservedEntityHandle() const
{
	return State != EState::Committed && EntityHandle.IsValid();
}

inline bool FEntityBuilder::IsCommitted() const
{
	return State == EState::Committed;
}

inline TSharedRef<FMassEntityManager> FEntityBuilder::GetEntityManager()
{
	return EntityManager;
}

inline void FEntityBuilder::ConfigureArchetypeCreation(const FMassArchetypeCreationParams& InCreationParams)
{
	ArchetypeCreationParams = InCreationParams;
}

inline FEntityBuilder& FEntityBuilder::SetForceDeferredCommit(const bool bNewValue)
{
	bForceDeferred = bNewValue;
	return *this;
}

} // namespace UE::Mass

#undef UE_API