// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Mass/EntityHandle.h"
#include "MassArchetypeTypes.h"

namespace UE::Mass
{
	/**
	 * Type represents a collection of arbitrary EntityHandles. Under the hood the type stores also
	 * an array of FMassArchetypeEntityCollection instances. These cached collections can be tested for 
	 * being up to date, and re-created on demand, based on stored entity handles.
	 * 
	 * The type is intended to be used to collect entities available through different means: individual
	 * handles, handle arrays and or FMassArchetypeEntityCollection instances. Such accumulated handles
	 * can at any moment be turned into an array of up-to-date FMassArchetypeEntityCollection instance,
	 * which in turn is how entity sets are provided to MassEntityManager's batched API.
	 *
	 * The biggest win while using this type is that the user doesn't have to worry about FMassArchetypeEntityCollection
	 * instances going out of date (which happens whenever the target archetype is touched in a way that
	 * changes internal entity indices). The type automatically updates the collections and caches the result.
	 */
	struct FEntityCollection
	{
		FEntityCollection() = default;

		/**
		 * The following constructor are equivalent to using the default constructor and subsequently
		 * calling AppendCollection or AppendHandles.
		 */
		MASSENTITY_API explicit FEntityCollection(FMassArchetypeEntityCollection&& InEntityCollection);
		MASSENTITY_API explicit FEntityCollection(const FMassArchetypeEntityCollection& InEntityCollection);
		MASSENTITY_API explicit FEntityCollection(const TConstArrayView<FMassEntityHandle> InEntityHandles);
		MASSENTITY_API FEntityCollection(const TConstArrayView<FMassEntityHandle> InEntityHandles, FMassArchetypeEntityCollection&& InEntityCollection);

		//-----------------------------------------------------------------------------
		// mutating API
		//-----------------------------------------------------------------------------
		/**
		 * Appends Handles to stored EntityHandles. Results in marking cached FMassArchetypeEntityCollection as dirty.
		 */
		MASSENTITY_API void AppendHandles(TConstArrayView<FMassEntityHandle> Handles);

		/**
		 * Appends Handles to stored EntityHandles.
		 * The second parameter is relevant if, at the moment of calling, the cached FMassArchetypeEntityCollection instances
		 * are in sync with stored entity handles (meaning all entities stored in EntityHandles are also captured by one of
		 * FMassArchetypeEntityCollection). If that's the case then the InEntityCollection gets stored along with
		 * existing collections. Otherwise, InEntityCollection will be ignored. 
		 */
		MASSENTITY_API void AppendHandles(TConstArrayView<FMassEntityHandle> Handles, FMassArchetypeEntityCollection&& InEntityCollection);

		/**
		 * Appends Handles to stored EntityHandles. Results in marking cached FMassArchetypeEntityCollection as dirty.
		 */
		MASSENTITY_API void AppendHandles(TArray<FMassEntityHandle>&& Handles);

		/**
		 * Appends the Handle to stored EntityHandles. Results in marking cached FMassArchetypeEntityCollection as dirty.
		 */
		MASSENTITY_API void AddHandle(FMassEntityHandle Handle);

		/**
		 * Based on the provided FMassArchetypeEntityCollection creates an array of entity handles and stores them. 
		 * If up to this point the cached FMassArchetypeEntityCollection-s are consistent with stored EntityHandles
		 * then InEntityCollection gets stored as well, and stored collections are not marked as dirty.
		 */
		template<typename T> requires std::is_same_v<typename TDecay<T>::Type, FMassArchetypeEntityCollection>
		void AppendCollection(T&& InEntityCollection)
		{
			if (UNLIKELY(InEntityCollection.IsEmpty()))
			{
				return;
			}

			const bool bWasEmpty = EntityHandles.IsEmpty();
			if (InEntityCollection.ExportEntityHandles(EntityHandles))
			{
				ConditionallyStoreCollection(bWasEmpty, Forward<T>(InEntityCollection));
			}
		}

		/**
		 * Results in duplicate handles being removed from EntityHandles, the cached collections being up-to-date
		 * and CollectionCreationDuplicatesHandling being set to FMassArchetypeEntityCollection::NoDuplicates
		 * 
		 * @param bForceOperation by default the entity handles will be re-exported only if
		 *	CollectionCreationDuplicatesHandling == FMassArchetypeEntityCollection::FoldDuplicates
		 *	(which means we cannot rule out that there are duplicates). Using bForceOperation = true
		 *	will perform the operation regardless.
		 *	
		 * @return whether any duplicates were detected
		 */
		MASSENTITY_API bool UpdateAndRemoveDuplicates(const FMassEntityManager& EntityManager, bool bForceOperation = false);

		//-----------------------------------------------------------------------------
		// state-querying API
		//-----------------------------------------------------------------------------
		void MarkDirty()
		{
			CachedCollections.Reset();
		}

		bool IsEmpty() const
		{
			ensureMsgf(!(EntityHandles.Num() == 0 && CachedCollections.Num() != 0), TEXT("Stored entity array is empty while there are stored collections. This is unexpected."));
			return EntityHandles.IsEmpty();
		}

		/**
		 * Checks if cached collection data is up to date
		 * If CachedCollections are not up-to-date we reset them to cache the information (and make the subsequent tests cheaper)
		 * Note that, depending on the contents, the test might be non-trivial. Use responsibly.
		 */
		MASSENTITY_API bool IsUpToDate() const;

		//-----------------------------------------------------------------------------
		// data reading API
		//-----------------------------------------------------------------------------
		TConstArrayView<FMassEntityHandle> GetEntityHandlesView() const
		{
			return EntityHandles;
		}

		/**
		 * Retrieves the view to current contents of CachedCollections, which may be out of date.
		 * If you need valid, up-to-date collections call GetUpToDatePerArchetypeCollections instead.
		 */
		TConstArrayView<FMassArchetypeEntityCollection> GetCachedPerArchetypeCollections() const
		{
			return CachedCollections;
		}

		/**
		 * Fetches up-to-date FMassArchetypeEntityCollection instances matching stored
		 * entity handles.
		 */
		TConstArrayView<FMassArchetypeEntityCollection> GetUpToDatePerArchetypeCollections(const FMassEntityManager& EntityManager) const
		{
			ConditionallyUpdate(EntityManager);
			return CachedCollections;
		}

		/**
		 * Updates cached archetype collections and returns the container with move semantics
		 */
		TArray<FMassArchetypeEntityCollection> ConsumeArchetypeCollections(const FMassEntityManager& EntityManager) &&
		{
			ConditionallyUpdate(EntityManager);
			return MoveTemp(CachedCollections);
		}

	private:
		template<typename T> requires std::is_same_v<typename TDecay<T>::Type, FMassArchetypeEntityCollection>
		void ConditionallyStoreCollection(const bool bWasEmpty, T&& InEntityCollection)
		{
			// if there was no previous data, or the data was "complete", meaning we had collections for stored handles
			// Note: this condition only works as expected if we make sure that adding handles without associated collections
			// resets the CachedCollections array
			// @todo add unit tests to ensure this behavior
			if (bWasEmpty || !CachedCollections.IsEmpty())
			{
				if (!CachedCollections.IsEmpty() && CachedCollections.Last().IsSameArchetype(InEntityCollection))
				{
					// merge with the last collection since it's the same archetype.
					CachedCollections.Last().Append(Forward<T>(InEntityCollection));
				}
				else
				{
					CachedCollections.Emplace(Forward<T>(InEntityCollection));
				}
			}
			else
			{
				CachedCollections.Reset();
			}
		}

		MASSENTITY_API void ConditionallyUpdate(const FMassEntityManager& EntityManager) const;

		/**
		 * these are the entities represented by given instance of FEntityCollection.
		 * EntityHandles are the authority, source of truth regarding the contents
		 */
		TArray<FMassEntityHandle> EntityHandles;

		/**
		 * Cached per-archetype collections of entities. Can go our of date due to
		 * operations performed on this FEntityCollection instance (in this case we
		 * reset cached CachedCollections) or due to the stored entities being moved
		 * between archetypes.
		 */
		mutable TArray<FMassArchetypeEntityCollection> CachedCollections;

		/**
		 * Stores information whether we can expect duplicates in EntityHandles when building CachedCollections
		 */
		FMassArchetypeEntityCollection::EDuplicatesHandling CollectionCreationDuplicatesHandling = FMassArchetypeEntityCollection::NoDuplicates;
	};
} // namespace UE::Mass
