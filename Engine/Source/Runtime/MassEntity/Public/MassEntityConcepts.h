// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityElementTypes.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "Subsystems/Subsystem.h"


namespace UE::Mass
{
	template<typename T>
	using Clean = typename TRemoveReference<T>::Type;
	
	template<typename T>
	concept CFragment = TIsDerivedFrom<Clean<T>, FMassFragment>::Value &&
		(
			std::is_trivially_copyable_v<Clean<T>> ||
			static_cast<bool>(TMassFragmentTraits<Clean<T>>::AuthorAcceptsItsNotTriviallyCopyable)
		);
	
	template<typename T>
	concept CTag = TIsDerivedFrom<Clean<T>, FMassTag>::Value;

	template<typename T>
	concept CChunkFragment = TIsDerivedFrom<Clean<T>, FMassChunkFragment>::Value;

	template<typename T>
	concept CSharedFragment = TIsDerivedFrom<Clean<T>, FMassSharedFragment>::Value;

	template<typename T>
	concept CConstSharedFragment = TIsDerivedFrom<Clean<T>, FMassConstSharedFragment>::Value;

	template<typename T>
	concept CNonTag = CFragment<T> || CChunkFragment<T> || CSharedFragment<T> || CConstSharedFragment<T>;

	template<typename T>
	concept CElement = CNonTag<T> || CTag<T>;

	template<typename T>
	concept CSubsystem = TIsDerivedFrom<Clean<T>, USubsystem>::Value;

	template<typename T>
	concept CSparse = TIsDerivedFrom<Clean<T>, FMassSparseFragment>::Value || TIsDerivedFrom<Clean<T>, FMassSparseTag>::Value;

	namespace Private
	{
		template<CElement T>
		struct TElementTypeHelper
		{
			using Type = std::conditional_t<CFragment<T>, FMassFragment
				, std::conditional_t<CTag<T>, FMassTag
				, std::conditional_t<CChunkFragment<T>, FMassChunkFragment
				, std::conditional_t<CSharedFragment<T>, FMassSharedFragment
				, std::conditional_t<CConstSharedFragment<T>, FMassConstSharedFragment
				, void>>>>>;
		};
	} // namespace Private

	template<typename T>
	using TElementType = typename Private::TElementTypeHelper<T>::Type;

	template<typename T>
	bool IsSparse()
	{
		constexpr bool bIsSparse = CSparse<typename TRemoveReference<T>::Type>;
		return bIsSparse;
	}
} // namespace UE::Mass

// Provides two targeted static_assert checks instead of the single combined CFragment concept check.
// The first failing assert gives the user a clear, specific error message rather than the full
// concept failure wall. Place this macro where static_assert(UE::Mass::CFragment<T>, ...) was used.
#define MASS_STATIC_CHECK_FRAGMENT(T) \
	static_assert(TIsDerivedFrom<UE::Mass::Clean<T>, FMassFragment>::Value, \
		"Type must inherit from FMassFragment or one of its child-types."); \
	static_assert(std::is_trivially_copyable_v<UE::Mass::Clean<T>> \
		|| static_cast<bool>(TMassFragmentTraits<UE::Mass::Clean<T>>::AuthorAcceptsItsNotTriviallyCopyable), \
		"Fragment type is not trivially copyable. Heap-allocating types like FString, TArray, " \
		"TMap are forbidden in fragments. Use FName instead of FString, fixed arrays instead " \
		"of TArray. To opt out: specialize TMassFragmentTraits<YourType> with " \
		"AuthorAcceptsItsNotTriviallyCopyable = true.")
