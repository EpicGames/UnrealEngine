// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
// HEADER_UNIT_SKIP - Deprecated
#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.8, "MassEntityHandle.h has moved to the MassCore module and been renamed to Mass/EntityHandle.h. "
    "Include Mass/EntityHandle.h from MassCore and add a MassCore dependency to your .Build.cs.")
#include "Mass/EntityHandle.h"
