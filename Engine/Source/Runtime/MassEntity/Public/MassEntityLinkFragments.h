// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "Mass/EntityHandle.h"
#include "MassEntityLinkFragments.generated.h"

/**
 * Add this to an entity to link it to another entity.
 * Groups entities that access the same LinkedEntityHandle indirectly for better cache behavior.
 * Use this to minimize random access when many entities access the same shared data.
 */
USTRUCT()
struct FMassEntityLinkFragment : public FMassConstSharedFragment
{
	UPROPERTY()
	FMassEntityHandle LinkedEntityHandle;

	GENERATED_BODY()
};
