// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
// HEADER_UNIT_SKIP - Deprecated
#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.8, "MassEntityMacros.h has moved to the MassCore module and been renamed to Mass/EntityMacros.h. "
    "Include Mass/EntityMacros.h from MassCore and add a MassCore dependency to your .Build.cs.")
#include "Mass/EntityMacros.h"
