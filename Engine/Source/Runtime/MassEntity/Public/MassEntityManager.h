// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "Containers/StaticArray.h"
#include "MassEntityTypes.h"
#include "MassProcessingTypes.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "MassEntityQuery.h"
#endif
#include "MassObserverManager.h"
#include "Containers/MpscQueue.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "MassRequirementAccessDetector.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Templates/FunctionFwd.h"
#include "MassEntityManagerStorage.h"
#include "Mass/ArchetypeGroup.h"
#include "MassRelationManager.h"
#include "MassEntityRelations.h"
#include "MassSparseElementsStorage.h"

#if WITH_MASSENTITY_DEBUG
#include "Misc/EnumClassFlags.h"
#include "MassRequirementAccessDetector.h"
#else
struct FMassRequirementAccessDetector;
#endif

#define UE_API MASSENTITY_API

struct FInstancedStruct;
struct FMassEntityQuery;
struct FMassExecutionContext;
struct FMassArchetypeData;
struct FMassCommandBuffer;
struct FMassArchetypeEntityCollection;
class FOutputDevice;
struct FMassDebugger;
struct FMassFragmentRequirements;
enum class EMassFragmentAccess : uint8;
enum class EForkProcessRole : uint8;
namespace UE::Mass
{
	struct FEntityBuilder;
	struct FTypeManager;
	struct FTypeHandle;
	namespace Private
	{
		struct FEntityStorageInitializer;
	} // namespace Private
	namespace ObserverManager
	{
		struct FObserverLock;
		struct FCreationContext;
	} // namespace ObserverManager

	template<typename T>
	concept CStructType = TIsDerivedFrom<Clean<T>, FInstancedStruct>::Value
		|| TIsDerivedFrom<Clean<T>, FStructView>::Value
		|| TIsDerivedFrom<Clean<T>, FConstStructView>::Value;
} // namespace UE::Mass

// UE_DEPRECATED 5.8: REQUESTED_MASS_CONCURRENT_RESERVE and WITH_MASS_CONCURRENT_RESERVE are deprecated.
// Single threaded entity storage is being removed. Concurrent storage will always be used.
// Setting REQUESTED_MASS_CONCURRENT_RESERVE=0 is no longer supported and will be ignored in a future release.
#ifndef REQUESTED_MASS_CONCURRENT_RESERVE
#define REQUESTED_MASS_CONCURRENT_RESERVE 1
#endif

#define WITH_MASS_CONCURRENT_RESERVE (REQUESTED_MASS_CONCURRENT_RESERVE || WITH_EDITOR)

namespace UE::Mass
{
#if WITH_MASS_CONCURRENT_RESERVE
	using FStorageType = IEntityStorageInterface;
#else
	using FStorageType = FSingleThreadedEntityStorage;
#endif //WITH_MASS_CONCURRENT_RESERVE
} // namespace UE::Mass

/**
 * The type responsible for hosting Entities managing Archetypes.
 * Entities are stored as FEntityData entries in a chunked array. 
 * Each valid entity is assigned to an Archetype that stored fragments associated with a given entity at the moment. 
 * 
 * FMassEntityManager supplies API for entity creation (that can result in archetype creation) and entity manipulation.
 * Even though synchronized manipulation methods are available in most cases the entity operations are performed via a command 
 * buffer. The default command buffer can be obtained with a Defer() call. @see FMassCommandBuffer for more details.
 * 
 * FMassEntityManager are meant to be stored with a TSharedPtr or TSharedRef. Some of Mass API pass around 
 * FMassEntityManager& but programmers can always use AsShared() call to obtain a shared ref for a given manager instance 
 * (as supplied by deriving from TSharedFromThis<FMassEntityManager>).
 * IMPORTANT: if you create your own FMassEntityManager instance remember to call Initialize() before using it.
 */
struct FMassEntityManager : public TSharedFromThis<FMassEntityManager>, public FGCObject
{
	friend FMassEntityQuery;
	friend FMassDebugger;
	friend FMassExecutionContext; // Execution context needs direct access to unsafe api.

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewArchetypeDelegate, const FMassArchetypeHandle&);

private:
	// Index 0 is reserved so we can treat that index as an invalid entity handle
	constexpr static int32 NumReservedEntities = 1;
	
public:
	struct FScopedProcessing
	{
		explicit FScopedProcessing(std::atomic<int32>& InProcessingScopeCount) : ScopedProcessingCount(InProcessingScopeCount)
		{
			UE_AUTORTFM_OPEN
			{
				ScopedProcessingCount.fetch_add(1, std::memory_order_relaxed);
			};
			AutoRTFM::PushOnAbortHandler(this, [this]
				{
					ScopedProcessingCount.fetch_sub(1, std::memory_order_relaxed);
				});
		}
		~FScopedProcessing()
		{
			UE_AUTORTFM_OPEN
			{
				ScopedProcessingCount.fetch_sub(1, std::memory_order_relaxed);
			};
			AutoRTFM::PopOnAbortHandler(this);
		}
	private:
		std::atomic<int32>& ScopedProcessingCount;
	};
	using FStructInitializationCallback = TFunctionRef<void(void* Fragment, const UScriptStruct& FragmentType)>;

	const UE_API static FMassEntityHandle InvalidEntity;

	UE_API explicit FMassEntityManager(UObject* InOwner = nullptr);
	FMassEntityManager(const FMassEntityManager& Other) = delete;
	UE_API virtual ~FMassEntityManager();

	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMassEntityManager");
	}
	// End of FGCObject interface
	UE_API void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);

	// Default to use single threaded implementation
	UE_DEPRECATED(5.8, "Single threaded entity storage is being removed. Use Initialize(FMassEntityManagerStorageInitParams) with FMassEntityManager_InitParams_Concurrent instead.")
	UE_API void Initialize();
	UE_API void Initialize(const FMassEntityManagerStorageInitParams& InitializationParams);
	UE_API void PostInitialize();
	UE_API void Deinitialize();

	/** 
	 * A special, relaxed but slower version of CreateArchetype functions that allows FragmentAngTagsList to contain 
	 * both fragments and tags. 
	 */
	UE_API FMassArchetypeHandle CreateArchetype(TConstArrayView<const UScriptStruct*> FragmentsAndTagsList, const FMassArchetypeCreationParams& CreationParams = FMassArchetypeCreationParams());

	/**
	 * A special, relaxed but slower version of CreateArchetype functions that allows FragmentAngTagsList to contain
	 * both fragments and tags. This version takes an original archetype and copies it layout, then appends any fragments and tags from the
	 * provided list if they're not already in the original archetype.
	 * 
	 * @param SourceArchetype The archetype where the composition will be copied from.
	 * @param FragmentsAndTagsList The list of fragments and tags to add to the copied composition.
	 */
	UE_API FMassArchetypeHandle CreateArchetype(FMassArchetypeHandle SourceArchetype, TConstArrayView<const UScriptStruct*> FragmentsAndTagsList);
	
	/**
	 * A special, relaxed but slower version of CreateArchetype functions that allows FragmentAngTagsList to contain
	 * both fragments and tags. This version takes an original archetype and copies it layout, then appends any fragments and tags from the
	 * provided list if they're not already in the original archetype.
	 * 
	 * @param SourceArchetype The archetype where the composition will be copied from.
	 * @param FragmentsAndTagsList The list of fragments and tags to add to the copied composition.
	 * @param CreationParams Additional arguments used to create the new archetype.
	 */
	UE_API FMassArchetypeHandle CreateArchetype(FMassArchetypeHandle SourceArchetype, TConstArrayView<const UScriptStruct*> FragmentsAndTagsList, 
		const FMassArchetypeCreationParams& CreationParams);

	/**
	 * CreateArchetype from a composition descriptor and initial values
	 *
	 * @param Composition of fragment, tag and chunk fragment types
	 * @param CreationParams Parameters used during archetype construction
	 * @return a handle of a new archetype 
	 */
	UE_API FMassArchetypeHandle CreateArchetype(const FMassArchetypeCompositionDescriptor& Composition, const FMassArchetypeCreationParams& CreationParams = FMassArchetypeCreationParams());

	/**
	 * CreateArchetype from a mass element bitset and initial values
	 *
	 * @param ElementBitSet Bitset of fragment, tag and chunk fragment types
	 * @param CreationParams Parameters used during archetype construction
	 * @return a handle of a new archetype
	 */
	UE_API FMassArchetypeHandle CreateArchetype(const FMassElementBitSet& ElementBitSet, const FMassArchetypeCreationParams& CreationParams = FMassArchetypeCreationParams());

	/**
	 *  Creates an archetype like SourceArchetype + InFragments.
	 *  @param SourceArchetype the archetype used to initially populate the list of fragments of the archetype being created.
	 *  @param InFragments list of unique fragments to add to fragments fetched from SourceArchetype. Note that
	 *   adding an empty list is not supported and doing so will result in failing a `check`
	 *  @return a handle of a new archetype
	 *  @note it's caller's responsibility to ensure that NewFragmentList is not empty and contains only fragment
	 *   types that SourceArchetype doesn't already have. If the caller cannot guarantee it use of AddFragment functions
	 *   family is recommended.
	 */
	UE_API FMassArchetypeHandle CreateArchetype(const TSharedPtr<FMassArchetypeData>& SourceArchetype, const FMassFragmentBitSet& InFragments);
	
	/** 
	 *  Creates an archetype like SourceArchetype + InFragments. 
	 *  @param SourceArchetype the archetype used to initially populate the list of fragments of the archetype being created. 
	 *  @param InFragments list of unique fragments to add to fragments fetched from SourceArchetype. Note that 
	 *   adding an empty list is not supported and doing so will result in failing a `check`
	 *  @param CreationParams Parameters used during archetype construction
	 *  @return a handle of a new archetype
	 *  @note it's caller's responsibility to ensure that NewFragmentList is not empty and contains only fragment
	 *   types that SourceArchetype doesn't already have. If the caller cannot guarantee it use of AddFragment functions
	 *   family is recommended.
	 */
	UE_API FMassArchetypeHandle CreateArchetype(const TSharedPtr<FMassArchetypeData>& SourceArchetype, const FMassFragmentBitSet& InFragments, 
		const FMassArchetypeCreationParams& CreationParams);

	/** 
	 * A helper function to be used when creating entities with shared fragments provided, or when adding shared fragments
	 * to existing entities
	 * @param ArchetypeHandle that's the assumed target archetype. But we'll be making sure its composition matches SharedFragmentsBitSet
	 * @param SharedFragmentBitSet indicates which shared fragments we want the target archetype to have. If ArchetypeHandle 
	 *	doesn't have these a new archetype will be created.
	 */
	UE_API FMassArchetypeHandle GetOrCreateSuitableArchetype(const FMassArchetypeHandle& ArchetypeHandle
		, const FMassSharedFragmentBitSet& SharedFragmentBitSet
		, const FMassConstSharedFragmentBitSet& ConstSharedFragmentBitSet
		, const FMassArchetypeCreationParams& CreationParams = FMassArchetypeCreationParams());

	UE_API FMassArchetypeHandle GetOrCreateSuitableArchetype(const FMassArchetypeHandle& ArchetypeHandle
		, const FMassElementBitSet& SharedElementsBitSet
		, const FMassArchetypeCreationParams& CreationParams = FMassArchetypeCreationParams());

	/** Fetches the archetype for a given EntityHandle. If EntityHandle is not valid it will still return a handle, just with an invalid archetype */
	UE_API FMassArchetypeHandle GetArchetypeForEntity(FMassEntityHandle EntityHandle) const;
	/**
	 * Fetches the archetype for a given EntityHandle. Note that it's callers responsibility the given EntityHandle handle is valid.
	 * If you can't ensure that call GetArchetypeForEntity.
	 */
	UE_API FMassArchetypeHandle GetArchetypeForEntityUnsafe(FMassEntityHandle EntityHandle) const;

	/**
	 * Searches through all known archetypes and matches them to the provided requirements. All archetypes that pass the requirement check are returned.
	 * @param Requirements The set of fragments and tags that need to be on available in the request form on an archetype before it's added.
	 * @param OutValidArchetypes Archetypes that pass the requirements test are added here.
	 */
	UE_API void GetMatchingArchetypes(const FMassFragmentRequirements& Requirements, TArray<FMassArchetypeHandle>& OutValidArchetypes) const;

	/** 
	 * Method to iterate on all the element (fragments, tags, etc.) types of an archetype.
	 * Note that sparse elements are not archetype characteristics and therefore not included in this call.
	 */
	static UE_API void ForEachArchetypeElementType(const FMassArchetypeHandle& ArchetypeHandle, TFunctionRef< bool(const UScriptStruct* /*ElementType*/)> Function);

	/** Method to iterate on all the fragment types of an archetype */
	static UE_API void ForEachArchetypeFragmentType(const FMassArchetypeHandle& ArchetypeHandle, TFunction< void(const UScriptStruct* /*FragmentType*/)> Function);

	/** List all entities in an archetype per chunk. */
	static UE_API void ForEachArchetypeChunkEntityList(const FMassArchetypeHandle& ArchetypeHandle, TFunctionRef<void(TConstArrayView<FMassEntityHandle> /*Entities*/)> Function);

	/**
	 * Go through all archetypes and compact entities
	 * @param TimeAllowed to do entity compaction, once it reach that time it will stop and return
	 */
	UE_API void DoEntityCompaction(const double TimeAllowed);

	/**
	 * Creates fully built entity ready to be used by the subsystem
	 * @param ArchetypeHandle you want this entity to be
	 * @param SharedFragmentValues to be associated with the entity
	 * @return FMassEntityHandle id of the newly created entity */
	UE_API FMassEntityHandle CreateEntity(const FMassArchetypeHandle& ArchetypeHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues = {});

	/**
	 * Creates fully built entity ready to be used by the subsystem
	 * @param FragmentInstanceList is the fragments to create the entity from and initialize values
	 * @param SharedFragmentValues to be associated with the entity
	 * @return FMassEntityHandle id of the newly created entity */
	UE_API FMassEntityHandle CreateEntity(TConstArrayView<FInstancedStruct> FragmentInstanceList, const FMassArchetypeSharedFragmentValues& SharedFragmentValues = {}, const FMassArchetypeCreationParams& CreationParams = FMassArchetypeCreationParams());

	using FEntityCreationContext = UE::Mass::ObserverManager::FCreationContext;

	/**
	 * The main use-case for this function is to create a blank FEntityCreationContext and hold on to it while creating 
	 * a bunch of entities (with multiple calls to BatchCreate* and/or BatchBuild*) and modifying them (with mutating batched API)
	 * while not causing multiple Observers to trigger. All the observers will be triggered at one go, once the FEntityCreationContext 
	 * instance gets destroyed.
	 * 
	 * !Important note: the "Creation Context" is a specialized wrapper for an "Observers Lock" (@see GetOrMakeObserversLock).
	 * As long as the creation context is alive all the operations will be assumed to affect the newly created entities.
	 * The consequence of that is operations performed on already existing entities won't be tracked, as long
	 * as the creation context is alive.
	 * Note that you can hold a FMassObserverManager::FObserverLock instance while the creation lock gets destroyed, the
	 * observers lock is a lower-level concept than the creation context.
	 * 
	 * @return the existing (if valid) or a newly created creation context
	 */
	UE_API TSharedRef<FEntityCreationContext> GetOrMakeCreationContext();

	/**
	 * Fetches the observers lock (as hosted by FMassObserverManager). If one is not currently active,
	 * one will be created. While the lock is active all the observers notifications are suspended, and
	 * will be sent out when FMassObserverManager::FObserverLock instance gets destroyed.
	 * Locking observers needs to be used when entities are being configured with multiple operations,
	 * and we want observers to be triggered only once all the operations are executed.
	 *
	 * Note that while the observers are locked we're unable to send "Remove" notifications, so once
	 * the lock is released and the observers get notified, the data being removed won't be available anymore
	 * (which is a difference in behavior as compared to removal notifications while the observers are not locked).
	 */
	TSharedRef<UE::Mass::ObserverManager::FObserverLock> GetOrMakeObserversLock();

	/**
	 * A version of CreateEntity that's creating a number of entities (Count) in one go
	 * @param ArchetypeHandle you want this entity to be
	 * @param SharedFragmentValues to be associated with the entities
	 * @param ReservedEntities a list of reserved entities that have not yet been assigned to an archetype.
	 * @return a creation context that will notify all the interested observers about newly created fragments once the context is released */
	UE_API TSharedRef<FEntityCreationContext> BatchCreateReservedEntities(const FMassArchetypeHandle& ArchetypeHandle,
		const FMassArchetypeSharedFragmentValues& SharedFragmentValues, TConstArrayView<FMassEntityHandle> ReservedEntities);
	inline TSharedRef<FEntityCreationContext> BatchCreateReservedEntities(const FMassArchetypeHandle& ArchetypeHandle,
		TConstArrayView<FMassEntityHandle> OutEntities)
	{
		return BatchCreateReservedEntities(ArchetypeHandle, FMassArchetypeSharedFragmentValues(), OutEntities);
	}
	/**
	 * A version of CreateEntity that's creating a number of entities (Count) in one go
	 * @param ArchetypeHandle you want this entity to be
	 * @param SharedFragmentValues to be associated with the entities
	 * @param Count number of entities to create
	 * @param InOutEntities the newly created entities are appended to given array, i.e. the pre-existing content of OutEntities won't be affected by the call
	 * @return a creation context that will notify all the interested observers about newly created fragments once the context is released */
	UE_API TSharedRef<FEntityCreationContext> BatchCreateEntities(const FMassArchetypeHandle& ArchetypeHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, const int32 Count, TArray<FMassEntityHandle>& InOutEntities);
	inline TSharedRef<FEntityCreationContext> BatchCreateEntities(const FMassArchetypeHandle& ArchetypeHandle, const int32 Count, TArray<FMassEntityHandle>& InOutEntities)
	{
		return BatchCreateEntities(ArchetypeHandle, FMassArchetypeSharedFragmentValues(), Count, InOutEntities);
	}

	/**
	 * Destroys a fully built entity, use ReleaseReservedEntity if entity was not yet built.
	 * @param EntityHandle identifying the entity to destroy */
	UE_API void DestroyEntity(FMassEntityHandle EntityHandle);

	/**
	 * Reserves an entity in the subsystem, the entity is still not ready to be used by the subsystem, need to call BuildEntity()
	 * @return FMassEntityHandle id of the reserved entity */
	UE_API FMassEntityHandle ReserveEntity();

	/**
	 * Builds an entity for it to be ready to be used by the subsystem
	 * @param EntityHandle identifying the entity to build, which was retrieved with ReserveEntity() method
	 * @param ArchetypeHandle you want this entity to be
	 * @param SharedFragmentValues to be associated with the entity
	 */
	UE_API void BuildEntity(FMassEntityHandle EntityHandle, const FMassArchetypeHandle& ArchetypeHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues = {});

	/**
	 * Builds an entity for it to be ready to be used by the subsystem
	 * @param EntityHandle identifying the entity to build, which was retrieved with ReserveEntity() method
	 * @param FragmentInstanceList is the fragments to create the entity from and initialize values
	 * @param SharedFragmentValues to be associated with the entity
	 */
	UE_API void BuildEntity(FMassEntityHandle EntityHandle, TConstArrayView<FInstancedStruct> FragmentInstanceList, const FMassArchetypeSharedFragmentValues& SharedFragmentValues = {});

	/*
	 * Releases a previously reserved entity handle that was not yet built, otherwise call DestroyEntity
	 * @param EntityHandle to release */
	UE_API void ReleaseReservedEntity(FMassEntityHandle EntityHandle);

	/**
	 * Destroys all the entities in the provided array of entities. The function will also gracefully handle entities
	 * that have been reserved but not created yet.
	 * @note the function doesn't handle duplicates in InEntities.
	 * @param InEntities to destroy
	 */
	UE_API void BatchDestroyEntities(TConstArrayView<FMassEntityHandle> InEntities);

	/**
	 * Destroys all the entities provided via the Collection. The function will also gracefully handle entities
	 * that have been reserved but not created yet.
	 * @param Collection to destroy
	 */
	UE_API void BatchDestroyEntityChunks(const FMassArchetypeEntityCollection& Collection);
	UE_API void BatchDestroyEntityChunks(TConstArrayView<FMassArchetypeEntityCollection> Collections);

	/**
	 * Assigns all entities indicated by Collections to a given archetype group.
	 * Note that depending on their individual composition each entity can end up in a different archetype.
	 * @param GroupHandle indicates the target group. Passing an invalid group handle will get logged as warning and ignored.
	 */
	UE_API void BatchGroupEntities(const UE::Mass::FArchetypeGroupHandle GroupHandle, TConstArrayView<FMassArchetypeEntityCollection> Collections);

	/**
	 * Assigns all entities indicated by InEntities to a given archetype group.
	 * Note that depending on their individual composition each entity can end up in a different archetype.
	 * @param GroupHandle indicates the target group. Passing an invalid group handle will get logged as warning and ignored.
	 */
	UE_API void BatchGroupEntities(const UE::Mass::FArchetypeGroupHandle GroupHandle, TConstArrayView<FMassEntityHandle> InEntities);

	/**
	 * Fetches FArchetypeGroupType instance (copy) associated with the given GroupName. A new group type
	 * is created if GroupName has not been used in the past.
	 */
	UE_API UE::Mass::FArchetypeGroupType FindOrAddArchetypeGroupType(const FName GroupName);

	UE_API const UE::Mass::FArchetypeGroups& GetGroupsForArchetype(const FMassArchetypeHandle& ArchetypeHandle) const;

	UE_API void AddFragmentToEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> FragmentType);
	UE_API void AddFragmentToEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> FragmentType, const FStructInitializationCallback& Initializer);

	/** 
	 *  Ensures that only unique fragments are added. 
	 *  @note It's caller's responsibility to ensure EntityHandle's and FragmentList's validity. 
	 */
	UE_API void AddFragmentListToEntity(FMassEntityHandle EntityHandle, TConstArrayView<const UScriptStruct*> FragmentList);

	UE_API void AddFragmentInstanceListToEntity(FMassEntityHandle EntityHandle, TConstArrayView<FInstancedStruct> FragmentInstanceList);
	UE_API void RemoveFragmentFromEntity(FMassEntityHandle EntityHandle, const UScriptStruct* FragmentType);
	UE_API void RemoveFragmentListFromEntity(FMassEntityHandle EntityHandle, TConstArrayView<const UScriptStruct*> FragmentList);

	UE_API void AddTagToEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> TagType);
	UE_API void RemoveTagFromEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> TagType);
	UE_API void SwapTagsForEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> FromFragmentType, TNotNull<const UScriptStruct*> ToFragmentType);

	/**
	 * Adds sparse element indicated by ElementType to the entity indicated by EntityHandle.
	 * Works with tags too, but returns an empty struct view
	 * For more information on Sparse Elements see FSparseElementsStorage
	 */
	UE_API FStructView AddSparseElementToEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType);

	/**
	 * Adds a sparse fragment to the entity and returns a reference to the newly created fragments.
	 * This function can only fail if EntityHandle doesn't represent a valid, active entity.
	 */
	template<UE::Mass::CFragment T>
	requires UE::Mass::CSparse<T>
	T& AddSparseElementToEntity(FMassEntityHandle EntityHandle)
	{
		CheckIfEntityIsActive(EntityHandle);
		return AddSparseElementToEntity(EntityHandle, T::StaticStruct()).template Get<T>();
	}

	template<UE::Mass::CTag T>
	requires UE::Mass::CSparse<T>
	void AddSparseElementToEntity(FMassEntityHandle EntityHandle)
	{
		FStructView _ = AddSparseElementToEntity(EntityHandle, T::StaticStruct());
	}

	UE_API void RemoveSparseElementFromEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType);
	template<UE::Mass::CSparse T>
	void RemoveSparseElementFromEntity(FMassEntityHandle EntityHandle)
	{
		RemoveSparseElementFromEntity(EntityHandle, T::StaticStruct());
	}

	/**
	 * Adds sparse elements indicated by ElementsBitset to the entity indicated by EntityHandle.
	 * For more information on Sparse Elements see FSparseElementsStorage
	 */
	UE_API void AddSparseElementsToEntity(FMassEntityHandle EntityHandle, const FMassElementBitSet& ElementsBitset);

	/**
	 * Adds ElementType to the entities, treating it accordingly based on which element type it represents (i.e. Fragment or Tag).
	 * The function will assert on unhandled types.
	 */
	UE_API void BatchAddElementToEntities(TConstArrayView<FMassEntityHandle> Entities, TNotNull<const UScriptStruct*> ElementType);

	/**
	 * Adds ElementType to the entity, treating it accordingly based on which element type it represents (i.e. Fragment or Tag).
	 * This function handles sparse elements.
	 * The function will assert on unhandled types.
	 */
	UE_API void AddElementToEntity(FMassEntityHandle Entity, TNotNull<const UScriptStruct*> ElementType);

	/*
	 * Removes ElementType from the entities, treating it accordingly based on which element type it represents (i.e. Fragment or Tag).
	 * The function will assert on unhandled types.
	 */
	UE_API void BatchRemoveElementFromEntities(TConstArrayView<FMassEntityHandle> Entities, TNotNull<const UScriptStruct*> ElementType);

	/*
	 * Removes ElementType from the entity, treating it accordingly based on which element type it represents (i.e. Fragment or Tag).
	 * The function will assert on unhandled types.
	 */
	UE_API void RemoveElementFromEntity(FMassEntityHandle Entity, TNotNull<const UScriptStruct*> ElementType);

	/** @return whether Entity has an element of ElementType */
	UE_API bool DoesEntityHaveElement(FMassEntityHandle Entity, TNotNull<const UScriptStruct*> ElementType) const;

	template<UE::Mass::CElement T>
	bool DoesEntityHaveElement(FMassEntityHandle Entity) const
	{
		return DoesEntityHaveElement(Entity, T::StaticStruct());
	}

	/**
	 * Adds a new const shared fragment to the given entity if it doesn't have a shared fragment of the given type, 
	 * moves it to its new archetype and notify observers.
	 * In case a fragment of that type is already present, then it replaces it,
	 * moves the entity to its new chunk within the archetype and notifies the observers.
	 * @return whether the Fragment value is valid and successfully assigned to the entity (added or replaced)
	 */
	UE_API bool SwapConstSharedFragmentForEntity(const FMassEntityHandle EntityHandle, const FConstSharedStruct& InConstSharedFragment);

	/** 
	 * Adds a new const shared fragment to the given entity. Note that it only works if the given entity doesn't have
	 * a shared fragment of the given type. The function will give a soft "pass" if the entity has the shared fragment
	 * of the same value. Setting shared fragment value (i.e. changing) is not supported and the function will log
	 * a warning if that's attempted.
	 * @return whether the entity has the Fragment value assigned to it, regardless of its original state (i.e. the function will
	 *	return true also if the entity already had the same values associated with it)
	 */
	UE_API bool AddConstSharedFragmentToEntity(const FMassEntityHandle EntityHandle, const FConstSharedStruct& InConstSharedFragment);

	/**
	 * Removes a const shared fragment of the given type from the entity.
	 * Will do nothing if entity did not have the shared fragment.
	 * @return True if fragment removed from entity, false otherwise.
	 */
	UE_API bool RemoveConstSharedFragmentFromEntity(const FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ConstSharedFragmentType);

	template<UE::Mass::CConstSharedFragment T>
	bool RemoveConstSharedFragmentFromEntity(const FMassEntityHandle EntityHandle)
	{
		return RemoveConstSharedFragmentFromEntity(EntityHandle, T::StaticStruct());
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	bool RemoveConstSharedFragmentFromEntity(const FMassEntityHandle EntityHandle, const UScriptStruct& ConstSharedFragmentType)
	{
		return RemoveConstSharedFragmentFromEntity(EntityHandle, &ConstSharedFragmentType);
	}

	/**
	 * Adds a new shared fragment to the given entity. Note that it only works if the given entity doesn't have
	 * a shared fragment of the given type. The function will give a soft "pass" if the entity has the shared fragment
	 * of the same value. Setting shared fragment value (i.e. changing) is not supported and the function will log
	 * a warning if that's attempted.
	 * @return whether the entity has the Fragment value assigned to it, regardless of its original state (i.e. the function will
	 *	return true also if the entity already had the same values associated with it)
	 */
	UE_API bool AddSharedFragmentToEntity(const FMassEntityHandle EntityHandle, const FSharedStruct& InSharedFragment);

	/**
	 * Removes a shared fragment of the given type from the entity.
	 * Will do nothing if entity did not have the shared fragment.
	 * @return True if fragment removed from entity, false otherwise.
	 */
	UE_API bool RemoveSharedFragmentFromEntity(const FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> SharedFragmentType);

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	bool RemoveSharedFragmentFromEntity(const FMassEntityHandle EntityHandle, const UScriptStruct& SharedFragmentType)
	{
		return RemoveSharedFragmentFromEntity(EntityHandle, &SharedFragmentType);
	}

	/**
	 * Removes EntityHandle from any-and-all groups of given type - i.e. the entity will be moved to an archetype
	 * not in any of the groups of the given type.
	 */
	UE_API void RemoveEntityFromGroupType(FMassEntityHandle EntityHandle, UE::Mass::FArchetypeGroupType GroupType);

	/**
	 * @return the group handle of the specific group of type GroupType that the entity belongs to
	 */
	UE_API UE::Mass::FArchetypeGroupHandle GetGroupForEntity(FMassEntityHandle EntityHandle, UE::Mass::FArchetypeGroupType GroupType) const;

	/**
	 * Reserves Count number of entities and appends them to InOutEntities
	 * @return a view into InOutEntities containing only the freshly reserved entities
	 */
	UE_API TConstArrayView<FMassEntityHandle> BatchReserveEntities(const int32 Count, TArray<FMassEntityHandle>& InOutEntities);
	
	/**
	 * Reserves number of entities corresponding to number of entries in the provided array view InOutEntities.
	 * As a result InOutEntities gets filled with handles of reserved entities
	 * @return the number of entities reserved
	 */
	UE_API int32 BatchReserveEntities(TArrayView<FMassEntityHandle> InOutEntities);

	UE_API TSharedRef<FEntityCreationContext> BatchBuildEntities(
		const FMassArchetypeEntityCollectionWithPayload& EncodedEntitiesWithPayload
		, const FMassFragmentBitSet& FragmentsAffected
		, const FMassArchetypeSharedFragmentValues& SharedFragmentValues = {}
		, const FMassArchetypeCreationParams& CreationParams = FMassArchetypeCreationParams());

	UE_API TSharedRef<FEntityCreationContext> BatchBuildEntities(
		const FMassArchetypeEntityCollectionWithPayload& EncodedEntitiesWithPayload
		, const FMassArchetypeCompositionDescriptor& Composition
		, const FMassArchetypeSharedFragmentValues& SharedFragmentValues = {}
		, const FMassArchetypeCreationParams& CreationParams = FMassArchetypeCreationParams());

	UE_API TSharedRef<FEntityCreationContext> BatchBuildEntities(
		const FMassArchetypeEntityCollectionWithPayload& EncodedEntitiesWithPayload
		, const FMassElementBitSet& Composition
		, const FMassArchetypeSharedFragmentValues& SharedFragmentValues = {}
		, const FMassArchetypeCreationParams& CreationParams = FMassArchetypeCreationParams());

	UE_API void BatchChangeCompositionForEntities(
		TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
		, const FMassElementBitSet& ElementsToAdd
		, const FMassElementBitSet& ElementsToRemove);

	/**
	 * Overload that also adds shared fragment values during the same entity move.
	 * ElementsToAdd must include the shared fragment bits from SharedFragmentValuesToAdd.
	 * Sparse element types must not appear in ElementsToAdd; use the dedicated sparse element APIs instead.
	 */
	UE_API void BatchChangeCompositionForEntities(
		TConstArrayView<FMassArchetypeEntityCollection> EntityCollections,
		const FMassElementBitSet& ElementsToAdd,
		const FMassElementBitSet& ElementsToRemove,
		const FMassArchetypeSharedFragmentValues& SharedFragmentValuesToAdd);

	void BatchChangeTagsForEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, const FMassTagBitSet& TagsToAdd, const FMassTagBitSet& TagsToRemove)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchChangeTagsForEntities);
		BatchChangeCompositionForEntities(EntityCollections, TagsToAdd, TagsToRemove);
	}
	void BatchChangeFragmentCompositionForEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, const FMassFragmentBitSet& FragmentsToAdd, const FMassFragmentBitSet& FragmentsToRemove)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Mass_BatchChangeFragmentCompositionForEntities);
		BatchChangeCompositionForEntities(EntityCollections, FragmentsToAdd, FragmentsToRemove);
	}

	UE_API void BatchAddFragmentInstancesForEntities(TConstArrayView<FMassArchetypeEntityCollectionWithPayload> EntityCollections, const FMassFragmentBitSet& FragmentsAffected);

	/**
	 * Adds fragment types with per-entity values AND tags to entities in a single entity move.
	 * Tags are included in the composition change but excluded from value payload (tags have zero-size storage).
	 */
	UE_API void BatchAddFragmentInstancesForEntities(TConstArrayView<FMassArchetypeEntityCollectionWithPayload> EntityCollections, const FMassFragmentBitSet& FragmentsAffected, const FMassTagBitSet& TagsToAdd);

	/**
	 * Adds fragment types with per-entity values, tags, and shared fragment values — all in a single entity move.
	 * Tags are included in composition but excluded from value payload. Shared fragment values are set per-chunk.
	 */
	UE_API void BatchAddFragmentInstancesForEntities(TConstArrayView<FMassArchetypeEntityCollectionWithPayload> EntityCollections, const FMassFragmentBitSet& FragmentsAffected, const FMassTagBitSet& TagsToAdd, const FMassArchetypeSharedFragmentValues& SharedFragmentValuesToAdd);

	/**
	 * Adds fragment types, sets their per-entity values, and adds shared fragment values — all in a single entity move.
	 * Combines the functionality of BatchAddFragmentInstancesForEntities and BatchAddSharedFragmentsForEntities.
	 */
	UE_API void BatchAddFragmentInstancesForEntities(TConstArrayView<FMassArchetypeEntityCollectionWithPayload> EntityCollections, const FMassFragmentBitSet& FragmentsAffected, const FMassArchetypeSharedFragmentValues& SharedFragmentValuesToAdd);

	/**
	 * Adds a new const and non-const shared fragments to all entities provided via EntityCollections
	 */
	UE_API void BatchAddSharedFragmentsForEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, const FMassArchetypeSharedFragmentValues& AddedFragmentValues);

	UE_API void BatchAddSparseElementToEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TNotNull<const UScriptStruct*> ElementType);
	// @todo add a version with FMassArchetypeEntityCollectionWithPayload to add pre-configured fragment instances
	UE_API void BatchRemoveSparseElementFromEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TNotNull<const UScriptStruct*> ElementType);

	/**
	 * Removes the specified shared fragment types from all entities in the provided collections.
	 * Entities are moved to new archetypes without the removed shared fragment types.
	 * @param EntityCollections the entity collections to process
	 * @param ToRemove bitset of shared fragment types to remove
	 */
	UE_API void BatchRemoveSharedFragmentFromEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, const FMassSharedFragmentBitSet& ToRemove);

	/**
	 * Removes the specified const shared fragment types from all entities in the provided collections.
	 * Entities are moved to new archetypes without the removed shared fragment types.
	 * @param EntityCollections the entity collections to process
	 * @param ToRemove bitset of const shared fragment types to remove
	 */
	UE_API void BatchRemoveConstSharedFragmentFromEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, const FMassConstSharedFragmentBitSet& ToRemove);

	/**
	 * Adds elements indicated by InOutDescriptor to the entity indicated by EntityHandle. The function also figures out which elements
	 * in InOutDescriptor are missing from the current composition of the given entity and then returns the resulting 
	 * delta via InOutDescriptor.
	 * If InOutDescriptor indicates shared fragments to be added the caller is required to provide matching values for the indicated
	 * shared fragment types, via AddedSharedFragmentValues.
	 */
	UE_API void AddCompositionToEntity_GetDelta(FMassEntityHandle EntityHandle, FMassArchetypeCompositionDescriptor& InOutDescriptor, const FMassArchetypeSharedFragmentValues* AddedSharedFragmentValues = nullptr);
	UE_API void RemoveCompositionFromEntity(FMassEntityHandle EntityHandle, const FMassArchetypeCompositionDescriptor& InDescriptor);

	UE_API FMassArchetypeCompositionDescriptor GetArchetypeComposition(const FMassArchetypeHandle& ArchetypeHandle) const;

	/** 
	 * Moves an entity over to a new archetype by copying over fragments common to both archetypes
	 * @param EntityHandle indicates the entity to move 
	 * @param NewArchetypeHandle the handle to the new archetype
	 * @param SharedFragmentValuesOverride if provided will override all given entity's shared fragment values
	 */
	UE_API void MoveEntityToAnotherArchetype(FMassEntityHandle EntityHandle, FMassArchetypeHandle NewArchetypeHandle, const FMassArchetypeSharedFragmentValues* SharedFragmentValuesOverride = nullptr);

	/** 
	 *  Copies values from FragmentInstanceList over to target entity's fragment. Caller is responsible for ensuring that 
	 *  the given entity does have given fragments. Failing this assumption will cause a check-fail.
	 *  @param EntityHandle indicates the target entity
	 */
	void SetEntityFragmentValues(FMassEntityHandle EntityHandle, TArrayView<const FInstancedStruct> FragmentInstanceList);

	template<UE::Mass::CStructType TStruct>
	void SetEntityFragmentValues(FMassEntityHandle EntityHandle, TConstArrayView<TStruct> FragmentInstanceList);

	/** Copies values from FragmentInstanceList over to fragments of given entities collection. The caller is responsible 
	 *  for ensuring that the given entity archetype (FMassArchetypeEntityCollection .Archetype) does have given fragments. 
	 *  Failing this assumption will cause a check-fail. */
	void BatchSetEntityFragmentValues(const FMassArchetypeEntityCollection& SparseEntities, TArrayView<const FInstancedStruct> FragmentInstanceList);

	template<UE::Mass::CStructType TStruct>
	void BatchSetEntityFragmentValues(const FMassArchetypeEntityCollection& SparseEntities, TConstArrayView<TStruct> FragmentInstanceList);

	void BatchSetEntityFragmentValues(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TArrayView<const FInstancedStruct> FragmentInstanceList);

	template<UE::Mass::CStructType TStruct>
	void BatchSetEntityFragmentValues(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TConstArrayView<TStruct> FragmentInstanceList);

	/**
	 * @return whether the given handle represents a valid and built entity
	 *	 (i.e., the handle is valid and the entity represent has been constructed already)
	 */
	UE_API bool IsEntityActive(FMassEntityHandle EntityHandle) const;

	/**
	 * @return whether the given entity handle is valid, i.e. it points
	 *	 to a valid spot in the entity storage and the handle's serial number is up to date
	 */
	UE_API bool IsEntityValid(FMassEntityHandle EntityHandle) const;

	/** whether the entity handle represents an entity that has been fully built (expecting a valid EntityHandle) */
	UE_API bool IsEntityBuilt(FMassEntityHandle EntityHandle) const;

	/**
	 * @return whether the given EntityHandle is valid and the entity it represents is in `Reserved` state
	 *	 (i.e. it will also fail if the entity has already been `Created`)
	 */
	UE_API bool IsEntityReserved(FMassEntityHandle EntityHandle) const;

	/** Asserts that IsEntityValid */
	inline void CheckIfEntityIsValid(FMassEntityHandle EntityHandle) const
	{
		checkf(IsEntityValid(EntityHandle), TEXT("Invalid entity (ID: %d, SN:%d, %s)"), EntityHandle.Index, EntityHandle.SerialNumber,
			   (EntityHandle.Index == 0) ? TEXT("was never initialized") : TEXT("already destroyed"));
	}

	/** Asserts that IsEntityBuilt */
	inline void CheckIfEntityIsActive(FMassEntityHandle EntityHandle) const
	{
		checkf(IsEntityBuilt(EntityHandle), TEXT("Entity not yet created(ID: %d, SN:%d)"), EntityHandle.Index, EntityHandle.SerialNumber);
	}

	/**
	 * Generate valid, up-to-date entity handle for the entity at given index.
	 */
	UE_API FMassEntityHandle CreateEntityIndexHandle(const int32 EntityIndex) const;

	template<typename FragmentType>
	FragmentType& GetFragmentDataChecked(FMassEntityHandle EntityHandle) const
	{
		MASS_STATIC_CHECK_FRAGMENT(FragmentType);
		return *((FragmentType*)InternalGetFragmentDataChecked(EntityHandle, FragmentType::StaticStruct()));
	}

	template<typename FragmentType>
	FragmentType* GetFragmentDataPtr(FMassEntityHandle EntityHandle) const
	{
		MASS_STATIC_CHECK_FRAGMENT(FragmentType);
		return (FragmentType*)InternalGetFragmentDataPtr(EntityHandle, FragmentType::StaticStruct());
	}

	/** Returns a FStructView for the given fragment on the specified entity. The view points
	 *  directly into archetype chunk storage.
	 *  WARNING: For initializing fragments on newly created entities, prefer the
	 *  CreateEntity(TConstArrayView<FInstancedStruct>,...) overload which sets initial values
	 *  atomically. The two-step pattern (CreateEntity + GetFragmentDataStruct + mutate) is only
	 *  safe when no processors are running (e.g., during BeginPlay or subsystem Initialize).
	 *  @return an empty FStructView if the entity does not have the fragment. */
	FStructView GetFragmentDataStruct(FMassEntityHandle EntityHandle, const UScriptStruct* FragmentType) const
	{
		checkf(UE::Mass::IsA<FMassFragment>(FragmentType)
			, TEXT("GetFragmentDataStruct called with an invalid fragment type '%s'"), *GetPathNameSafe(FragmentType));
		return FStructView(FragmentType, static_cast<uint8*>(InternalGetFragmentDataPtr(EntityHandle, FragmentType)));
	}

	template<typename ConstSharedFragmentType>
	ConstSharedFragmentType* GetConstSharedFragmentDataPtr(FMassEntityHandle EntityHandle) const
	{
		static_assert(UE::Mass::CConstSharedFragment<ConstSharedFragmentType>, "Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.");
		const FConstSharedStruct* ConstSharedStruct = InternalGetConstSharedFragmentPtr(EntityHandle, ConstSharedFragmentType::StaticStruct());
		return (ConstSharedFragmentType*)(ConstSharedStruct ? ConstSharedStruct->GetMemory() : nullptr);
	}

	template<typename ConstSharedFragmentType>
	ConstSharedFragmentType& GetConstSharedFragmentDataChecked(FMassEntityHandle EntityHandle) const
	{
		ConstSharedFragmentType* TypePtr = GetConstSharedFragmentDataPtr<ConstSharedFragmentType>(EntityHandle);
		check(TypePtr);
		return *TypePtr;
	}

	FConstStructView GetConstSharedFragmentDataStruct(FMassEntityHandle EntityHandle, const UScriptStruct* ConstSharedFragmentType) const
	{
		checkf(UE::Mass::IsA<FMassConstSharedFragment>(ConstSharedFragmentType)
			, TEXT("GetConstSharedFragmentDataStruct called with an invalid fragment type '%s'"), *GetPathNameSafe(ConstSharedFragmentType));
		const FConstSharedStruct* ConstSharedStruct = InternalGetConstSharedFragmentPtr(EntityHandle, ConstSharedFragmentType);
		return ConstSharedStruct
			? FConstStructView(*ConstSharedStruct)
			: FConstStructView();
	}

	template<typename SharedFragmentType>
	TConstArrayView<FSharedStruct> GetSharedFragmentsOfType()
	{
		static_assert(UE::Mass::CSharedFragment<SharedFragmentType>
			, "Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types.");

		TArray<FSharedStruct>* InstancesOfType = SharedFragmentsContainer.Find(SharedFragmentType::StaticStruct());
		return InstancesOfType ? *InstancesOfType : TConstArrayView<FSharedStruct>();
	}

	template<typename SharedFragmentType>
	SharedFragmentType* GetSharedFragmentDataPtr(FMassEntityHandle EntityHandle) const
	{
		static_assert(UE::Mass::CSharedFragment<SharedFragmentType>
			, "Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types.");
		const FSharedStruct* FragmentPtr = InternalGetSharedFragmentPtr(EntityHandle, SharedFragmentType::StaticStruct());
		return (SharedFragmentType*)(FragmentPtr ? FragmentPtr->GetMemory() : nullptr);
	}

	template<typename SharedFragmentType>
	SharedFragmentType& GetSharedFragmentDataChecked(FMassEntityHandle EntityHandle) const
	{
		SharedFragmentType* TypePtr = GetSharedFragmentDataPtr<SharedFragmentType>(EntityHandle);
		check(TypePtr);
		return *TypePtr;
	}

	FConstStructView GetSharedFragmentDataStruct(FMassEntityHandle EntityHandle, const UScriptStruct* SharedFragmentType) const
	{
		checkf(UE::Mass::IsA<FMassSharedFragment>(SharedFragmentType)
			, TEXT("GetSharedFragmentDataStruct called with an invalid fragment type '%s'"), *GetPathNameSafe(SharedFragmentType));
		const FSharedStruct* FragmentPtr = InternalGetSharedFragmentPtr(EntityHandle, SharedFragmentType);
		return FragmentPtr
			? FConstStructView(*FragmentPtr)
			: FConstStructView();
	}

	template<typename T>
	FConstStructView GetElementDataStruct(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> FragmentType) const
	{
		if constexpr (std::is_same_v<T, FMassFragment>)
		{
			return GetFragmentDataStruct(EntityHandle, FragmentType);
		}
		else if constexpr (std::is_same_v<T, FMassSharedFragment>)
		{
			return GetSharedFragmentDataStruct(EntityHandle, FragmentType);
		}
		else if constexpr (std::is_same_v<T, FMassConstSharedFragment>)
		{
			return GetConstSharedFragmentDataStruct(EntityHandle, FragmentType);
		}
		else
		{
			static_assert(UE::Mass::TAlwaysFalse<T>, "Unsupported element type passed to GetElementDataStruct");
			return {};
		}
	}

	UE_API FStructView GetMutableSparseElementDataForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle);
	UE_API FConstStructView GetSparseElementDataForEntity(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle) const;

	FMassElementBitSet GetSparseElementsBitSetForEntity(const FMassEntityHandle EntityHandle) const;

	uint32 GetArchetypeDataVersion() const
	{
		return ArchetypeDataVersion;
	}

	/**
	 * Creates and initializes a FMassExecutionContext instance.
	 */
	UE_API FMassExecutionContext CreateExecutionContext(const float DeltaSeconds);

	FScopedProcessing NewProcessingScope()
	{
		return FScopedProcessing(ProcessingScopeCount);
	}

	/** 
	 * Indicates whether there are processors out there performing operations on this instance of MassEntityManager. 
	 * Used to ensure that mutating operations (like entity destruction) are not performed while processors are running, 
	 * which rely on the assumption that the data layout doesn't change during calculations. 
	 */
	bool IsProcessing() const
	{
		return ProcessingScopeCount > 0;
	}

	FMassCommandBuffer& Defer() const
	{
		return *DeferredCommandBuffers[OpenedCommandBufferIndex].Get();
	}
	/** 
	 * @param InCommandBuffer if not set then the default command buffer will be flushed. If set and there's already 
	 *		a command buffer being flushed (be it the main one or a previously requested one) then this command buffer 
	 *		will be queued itself.
	 */
	UE_API void FlushCommands(const TSharedPtr<FMassCommandBuffer>& InCommandBuffer);

	UE_API void FlushCommands();

	/** 
	 * Depending on the current state of Manager's command buffer the function will either move all the commands out of 
	 * InOutCommandBuffer into the main command buffer or append it to the list of command buffers waiting to be flushed.
	 * @note as a consequence of the call InOutCommandBuffer can get its contents emptied due some of the underlying code using Move semantics
	 */
	UE_API void AppendCommands(const TSharedPtr<FMassCommandBuffer>& InOutCommandBuffer);

private:
	template<typename T, typename... TArgs>
	const FSharedStruct& GetOrCreateSharedFragmentByHash(const uint32 Hash, TArgs&&... InArgs)
	{
		static_assert(UE::Mass::CSharedFragment<T>, "Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types.");
		return SharedFragmentsContainer.FindOrAdd<T>(Hash, Forward<TArgs>(InArgs)...);
	}

	template<typename T>
	const FConstSharedStruct& GetOrCreateConstSharedFragmentByHash(const uint32 Hash, const T& Fragment)
	{
		static_assert(UE::Mass::CConstSharedFragment<T>, "Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.");
		return ConstSharedFragmentsContainer.FindOrAdd(Hash, T::StaticStruct(), reinterpret_cast<const uint8*>(&Fragment));
	}

	template<typename T>
	const FSharedStruct& GetOrCreateSharedFragmentByHash(const uint32 Hash, const T& Fragment)
	{
		static_assert(UE::Mass::CSharedFragment<T>, "Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types.");
		return SharedFragmentsContainer.FindOrAdd(Hash, T::StaticStruct(), reinterpret_cast<const uint8*>(&Fragment));
	}

	const FConstSharedStruct& GetOrCreateConstSharedFragmentByHash(const uint32 Hash, const UScriptStruct* InScriptStruct, const uint8* InStructMemory)
	{
		return ConstSharedFragmentsContainer.FindOrAdd(Hash, InScriptStruct, InStructMemory);
	}

	const FSharedStruct& GetOrCreateSharedFragmentByHash(const uint32 Hash, const UScriptStruct* InScriptStruct, const uint8* InStructMemory)
	{
		return SharedFragmentsContainer.FindOrAdd(Hash, InScriptStruct, InStructMemory);
	}

	template<typename T, typename... TArgs>
	const FConstSharedStruct& GetOrCreateConstSharedFragmentByHash(const uint32 Hash, TArgs&&... InArgs)
	{
		static_assert(UE::Mass::CConstSharedFragment<T>, "Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.");
		return ConstSharedFragmentsContainer.FindOrAdd<T>(Hash, Forward<TArgs>(InArgs)...);
	}

	/**
	 * Creates fully built entity ready to be used by the subsystem
	 * Does not check for synchronous execution and should only be used by processors through the execution context
	 * This should only ever be used by FMassExecutionContext which will validate data access safety.
	 * @param ArchetypeHandle the archetype you want this entity to be
	 * @param SharedFragmentValues to be associated with the entity
	 * @return FMassEntityHandle id of the newly created entity
	 */
	UE_API FMassEntityHandle CreateEntityAsyncUnsafe(const FMassArchetypeHandle& ArchetypeHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues = {});
	
	/**
	 * Fully buils an entity with a reserved handle ready to be used by the subsystem
	 * Does not check for synchronous execution and should only be used by processors through the execution context
	 * This should only ever be used by FMassExecutionContext which will validate data access safety.
	 * @param ArchetypeHandle the archetype you want this entity to be
	 * @param ReservedEntityHandle the already reserved handle to use for the new enetity
	 * @param SharedFragmentValues to be associated with the entity
	 */
	UE_API void BuildEntityAsyncUnsafe(const FMassArchetypeHandle& ArchetypeHandle, const FMassEntityHandle& ReservedEntityHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues = {});

	/**
	 * A version of CreateEntity that's creating a number of entities (Count) in one go
	 * Unsafe version only for use in processors via ExecutionContext
	 * Note: this does not create or return a FEntityCreationContext because it's assumed that observers are already locked by the processing phase.
	 * @param ArchetypeHandle you want this entity to be
	 * @param SharedFragmentValues to be associated with the entities
	 * @param Count number of entities to create
	 * @param InOutEntities the newly created entities are appended to given array, i.e. the pre-existing content of OutEntities won't be affected by the call
	 */
	UE_API void BatchCreateEntitiesAsyncUnsafe(const FMassArchetypeHandle& ArchetypeHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, const int32 Count, TArray<FMassEntityHandle>& InOutEntities);
	void BatchCreateEntitiesAsyncUnsafe(const FMassArchetypeHandle& ArchetypeHandle, const int32 Count, TArray<FMassEntityHandle>& InOutEntities)
	{
		BatchCreateEntitiesAsyncUnsafe(ArchetypeHandle, FMassArchetypeSharedFragmentValues(), Count, InOutEntities);
	}

public:

#if WITH_EDITOR || WITH_MASSENTITY_DEBUG
	template<typename CallableT>
	void ForEachArchetype(int32 BeginRange, int32 EndRange, const CallableT& Callable) const
	{
		if (EndRange > AllArchetypes.Num())
		{
			EndRange = AllArchetypes.Num();
		}
		for (int32 Cursor = BeginRange; Cursor < EndRange; ++Cursor)
		{
			FMassArchetypeHandle Handle(AllArchetypes[Cursor]);
			Callable(*this, Handle);
		}
	}
#endif

	//-----------------------------------------------------------------------------
	// Shared fragments
	//-----------------------------------------------------------------------------
	/**
	 * Returns or creates a shared struct associated to a given shared fragment set of values
	 * identified internally by a CRC.
	 * Use this overload when an instance of the desired const shared fragment type is available and
	 * that can be used directly to compute a CRC (i.e., UE::StructUtils::GetStructInstanceCrc32)
	 *	e.g.,
	 *	USTRUCT()
	 *	struct FIntConstSharedFragment : public FMassConstSharedFragment
	 *	{
	 *		GENERATED_BODY()
	 *
	 *		UPROPERTY()
	 *		int32 Value = 0;
	 *	};
	 *
	 *	FIntConstSharedFragment Fragment;
	 *	Fragment.Value = 123;
	 *	const FConstSharedStruct SharedStruct = EntityManager.GetOrCreateConstSharedFragment(Fragment);
	 *
	 * @params Fragment Instance of the desired fragment type
	 * @return FConstSharedStruct to the matching, or newly created shared fragment
	 */
	template<typename T>
	const FConstSharedStruct& GetOrCreateConstSharedFragment(const T& Fragment)
	{
		static_assert(UE::Mass::CConstSharedFragment<T>,
			"Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.");
		const uint32 Hash = UE::StructUtils::GetStructInstanceCrc32(FConstStructView::Make(Fragment));
		return GetOrCreateConstSharedFragmentByHash(Hash, Fragment);
	}

	/**
	 * Returns or creates a shared struct associated to a given shared fragment set of values
	 * identified internally by a CRC.
	 * Use this overload when an instance of the desired shared fragment type is available and
	 * that can be used directly to compute a CRC (i.e., UE::StructUtils::GetStructInstanceCrc32)
	 *	e.g.,
	 *	USTRUCT()
	 *	struct FIntSharedFragment : public FMassSharedFragment
	 *	{
	 *		GENERATED_BODY()
	 *
	 *		UPROPERTY()
	 *		int32 Value = 0;
	 *	};
	 *
	 *	FIntSharedFragment Fragment;
	 *	Fragment.Value = 123;
	 *	const FSharedStruct SharedStruct = EntityManager.GetOrCreateSharedFragment(Fragment);
	 *
	 * @params Fragment Instance of the desired fragment type
	 * @return FSharedStruct to the matching, or newly created shared fragment
	 */
	template<typename T>
	const FSharedStruct& GetOrCreateSharedFragment(const T& Fragment)
	{
		static_assert(UE::Mass::CSharedFragment<T>,
			"Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types.");
		const uint32 Hash = UE::StructUtils::GetStructInstanceCrc32(FConstStructView::Make(Fragment));
		return GetOrCreateSharedFragmentByHash(Hash, Fragment);
	}

	/**
	 * Returns or creates a shared struct associated to a given shared fragment set of values
	 * identified internally by a CRC.
	 * Use this overload when values can be provided as constructor arguments for the desired const shared fragment type and
	 * that can be used directly to compute a CRC (i.e., UE::StructUtils::GetStructInstanceCrc32)
 	 *	e.g.,
	 *	USTRUCT()
	 *	struct FIntConstSharedFragment : public FMassConstSharedFragment
	 *	{
	 *		GENERATED_BODY()
	 *
	 *		FIntConstSharedFragment(const int32 InValue) : Value(InValue) {}
	 *
	 *		UPROPERTY()
	 *		int32 Value = 0;
	 *	};
	 *
	 *	const FConstSharedStruct SharedStruct = EntityManager.GetOrCreateConstSharedFragment<FIntConstSharedFragment>(123);
	 *
	 * @params InArgs List of arguments provided to the constructor of the desired fragment type
	 * @return FConstSharedStruct to the matching, or newly created shared fragment
	 */
	template<typename T, typename... TArgs>
	const FConstSharedStruct& GetOrCreateConstSharedFragment(TArgs&&... InArgs)
	{
		static_assert(UE::Mass::CConstSharedFragment<T>,
			"Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.");
		T Struct(Forward<TArgs>(InArgs)...);
		const uint32 Hash = UE::StructUtils::GetStructInstanceCrc32(FConstStructView::Make(Struct));
		return GetOrCreateConstSharedFragmentByHash(Hash, MoveTemp(Struct));
	}

	/**
	 * Returns or creates a shared struct associated to a given shared fragment set of values
	 * identified internally by a CRC.
	 * Use this overload when values can be provided as constructor arguments for the desired shared fragment type and
	 * that can be used directly to compute a CRC (i.e., UE::StructUtils::GetStructInstanceCrc32)
 	 *	e.g.,
	 *	USTRUCT()
	 *	struct FIntSharedFragment : public FMassSharedFragment
	 *	{
	 *		GENERATED_BODY()
	 *
	 *		FIntSharedFragment(const int32 InValue) : Value(InValue) {}
	 *
	 *		UPROPERTY()
	 *		int32 Value = 0;
	 *	};
	 *
	 *	const FSharedStruct SharedStruct = EntityManager.GetOrCreateSharedFragment<FIntSharedFragment>(123);
	 *
	 * @params InArgs List of arguments provided to the constructor of the desired fragment type
	 * @return FSharedStruct to the matching, or newly created shared fragment
	 */
	template<typename T, typename... TArgs>
	const FSharedStruct& GetOrCreateSharedFragment(TArgs&&... InArgs)
	{
		static_assert(UE::Mass::CSharedFragment<T>,
			"Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types.");
		T Struct(Forward<TArgs>(InArgs)...);
		const uint32 Hash = UE::StructUtils::GetStructInstanceCrc32(FConstStructView::Make(Struct));
		return GetOrCreateSharedFragmentByHash(Hash, MoveTemp(Struct));
	}

	/**
	 * Returns or creates a shared struct associated to a given shared fragment set of values
	 * identified internally by a CRC.
	 * Use this overload when the reflection data and the memory of an instance of the desired const shared fragment type
	 * is available and that can be used directly to compute a CRC (i.e., UE::StructUtils::GetStructInstanceCrc32)
	 * e.g.,
	 * FSharedStruct SharedStruct = EntityManager.GetOrCreateConstSharedFragment(*StructView.GetScriptStruct(), StructView.GetMemory());
	 *
	 * @params InScriptStruct Reflection data structure associated to the desired fragment type
	 * @params InStructMemory Actual data of the desired fragment type 
	 * @return FConstSharedStruct to the matching, or newly created shared fragment
	 */
	const FConstSharedStruct& GetOrCreateConstSharedFragment(const UScriptStruct& InScriptStruct, const uint8* InStructMemory)
	{
		checkf(InScriptStruct.IsChildOf(TBaseStructure<FMassConstSharedFragment>::Get()),
			TEXT("Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types."));
		const uint32 Hash = UE::StructUtils::GetStructInstanceCrc32(InScriptStruct, InStructMemory);
		return GetOrCreateConstSharedFragmentByHash(Hash, &InScriptStruct, InStructMemory);
	}

	/**
	 * Returns or creates a shared struct associated to a given shared fragment set of values
	 * identified internally by a CRC.
	 * Use this overload when the reflection data and the memory of an instance of the desired shared fragment type
	 * is available and that can be used directly to compute a CRC (i.e., UE::StructUtils::GetStructInstanceCrc32)
	 * e.g.,
	 * FSharedStruct SharedStruct = EntityManager.GetOrCreateSharedFragment(*StructView.GetScriptStruct(), StructView.GetMemory());
	 *
	 * @params InScriptStruct Reflection data structure associated to the desired fragment type
	 * @params InStructMemory Actual data of the desired fragment type 
	 * @return FSharedStruct to the matching, or newly created shared fragment
	 */
	const FSharedStruct& GetOrCreateSharedFragment(const UScriptStruct& InScriptStruct, const uint8* InStructMemory)
	{
		checkf(InScriptStruct.IsChildOf(TBaseStructure<FMassSharedFragment>::Get()),
			TEXT("Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types."));
		const uint32 Hash = UE::StructUtils::GetStructInstanceCrc32(InScriptStruct, InStructMemory);
		return GetOrCreateSharedFragmentByHash(Hash, &InScriptStruct, InStructMemory);
	}

	/**
	 * Returns or creates a shared struct associated to a given shared fragment set of values
	 * identified internally by a CRC.
	 * Use this overload when a different struct should be used to compute a CRC (i.e., UE::StructUtils::GetStructInstanceCrc32)
	 * and values can be provided as constructor arguments for the desired const shared fragment type
	 *	e.g.,
	 *
	 *	USTRUCT()
	 *	struct FIntConstSharedFragmentParams
	 *	{
	 *		GENERATED_BODY()
	 *
	 *		FIntConstSharedFragmentParams(const int32 InValue) : Value(InValue) {}
	 *
	 *		UPROPERTY()
	 *		int32 Value = 0;
	 *	};
	 *
	 *	USTRUCT()
	 *	struct FIntConstSharedFragment : public FMassConstSharedFragment
	 *	{
	 *		GENERATED_BODY()
	 *
	 *		FIntConstSharedFragment(const FIntConstSharedFragmentParams& InParams) : Value(InParams.Value) {}
	 *
	 *		int32 Value = 0;
	 *	};
	 *
	 *	FIntConstSharedFragmentParams Params(123);
	 *	const FConstSharedStruct SharedStruct = EntityManager.GetOrCreateConstSharedFragment<FIntConstSharedFragment>(FConstStructView::Make(Params), Params);
	 *
	 * @params HashingHelperStruct Struct view passed to UE::StructUtils::GetStructInstanceCrc32 to compute the CRC
	 * @params InArgs List of arguments provided to the constructor of the desired fragment type
	 * @return FConstSharedStruct to the matching, or newly created shared fragment
	 */
	template<typename T, typename... TArgs>
	const FConstSharedStruct& GetOrCreateConstSharedFragment(const FConstStructView HashingHelperStruct, TArgs&&... InArgs)
	{
		static_assert(UE::Mass::CConstSharedFragment<T>,
			"Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.");
		T Fragment(Forward<TArgs>(InArgs)...);
		const uint32 Hash = UE::StructUtils::GetStructInstanceCrc32(HashingHelperStruct);
		return GetOrCreateConstSharedFragmentByHash(Hash, MoveTemp(Fragment));
	}

	/**
	 * Returns or creates a shared struct associated to a given shared fragment set of values
	 * identified internally by a CRC.
	 * Use this overload when a different struct should be used to compute a CRC (i.e., UE::StructUtils::GetStructInstanceCrc32)
	 * and values can be provided as constructor arguments for the desired shared fragment type
	 *	e.g.,
	 *
	 *	USTRUCT()
	 *	struct FIntSharedFragmentParams
	 *	{
	 *		GENERATED_BODY()
	 *
	 *		FInSharedFragmentParams(const int32 InValue) : Value(InValue) {}
	 *
	 *		UPROPERTY()
	 *		int32 Value = 0;
	 *	};
	 *
	 *	USTRUCT()
	 *	struct FIntSharedFragment : public FMassSharedFragment
	 *	{
	 *		GENERATED_BODY()
	 *
	 *		FIntSharedFragment(const FIntConstSharedFragmentParams& InParams) : Value(InParams.Value) {}
	 *
	 *		int32 Value = 0;
	 *	};
	 *
	 *	FIntSharedFragmentParams Params(123);
	 *	const FSharedStruct SharedStruct = EntityManager.GetOrCreateSharedFragment<FIntSharedFragment>(FConstStructView::Make(Params), Params);
	 *
	 * @params HashingHelperStruct Struct view passed to UE::StructUtils::GetStructInstanceCrc32 to compute the CRC
	 * @params InArgs List of arguments provided to the constructor of the desired fragment type
	 * @return FSharedStruct to the matching, or newly created shared fragment
	 */
	template<typename T, typename... TArgs>
	const FSharedStruct& GetOrCreateSharedFragment(const FConstStructView HashingHelperStruct, TArgs&&... InArgs)
	{
		static_assert(UE::Mass::CSharedFragment<T>,
			"Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types.");
		T Fragment(Forward<TArgs>(InArgs)...);
		const uint32 Hash = UE::StructUtils::GetStructInstanceCrc32(HashingHelperStruct);
		return GetOrCreateSharedFragmentByHash(Hash, MoveTemp(Fragment));
	}

	template<UE::Mass::CSharedFragment T>
	void ForEachSharedFragment(TFunctionRef< void(T& /*SharedFragment*/) > ExecuteFunction);

	template<UE::Mass::CSharedFragment T>
	void ForEachSharedFragmentConditional(TFunctionRef< bool(T& /*SharedFragment*/) > ConditionFunction, TFunctionRef< void(T& /*SharedFragment*/) > ExecuteFunction);

	template<UE::Mass::CConstSharedFragment T>
	void ForEachConstSharedFragment(TFunctionRef< void(const T& /*ConstSharedFragment*/) > ExecuteFunction);

	template<UE::Mass::CConstSharedFragment T>
	void ForEachConstSharedFragmentConditional(TFunctionRef< bool(const T& /*ConstSharedFragment*/) > ConditionFunction, TFunctionRef< void(const T& /*ConstSharedFragment*/) > ExecuteFunction);

	/**
	 * Creates UE::Mass::FSparseElementIterator instance that can be used to iterate over all instances of a given sparse element fragment
	 * Note: works only for sparse fragments, as tags contain no data.
	 */
	UE::Mass::FSparseElementIterator CreateSparseElementsIterator(TNotNull<const UScriptStruct*> ElementType)
	{
		return UE::Mass::IsA<FMassSparseFragment>(ElementType)
			? SparseElementsStorage.CreateElementIterator(ElementType)
			: UE::Mass::FSparseElementIterator();
	}

	template<UE::Mass::CSparse T>
	requires UE::Mass::CFragment<T>
	UE::Mass::FSparseElementIterator CreateSparseElementsIterator()
	{
		return SparseElementsStorage.CreateElementIterator(T::StaticStruct());
	}

	//-----------------------------------------------------------------------------
	// Relations
	//-----------------------------------------------------------------------------
	UE::Mass::FRelationManager& GetRelationManager();

	template<UE::Mass::CRelation T>
	bool BatchCreateRelations(TArrayView<FMassEntityHandle> Subjects, TArrayView<FMassEntityHandle> Objects);
	UE_API bool BatchCreateRelations(const UE::Mass::FTypeHandle RelationTypeHandle, TArrayView<FMassEntityHandle> Subjects, TArrayView<FMassEntityHandle> Objects);

	//-----------------------------------------------------------------------------
	// Type management
	//-----------------------------------------------------------------------------
	UE_API void OnNewTypeRegistered(UE::Mass::FTypeHandle RegisteredTypeHandle);

protected:
	void OnRelationTypeRegistered(UE::Mass::FTypeHandle RegisteredTypeHandle, const UE::Mass::FRelationTypeTraits& RelationTypeTraits);

public:
	//-----------------------------------------------------------------------------
	// Entity Builder
	//-----------------------------------------------------------------------------
	[[nodiscard]] UE_API UE::Mass::FEntityBuilder MakeEntityBuilder();

	//-----------------------------------------------------------------------------
	// Sub-Managers
	//-----------------------------------------------------------------------------
	const UE::Mass::FTypeManager& GetTypeManager() const;
	UE::Mass::FTypeManager& GetTypeManager();

	FMassObserverManager& GetObserverManager()
	{
		return ObserverManager;
	}

	FOnNewArchetypeDelegate& GetOnNewArchetypeEvent()
	{
		return OnNewArchetypeEvent;
	}
	/** 
	 * Fetches the world associated with the Owner. 
	 * @note that it's ok for a given EntityManager to not have an owner or the owner not being part of a UWorld, depending on the use case
	 */
	UWorld* GetWorld() const
	{
		return Owner.IsValid() ? Owner->GetWorld() : nullptr;
	}
	UObject* GetOwner() const
	{
		return Owner.Get();
	}

	bool IsDuringEntityCreation() const;

	UE_API void SetDebugName(const FString& NewDebugGame);
#if WITH_MASSENTITY_DEBUG
	UE_API void DebugPrintArchetypes(FOutputDevice& Ar, const bool bIncludeEmpty = true) const;
	UE_API void DebugGetArchetypesStringDetails(FOutputDevice& Ar, const bool bIncludeEmpty = true) const;
	UE_API void DebugGetArchetypeFragmentTypes(const FMassArchetypeHandle& Archetype, TArray<const UScriptStruct*>& InOutFragmentList) const;
	UE_API int32 DebugGetArchetypeEntitiesCount(const FMassArchetypeHandle& Archetype) const;
	UE_API int32 DebugGetArchetypeEntitiesCountPerChunk(const FMassArchetypeHandle& Archetype) const;
	UE_API int32 DebugGetEntityCount() const;
	UE_API int32 DebugGetArchetypesCount() const;
	UE_API void DebugRemoveAllEntities();
	UE_API void DebugForceArchetypeDataVersionBump();
	UE_API void DebugGetArchetypeStrings(const FMassArchetypeHandle& Archetype, TArray<FName>& OutFragmentNames, TArray<FName>& OutTagNames);
	UE_DEPRECATED(5.7, "Using CreateEntityIndexHandle instead.")
	UE_API FMassEntityHandle DebugGetEntityIndexHandle(const int32 EntityIndex) const;
	UE_API const FString& DebugGetName() const;

	enum class EDebugFeatures
	{
		None,
		TraceProcessors = 1 << 0, // Used to track information about processors such as their name.
		All = TraceProcessors
	};

	UE_API void DebugEnableDebugFeature(EDebugFeatures Features);
	UE_API void DebugDisableDebugFeature(EDebugFeatures Features);
	UE_API bool DebugHasAllDebugFeatures(EDebugFeatures Features) const;

	UE_API FMassRequirementAccessDetector& GetRequirementAccessDetector();

	// For use by the friend MassDebugger
	UE_API UE::Mass::FStorageType& DebugGetEntityStorageInterface();
	// For use by the friend MassDebugger
	UE_API const UE::Mass::FStorageType& DebugGetEntityStorageInterface() const;

	UE_API bool DebugHasCommandsToFlush() const;

	UE_API UE::Mass::FSparseElementsStorage& DebugGetSparseElementsStorage();
#endif // WITH_MASSENTITY_DEBUG

protected:
	/** Called on the child process upon process's forking */
	UE_API void OnPostFork(EForkProcessRole Role);

	UE_API void GetMatchingArchetypes(const FMassFragmentRequirements& Requirements, TArray<FMassArchetypeHandle>& OutValidArchetypes, const uint32 FromArchetypeDataVersion) const;
	
	/** 
	 * A "similar" archetype is an archetype exactly the same as SourceArchetype except for one composition aspect 
	 * like Fragments or "Tags" 
	 */
	UE_API FMassArchetypeHandle InternalCreateSimilarArchetype(const TSharedPtr<FMassArchetypeData>& SourceArchetype, FMassElementBitSet&& NewCompositionBitSet);
	UE_API FMassArchetypeHandle InternalCreateSimilarArchetype(const TSharedPtr<FMassArchetypeData>& SourceArchetype, const UE::Mass::FArchetypeGroups& GroupsOverride);
	UE_API FMassArchetypeHandle InternalCreateSimilarArchetype(const FMassArchetypeData& SourceArchetypeRef, FMassArchetypeCompositionDescriptor&& NewComposition, const UE::Mass::FArchetypeGroups& GroupsOverride);

	UE_API void InternalAppendFragmentsAndTagsToArchetypeCompositionDescriptor(FMassArchetypeCompositionDescriptor& InOutComposition,
		TConstArrayView<const UScriptStruct*> FragmentsAndTagsList) const;

private:
	void InternalBatchRemoveSharedFragmentsFromEntities(
		TConstArrayView<FMassArchetypeEntityCollection> EntityCollections,
		const FMassElementBitSet& ElementsToRemoveBitSet,
		const FMassSharedFragmentBitSet* SharedFragmentToRemoveBitSet,
		const FMassConstSharedFragmentBitSet* ConstSharedFragmentToRemoveBitSet);

	void InternalBuildEntity(FMassEntityHandle EntityHandle, const FMassArchetypeHandle& ArchetypeHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues);
	void InternalReleaseEntity(FMassEntityHandle EntityHandle);

	/** 
	 *  Adds fragments in FragmentList to the entity indicated by EntityHandle. Only the unique fragments will be added.
	 *  @return Bitset for the added fragments (might be empty or a subset of `InFragments` depending on the current archetype fragments)
	 */
	FMassElementBitSet InternalAddFragmentListToEntityChecked(FMassEntityHandle EntityHandle, const FMassElementBitSet& FragmentList);

	/** 
	 *  Similar to InternalAddFragmentListToEntity but expects NewFragmentList not overlapping with current entity's
	 *  fragment list. It's callers responsibility to ensure that's true. Failing this will cause a `check` fail.
	 */
	void InternalAddFragmentListToEntity(FMassEntityHandle EntityHandle, const FMassElementBitSet& FragmentList);
	/** Note that it's the caller's responsibility to ensure `FragmentType` is a kind of FMassFragment */
	UE_API void* InternalGetFragmentDataChecked(FMassEntityHandle EntityHandle, const UScriptStruct* FragmentType) const;
	/** Note that it's the caller's responsibility to ensure `FragmentType` is a kind of FMassFragment */
	UE_API void* InternalGetFragmentDataPtr(FMassEntityHandle EntityHandle, const UScriptStruct* FragmentType) const;
	/** Note that it's the caller's responsibility to ensure `ConstSharedFragmentType` is a kind of FMassSharedFragment */
	UE_API const FConstSharedStruct* InternalGetConstSharedFragmentPtr(FMassEntityHandle EntityHandle, const UScriptStruct* ConstSharedFragmentType) const;
	/** Note that it's the caller's responsibility to ensure `SharedFragmentType` is a kind of FMassSharedFragment */
	UE_API const FSharedStruct* InternalGetSharedFragmentPtr(FMassEntityHandle EntityHandle, const UScriptStruct* SharedFragmentType) const;

	TSharedPtr<FMassEntityManager::FEntityCreationContext> InternalBatchCreateReservedEntities(const FMassArchetypeHandle& ArchetypeHandle
		, const FMassArchetypeSharedFragmentValues& SharedFragmentValues, TConstArrayView<FMassEntityHandle> ReservedEntities
		, const bool bCreateContext);

	UE::Mass::FStorageType& GetEntityStorageInterface() const
	{
#if WITH_MASS_CONCURRENT_RESERVE
		using namespace UE::Mass;
		struct StorageSelector
		{
			UE::Mass::IEntityStorageInterface* operator()(FEmptyVariantState&) const
			{
				checkf(false, TEXT("Attempt to use EntityStorageInterface without initialization"));
				return nullptr;
			}
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			UE::Mass::IEntityStorageInterface* operator()(FSingleThreadedEntityStorage& Storage) const
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				return &Storage;
			}
			UE::Mass::IEntityStorageInterface* operator()(FConcurrentEntityStorage& Storage) const
			{
				return &Storage;
			}
		};

		UE::Mass::IEntityStorageInterface* Interface = Visit(StorageSelector{}, EntityStorage);

		return *Interface;
#else	
		return EntityStorage.Get<UE::Mass::FSingleThreadedEntityStorage>();
#endif
	}

	bool DebugDoCollectionsOverlapCreationContext(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections) const;
		
private:

	friend struct UE::Mass::Private::FEntityStorageInitializer;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	using FEntityStorageContainerType = TVariant<
		FEmptyVariantState,
		UE::Mass::FSingleThreadedEntityStorage,
		UE::Mass::FConcurrentEntityStorage>;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	mutable FEntityStorageContainerType EntityStorage;
#if WITH_MASSENTITY_DEBUG
	UE::Mass::FStorageType* DebugEntityStoragePtr = nullptr;
#endif // WITH_MASSENTITY_DEBUG

	std::atomic<int32> ProcessingScopeCount = 0;

	// the "version" number increased every time an archetype gets added
	uint32 ArchetypeDataVersion = 0;

	// Map of hash of sorted fragment list to archetypes with that hash
	TMap<uint32, TArray<TSharedPtr<FMassArchetypeData>>> FragmentHashToArchetypeMap;

	// Map to list of archetypes that contain the specified fragment type
	TMap<const UScriptStruct*, TArray<TSharedPtr<FMassArchetypeData>>> FragmentTypeToArchetypeMap;

	// Contains all archetypes ever created. The array always growing and a given archetypes remains at a given index 
	// throughout its lifetime, and the index is never reused for another archetype. 
	TArray<TSharedPtr<FMassArchetypeData>> AllArchetypes;

	/**
	 * This is a struct wrapping shared fragment management to ensure consistency between how
	 * shared and const shared fragment are added and fetched, across all the functions that do that
	 */
	template<typename TSharedStructType>
	struct TSharedFragmentsContainer
	{
		TArray<TSharedStructType>* Find(const UScriptStruct* Type)
		{
			return TypeToInstanceMap.Find(Type);
		}

		TSharedStructType& FindOrAdd(const uint32 Hash, const UScriptStruct* Type, const uint8* Data)
		{
			for (TMultiMap<uint32, int32>::TConstKeyIterator It = HashToInstanceIndexMap.CreateConstKeyIterator(Hash); It; ++It)
			{
				TSharedStructType& Instance = Instances[It.Value()];

				if (Instance.GetScriptStruct() == Type
					&& Type->CompareScriptStruct(Instance.GetMemory(), Data, PPF_None))
				{
					return Instance;
				}
			}

			int32 Index = Add(TSharedStructType::Make(Type, Data));
			HashToInstanceIndexMap.Add(Hash, Index);
			return Instances[Index];
		}

		template<typename T, typename... TArgs>
		TSharedStructType& FindOrAdd(const uint32 Hash, TArgs&&... InArgs)
		{
			// Need to actually construct the struct to make proper comparison to possible existing instance
			TSharedStructType TempInstance = TSharedStructType::template Make<T>(Forward<TArgs>(InArgs)...);

			for (TMultiMap<uint32, int32>::TConstKeyIterator It = HashToInstanceIndexMap.CreateConstKeyIterator(Hash); It; ++It)
			{
				TSharedStructType& Instance = Instances[It.Value()];

				if (Instance.GetScriptStruct() == T::StaticStruct()
					&& T::StaticStruct()->CompareScriptStruct(Instance.GetMemory(), TempInstance.GetMemory(), PPF_None))
				{
					return Instance;
				}
			}

			int32 Index = Add(MoveTemp(TempInstance));
			HashToInstanceIndexMap.Add(Hash, Index);
			return Instances[Index];
		}

		TSharedStructType& operator[](const int32 Index)
		{
			return Instances[Index];
		}

		TArrayView<TSharedStructType> GetAllInstances()
		{
			return Instances;
		}

	private:
		int32 Add(TSharedStructType&& SharedStruct)
		{
			TArray<TSharedStructType>& InstancesOfType = TypeToInstanceMap.FindOrAdd(SharedStruct.GetScriptStruct(), {});
			const int32 Index = Instances.Add(Forward<TSharedStructType>(SharedStruct));
			// note that even though we're copying the input F[Const]SharedStruct instance it's perfectly fine since 
			// F[Const]SharedStruct does guarantee there's not going to be data duplication (via a member shared pointer to hosted data)
			InstancesOfType.Add(Instances[Index]);
			return Index;
		}

		TArray<TSharedStructType> Instances;
		// Hash/Index in array pair
		TMultiMap<uint32, int32> HashToInstanceIndexMap;
		// Maps specific struct type to a collection of FSharedStruct instances of that type
		TMap<const UScriptStruct*, TArray<TSharedStructType>> TypeToInstanceMap;
	};

	TSharedFragmentsContainer<FConstSharedStruct> ConstSharedFragmentsContainer;
	TSharedFragmentsContainer<FSharedStruct> SharedFragmentsContainer;

	FMassObserverManager ObserverManager;

	TSharedRef<UE::Mass::FTypeManager> TypeManager;
	UE::Mass::FRelationManager RelationManager;

	/** Stores sparse elements instances. Not accessible from outside on purpose. */
	UE::Mass::FSparseElementsStorage SparseElementsStorage;

	TMap<const FName, const int32> GroupNameToTypeIndex;
	// @todo we'll probably have some "GroupTypeInformation" here in the future
	TArray<const FName> GroupTypes;

#if WITH_MASSENTITY_DEBUG
	FMassRequirementAccessDetector RequirementAccessDetector;
	FString DebugName;
	EDebugFeatures EnabledDebugFeatures = EDebugFeatures::All;
#endif // WITH_MASSENTITY_DEBUG

	TWeakObjectPtr<UObject> Owner;

	FOnNewArchetypeDelegate OnNewArchetypeEvent;

	FDelegateHandle OnPostForkHandle;

	/**
	 * This index will be enough to control which buffer is available for pushing commands since flashing is taking place 
	 * in the game thread and pushing commands to the buffer fetched by Defer() is only supported also on the game thread
	 * (due to checking the cached thread ID).
	 * The whole CL aims to support non-mass code trying to push commands while the flushing is going on (as triggered
	 * by MassObservers reacting to the commands being flushed currently).
	 */
	static constexpr int32 NumCommandBuffers = 2;
	TStaticArray<TSharedPtr<FMassCommandBuffer>, NumCommandBuffers> DeferredCommandBuffers;
	uint8 OpenedCommandBufferIndex = 0;
	std::atomic<bool> bCommandBufferFlushingInProgress = false;
	bool bFirstCommandFlush = true;

	enum class EInitializationState : uint8
	{
		Uninitialized,
		Initialized,
		Deinitialized
	};

	EInitializationState InitializationState = EInitializationState::Uninitialized;

public:
	//-----------------------------------------------------------------------------
	// DEPRECATED
	//-----------------------------------------------------------------------------
	UE_DEPRECATED(5.8, "Use BatchAddElementToEntities instead")
	void AddElementToEntities(TConstArrayView<FMassEntityHandle> Entities, TNotNull<const UScriptStruct*> ElementType)
	{
		BatchAddElementToEntities(Entities, ElementType);
	}
	UE_DEPRECATED(5.8, "Use BatchRemoveElementFromEntities instead")
	void RemoveElementFromEntities(TConstArrayView<FMassEntityHandle> Entities, TNotNull<const UScriptStruct*> ElementType)
	{
		BatchRemoveElementFromEntities(Entities, ElementType);
	}
};

//-----------------------------------------------------------------------------
// INLINE
//-----------------------------------------------------------------------------

#if WITH_MASSENTITY_DEBUG
ENUM_CLASS_FLAGS(FMassEntityManager::EDebugFeatures);
#endif

template<UE::Mass::CSharedFragment T>
void FMassEntityManager::ForEachSharedFragment(TFunctionRef< void(T& /*SharedFragment*/) > ExecuteFunction)
{
	if (TArray<FSharedStruct>* InstancesOfType = SharedFragmentsContainer.Find(T::StaticStruct()))
	{
		for (const FSharedStruct& SharedStruct : *InstancesOfType)
		{
			ExecuteFunction(SharedStruct.Get<T>());
		}
	}
}

template<UE::Mass::CSharedFragment T>
void FMassEntityManager::ForEachSharedFragmentConditional(TFunctionRef< bool(T& /*SharedFragment*/) > ConditionFunction, TFunctionRef< void(T& /*SharedFragment*/) > ExecuteFunction)
{
	if (TArray<FSharedStruct>* InstancesOfType = SharedFragmentsContainer.Find(T::StaticStruct()))
	{
		for (const FSharedStruct& SharedStruct : *InstancesOfType)
		{
			T& StructInstanceRef = SharedStruct.Get<T>();
			if (ConditionFunction(StructInstanceRef))
			{
				ExecuteFunction(StructInstanceRef);
			}
		}
	}
}

template<UE::Mass::CConstSharedFragment T>
void FMassEntityManager::ForEachConstSharedFragment(TFunctionRef< void(const T& /*ConstSharedFragment*/) > ExecuteFunction)
{
	if (TArray<FConstSharedStruct>* InstancesOfType = ConstSharedFragmentsContainer.Find(T::StaticStruct()))
	{
		for (const FConstSharedStruct& SharedStruct : *InstancesOfType)
		{
			ExecuteFunction(SharedStruct.Get<const T>());
		}
	}
}

template<UE::Mass::CConstSharedFragment T>
void FMassEntityManager::ForEachConstSharedFragmentConditional(TFunctionRef< bool(const T& /*ConstSharedFragment*/) > ConditionFunction, TFunctionRef< void(const T& /*ConstSharedFragment*/) > ExecuteFunction)
{
	if (TArray<FConstSharedStruct>* InstancesOfType = ConstSharedFragmentsContainer.Find(T::StaticStruct()))
	{
		for (const FConstSharedStruct& SharedStruct : *InstancesOfType)
		{
			const T& StructInstanceRef = SharedStruct.Get<const T>();
			if (ConditionFunction(StructInstanceRef))
			{
				ExecuteFunction(StructInstanceRef);
			}
		}
	}
}

inline const UE::Mass::FTypeManager& FMassEntityManager::GetTypeManager() const
{
	return *TypeManager;
}

inline UE::Mass::FTypeManager& FMassEntityManager::GetTypeManager()
{
	return *TypeManager;
}

inline bool FMassEntityManager::IsDuringEntityCreation() const
{
	return ObserverManager.GetCreationContext().IsValid();
}

inline TSharedRef<FMassObserverManager::FObserverLock> FMassEntityManager::GetOrMakeObserversLock()
{
	return ObserverManager.GetOrMakeObserverLock();
}

inline UE::Mass::FRelationManager& FMassEntityManager::GetRelationManager()
{
	return RelationManager;
}

template<UE::Mass::CRelation T>
bool FMassEntityManager::BatchCreateRelations(TArrayView<FMassEntityHandle> Subjects, TArrayView<FMassEntityHandle> Objects)
{
	return BatchCreateRelations(UE::Mass::FTypeManager::MakeTypeHandle(T::StaticStruct()), Subjects, Objects);
}

inline void FMassEntityManager::SetEntityFragmentValues(FMassEntityHandle EntityHandle, TArrayView<const FInstancedStruct> FragmentInstanceList)
{
	SetEntityFragmentValues<FInstancedStruct>(EntityHandle, FragmentInstanceList);
}

inline void FMassEntityManager::BatchSetEntityFragmentValues(const FMassArchetypeEntityCollection& SparseEntities, TArrayView<const FInstancedStruct> FragmentInstanceList)
{
	BatchSetEntityFragmentValues<FInstancedStruct>(SparseEntities, FragmentInstanceList);
}

inline void FMassEntityManager::BatchSetEntityFragmentValues(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TArrayView<const FInstancedStruct> FragmentInstanceList)
{
	BatchSetEntityFragmentValues<FInstancedStruct>(EntityCollections, FragmentInstanceList);
}

extern template UE_API void FMassEntityManager::SetEntityFragmentValues<FInstancedStruct>(FMassEntityHandle EntityHandle, TConstArrayView<FInstancedStruct> FragmentInstanceList);
extern template UE_API void FMassEntityManager::SetEntityFragmentValues<FStructView>(FMassEntityHandle EntityHandle, TConstArrayView<FStructView> FragmentInstanceList);
extern template UE_API void FMassEntityManager::SetEntityFragmentValues<FConstStructView>(FMassEntityHandle EntityHandle, TConstArrayView<FConstStructView> FragmentInstanceList);
extern template UE_API void FMassEntityManager::BatchSetEntityFragmentValues<FInstancedStruct>(const FMassArchetypeEntityCollection& SparseEntities, TConstArrayView<FInstancedStruct> FragmentInstanceList);
extern template UE_API void FMassEntityManager::BatchSetEntityFragmentValues<FStructView>(const FMassArchetypeEntityCollection& SparseEntities, TConstArrayView<FStructView> FragmentInstanceList);
extern template UE_API void FMassEntityManager::BatchSetEntityFragmentValues<FConstStructView>(const FMassArchetypeEntityCollection& SparseEntities, TConstArrayView<FConstStructView> FragmentInstanceList);
extern template UE_API void FMassEntityManager::BatchSetEntityFragmentValues<FInstancedStruct>(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TConstArrayView<FInstancedStruct> FragmentInstanceList);
extern template UE_API void FMassEntityManager::BatchSetEntityFragmentValues<FStructView>(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TConstArrayView<FStructView> FragmentInstanceList);
extern template UE_API void FMassEntityManager::BatchSetEntityFragmentValues<FConstStructView>(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, TConstArrayView<FConstStructView> FragmentInstanceList);

#undef UE_API
