// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/TransactionallySafeMutex.h"
#include "Containers/ChunkedArray.h"
#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"
#include "MassProcessingTypes.h"

struct FMassArchetypeData;
struct FMassEntityHandle;

/**
 * Initialization parameters to configure MassEntityManager to reserve entities only single threaded
 * Supported in all build configurations
 */
struct
	UE_DEPRECATED(5.8, "Single threaded entity storage is being removed. Use FMassEntityManager_InitParams_Concurrent instead.")
	FMassEntityManager_InitParams_SingleThreaded {};

/**
 * Initialization parameters to configure MassEntityManager to concurrently reserve entities
 * Only supported in editor builds.
 *
 * Expected static memory requirement for array of Page pointers can be computed:
 * MaxPages = MaxEntityCount / MaxEntitiesPerPage
 * MemorySize = MaxPages * sizeof(Page**)
 *
 * For default values, expectation is 128kB
 */
struct FMassEntityManager_InitParams_Concurrent
{
	/** 
	 * Maximum supported entities by the MassEntityManager
	 * Must be multiple of 2
	 */
	uint32 MaxEntityCount = 1 << 30; // 1 billion

	/** 
	 * Number of entities per chunk
	 * Must be multiple of 2
	 */
	uint32 MaxEntitiesPerPage = 1 << 16; // 65536
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS 
using FMassEntityManagerStorageInitParams = TVariant<FMassEntityManager_InitParams_SingleThreaded, FMassEntityManager_InitParams_Concurrent>;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

namespace UE::Mass
{
	/**
	 * Interface that abstracts the storage system for Mass Entities in the EntityManager
	 * This may be temporary until the concurrent mechanism has been vetted for performance
	 */
	class IEntityStorageInterface
	{
	public:
		enum class EEntityState
		{
			/** Entity index refers to an entity that is free to be reserved or created */
			Free,
			/** Entity index refers to a reserved entity */
			Reserved,
			/** Entity index refers to an entity assigned to an archetype */
			Created
		};
		virtual ~IEntityStorageInterface() = default;

		virtual FMassArchetypeData* GetArchetype(int32 Index) = 0;
		virtual const FMassArchetypeData* GetArchetype(int32 Index) const = 0;
		virtual TSharedPtr<FMassArchetypeData>& GetArchetypeAsShared(int32 Index) = 0;
		virtual const TSharedPtr<FMassArchetypeData>& GetArchetypeAsShared(int32 Index) const = 0;

		virtual void SetArchetypeFromShared(int32 Index, TSharedPtr<FMassArchetypeData>& Archetype) = 0;
		virtual void SetArchetypeFromShared(int32 Index, const TSharedPtr<FMassArchetypeData>& Archetype) = 0;

		/**
		 * Returns true if the given entity at index is currently reserved
		 * False if free or assigned an archetype
		 */
		virtual EEntityState GetEntityState(int32 Index) const = 0;

		virtual int32 GetSerialNumber(int32 Index) const = 0;

		/** Checks if index can be used to access entity data */
		virtual bool IsValidIndex(int32 Index) const = 0;

		/**
		 * Checks if the given handle is valid in the context od this storage, i.e. whether the
		 * index is valid and the serial number associated with it matches the handle's
		 */
		virtual bool IsValidHandle(FMassEntityHandle EntityHandle) const = 0;

		virtual bool IsEntityActive(FMassEntityHandle EntityHandle) const = 0;

		virtual SIZE_T GetAllocatedSize() const = 0;

		/** Checks if entity at Index is built */
		virtual bool IsValid(int32 Index) const = 0;

		/** Produce a single entity handle */
		virtual FMassEntityHandle AcquireOne() = 0;

		/**
		 * @return number of entities actually added
		 */
		int32 Acquire(const int32 Count, TArray<FMassEntityHandle>& OutEntityHandles);
		virtual int32 Acquire(TArrayView<FMassEntityHandle> OutEntityHandles) = 0;

		virtual int32 Release(TConstArrayView<FMassEntityHandle> Handles) = 0;
		virtual int32 ReleaseOne(FMassEntityHandle Handles) = 0;

		/**
		 * Bypasses Serial Number Check
		 * Only use if caller has ensured serial number matches or for debug purposes
		 */
		virtual int32 ForceRelease(TConstArrayView<FMassEntityHandle> Handles) = 0;
		virtual int32 ForceReleaseOne(FMassEntityHandle Handle) = 0;

		/**
		 * Returns the number of entities that are not free
		 * For debug purposes only. In multi-threaded environments, the result is going to be out of date
		 */
		virtual int32 Num() const = 0;

		/**
		 * Returns the number of entities that are free
		 * For debug purposes only. In multi-threaded environments, the result is going to be out of date
		 */
		virtual int32 ComputeFreeSize() const = 0;
	};

	//-----------------------------------------------------------------------------
	// FSingleThreadedEntityStorage
	//-----------------------------------------------------------------------------
	/**
	 * This storage backend should be used when the user of MassEntityManager can guarantee
	 * that all entity management will be done on a single thread.
	 */
	class
		UE_DEPRECATED(5.8, "Single threaded entity storage is being removed. Use FConcurrentEntityStorage instead.")
		FSingleThreadedEntityStorage final : public IEntityStorageInterface
	{
	public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		void Initialize(const FMassEntityManager_InitParams_SingleThreaded&);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		virtual FMassArchetypeData* GetArchetype(int32 Index) override;
		virtual const FMassArchetypeData* GetArchetype(int32 Index) const override;
		virtual TSharedPtr<FMassArchetypeData>& GetArchetypeAsShared(int32 Index) override;
		virtual const TSharedPtr<FMassArchetypeData>& GetArchetypeAsShared(int32 Index) const override;
		virtual void SetArchetypeFromShared(int32 Index, TSharedPtr<FMassArchetypeData>&) override;
		virtual void SetArchetypeFromShared(int32 Index, const TSharedPtr<FMassArchetypeData>&) override;
		virtual EEntityState GetEntityState(int32 Index) const override;
		virtual int32 GetSerialNumber(int32 Index) const override;
		virtual bool IsValidIndex(int32 Index) const override;
		virtual bool IsValidHandle(FMassEntityHandle EntityHandle) const override;
		virtual bool IsEntityActive(FMassEntityHandle EntityHandle) const override;
		virtual SIZE_T GetAllocatedSize() const override;
		virtual bool IsValid(int32 Index) const override;
		virtual FMassEntityHandle AcquireOne() override;
		using IEntityStorageInterface::Acquire;
		virtual int32 Acquire(TArrayView<FMassEntityHandle> OutEntityHandles) override;
		virtual int32 Release(TConstArrayView<FMassEntityHandle> Handles) override;
		virtual int32 ReleaseOne(FMassEntityHandle Handle) override;
		virtual int32 ForceRelease(TConstArrayView<FMassEntityHandle> Handles) override;
		virtual int32 ForceReleaseOne(FMassEntityHandle Handle) override;
		virtual int32 Num() const override;
		virtual int32 ComputeFreeSize() const override;

	private:
		struct FEntityData
		{
			TSharedPtr<FMassArchetypeData> CurrentArchetype;
			int32 SerialNumber = 0;

			~FEntityData();
			void Reset();
			bool IsValid() const;
		};

		std::atomic<int32> NextSerialNumber = 0;

		UE_AUTORTFM_ALWAYS_OPEN
		int32 GenerateSerialNumber()
		{
			// The serial number only needs to be unique; it doesn't need to be rolled back if an AutoRTFM transaction fails.
			return NextSerialNumber.fetch_add(1);
		}

		TChunkedArray<FEntityData> Entities;
		TArray<int32> EntityFreeIndexList;
	};

	//-----------------------------------------------------------------------------
	// FConcurrentEntityStorage
	//-----------------------------------------------------------------------------
	/**
	 * This storage backend allows for entities to be concurrently reserved. Reserved entities can also
	 * be concurrently freed.
	 * Creation of entities (i.e. assignment of an archetype and addition of data into chunks) cannot be done
	 * concurrently with this implementation.
	 */
	class FConcurrentEntityStorage final : public IEntityStorageInterface
	{
	public:

		void Initialize(const FMassEntityManager_InitParams_Concurrent& InInitializationParams);

		virtual ~FConcurrentEntityStorage() override;

		virtual FMassArchetypeData* GetArchetype(int32 Index) override;
		virtual const FMassArchetypeData* GetArchetype(int32 Index) const override;
		virtual TSharedPtr<FMassArchetypeData>& GetArchetypeAsShared(int32 Index) override;
		virtual const TSharedPtr<FMassArchetypeData>& GetArchetypeAsShared(int32 Index) const override;
		virtual void SetArchetypeFromShared(int32 Index, TSharedPtr<FMassArchetypeData>& Archetype) override;
		virtual void SetArchetypeFromShared(int32 Index, const TSharedPtr<FMassArchetypeData>& Archetype) override;
		virtual EEntityState GetEntityState(int32 Index) const override;
		virtual int32 GetSerialNumber(int32 Index) const override;
		virtual bool IsValidIndex(int32 Index) const override;
		virtual bool IsValidHandle(FMassEntityHandle EntityHandle) const override;
		virtual bool IsEntityActive(FMassEntityHandle EntityHandle) const override;
		virtual SIZE_T GetAllocatedSize() const override;
		virtual bool IsValid(int32 Index) const override;
		virtual FMassEntityHandle AcquireOne() override;
		using IEntityStorageInterface::Acquire;
		virtual int32 Acquire(TArrayView<FMassEntityHandle> OutEntityHandles) override;
		virtual int32 Release(TConstArrayView<FMassEntityHandle> Handles) override;
		virtual int32 ReleaseOne(FMassEntityHandle Handle) override;
		virtual int32 ForceRelease(TConstArrayView<FMassEntityHandle> Handles) override;
		virtual int32 ForceReleaseOne(FMassEntityHandle Handle) override;
		virtual int32 Num() const override;
		virtual int32 ComputeFreeSize() const override;
#if WITH_MASSENTITY_DEBUG
		/** @return whether the assumptions are still valid */
		MASSENTITY_API static bool DebugAssumptionsSelfTest();
#endif // WITH_MASSENTITY_DEBUG
	private:

		struct FEntityData
		{
			static constexpr int MaxGenerationBits = 30;

			TSharedPtr<FMassArchetypeData> CurrentArchetype;
			/** Generation ID or version of the entity in this slot */
			uint32 GenerationId : MaxGenerationBits = 0;
			/** 1 if the entity is NOT free */
			uint32 bIsAllocated : 1 = 0;

			~FEntityData();
			/** Converts EntityData state into a SerialNumber for public usage */
			int32 GetSerialNumber() const;

			bool operator==(const FEntityData& Other) const;
		};

		EEntityState GetEntityStateInternal(const FEntityData& EntityData) const;

		FEntityData& LookupEntity(int32 Index);
		const FEntityData& LookupEntity(int32 Index) const;

		/** Returns size of a page in bytes */
		uint64 ComputePageSize() const;

		/**
		 * @return whether the operation was successful. Will return false when OOM
		 */
		bool AddPage();

		/** Number of allocated Entities (only used for viewing in the debugger). */
		uint32 EntityCount = 0;
		uint32 MaximumEntityCountShift = 0;
		uint32 MaxEntitiesPerPage = 0;
		uint32 MaxEntitiesPerPageShift = 0;
		uint32 PageCount = 0;
		/** ALWAYS acquire FreeListMutex before this one */
		UE::FTransactionallySafeMutex PageAllocateMutex;
		/** Pointer to array of pages */
		FEntityData** EntityPages = nullptr;

		TArray<int32> EntityFreeIndexList;
		UE::FTransactionallySafeMutex FreeListMutex;
	};

} // namespace UE::Mass
