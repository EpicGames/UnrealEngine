// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#include "MassArchetypeTypes.h"
#include "Mass/ArchetypeGroup.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Mass/ExternalSubsystemTraits.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassRequirements.h"

#define UE_API MASSENTITY_API

#ifndef WITH_ARCHETYPE_MATCH_OVERRIDE
#define WITH_ARCHETYPE_MATCH_OVERRIDE (WITH_EDITOR)
#endif //  WITH_ARCHETYPE_MATCH_OVERRIDE

#ifndef MASS_ACHETYPE_OVERRIDE_MAX_SIZE
#define MASS_ACHETYPE_OVERRIDE_MAX_SIZE 16
#endif
	
#ifndef MASS_ACHETYPE_OVERRIDE_MAX_ALIGNMENT
#define MASS_ACHETYPE_OVERRIDE_MAX_ALIGNMENT 8
#endif

#if WITH_ARCHETYPE_MATCH_OVERRIDE
template<typename T>
concept TArchetypeMatchOverrideConcept = requires(const FMassArchetypeCompositionDescriptor& Descriptor)
{
	{ static_cast<const T*>(nullptr)->Match(Descriptor) } -> std::convertible_to<bool>;
};
#endif

struct FMassEntityManager;

/** 
 *  FMassEntityQuery is a structure that is used to trigger calculations on cached set of valid archetypes as described 
 *  by requirements. See the parent classes FMassFragmentRequirements and FMassSubsystemRequirements for setting up the 
 *	required fragments and subsystems.
 * 
 *  A query to be considered valid needs declared at least one EMassFragmentPresence::All, EMassFragmentPresence::Any 
 *  EMassFragmentPresence::Optional fragment requirement.
 */
struct FMassEntityQuery : public FMassFragmentRequirements, public FMassSubsystemRequirements
{
	static constexpr int32 ArchetypeMatchOverrideSize = MASS_ACHETYPE_OVERRIDE_MAX_SIZE;
	static constexpr uint32 ArchetypeMatchOverrideAlignment = MASS_ACHETYPE_OVERRIDE_MAX_ALIGNMENT;
	
	friend struct FMassDebugger;

private:
	struct FArchetypeMatchOverride
    {
		using MatchFunction = bool (*)(const void* /*Context*/, const FMassArchetypeCompositionDescriptor&);
		
		MatchFunction Match = nullptr;
    	TAlignedBytes<ArchetypeMatchOverrideSize, ArchetypeMatchOverrideAlignment> Data;
    };

public:
	enum class EParallelExecutionFlags
	{
		// Use whatever the whole system has been configured for.
		Default = 0,
		// Force parallel execution of a processor for each chunk even when parallel execution has been disabled.
		Force = 1 << 0,
		// The default behavior for parallel execution assigns each chunk to a thread before execution. This implicitly assumes all chunks
		// take roughly the same amount of time to process. If chunks vary in the time it takes to process this flag can be used to queue
		// chunks so threads can pick them up as soon as possible. This makes starting the processing of a chunk more expensive but can
		// result in better overall utilization of threads.
		AutoBalance = 1 << 1
	};

	FMassEntityQuery() = default;
	UE_API FMassEntityQuery(UMassProcessor& Owner);
	UE_API FMassEntityQuery(const TSharedPtr<FMassEntityManager>& EntityManager);
	UE_API FMassEntityQuery(const TSharedRef<FMassEntityManager>& EntityManager, std::initializer_list<UScriptStruct*> InitList);
	UE_API FMassEntityQuery(const TSharedRef<FMassEntityManager>& EntityManager, TConstArrayView<const UScriptStruct*> InitList);

	UE_API void RegisterWithProcessor(UMassProcessor& Owner);

	/** Runs ExecuteFunction on all entities matching Requirements */
	UE_API void ForEachEntityChunk(FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction);

	/**
	 * Runs ExecuteFunction on entities matching Requirements up to the EntityLimit specified in the ExecutionLimiter.
	 * Resumes from last index set in Limiter.
	 * Will always complete the current chunk once the limit is reached.
	 * Does not account for changes to entity count or organization between iterations (Entities could be skipped).
	 * Identical to the unlimited version if the Limit exceeds the number of applicable entities.
	 */
	UE_API void ForEachEntityChunk(FMassExecutionContext& ExecutionContext, UE::Mass::FExecutionLimiter& Limiter, const FMassExecuteFunction& ExecuteFunction);
	
	/**
	 * Executes provided ExecuteFunction for entities in EntityCollection. If EntityCollection's archetype
	 * doesn't match the query's requirements then ExecuteFunction won't be executed.
	 */
	UE_API void ForEachEntityChunk(const FMassArchetypeEntityCollection& EntityCollection
		, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction);

	UE_API void ForEachEntityChunk(FMassArchetypeEntityCollection&& EntityCollection
		, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction);

	/**
	 * Attempts to process every chunk of every affected archetype in parallel.
	 */
	UE_API void ParallelForEachEntityChunk(FMassExecutionContext& ExecutionContext
		, const FMassExecuteFunction& ExecuteFunction, const EParallelExecutionFlags Flags = EParallelExecutionFlags::Default);

	UE_API void ForEachEntityChunkInCollections(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
		, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction);

	UE_API void ParallelForEachEntityChunkInCollection(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
		, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction
		, const EParallelExecutionFlags Flags = EParallelExecutionFlags::Default);

	/** Will gather all archetypes from InEntityManager matching this->Requirements.
	 *  Note that no work will be done if the cached data is up to date (as tracked by EntitySubsystemHash and 
	 *	ArchetypeDataVersion properties). */
	UE_API void CacheArchetypes();

	void Clear()
	{
		FMassFragmentRequirements::Reset();
		FMassSubsystemRequirements::Reset();
		ResetGrouping();
		DirtyCachedData();
	}

	inline void DirtyCachedData()
	{
		EntitySubsystemHash = 0;
		LastUpdatedArchetypeDataVersion = 0;
	}

	using FMassSubsystemRequirements::AddSubsystemRequirement;

	FMassSubsystemRequirements& AddSubsystemRequirement(const TSubclassOf<USubsystem> SubsystemClass, const EMassFragmentAccess AccessMode)
	{
		FMassSubsystemRequirements::AddSubsystemRequirement(SubsystemClass, AccessMode, CachedEntityManager.ToSharedRef());
		return *this;
	}
	
	bool DoesRequireGameThreadExecution() const 
	{ 
		return FMassFragmentRequirements::DoesRequireGameThreadExecution() 
			|| FMassSubsystemRequirements::DoesRequireGameThreadExecution() 
			|| bRequiresMutatingWorldAccess;
	}

	void RequireMutatingWorldAccess()
	{
		bRequiresMutatingWorldAccess = true;
	}

	bool IsEmpty() const
	{
		return FMassFragmentRequirements::IsEmpty() && FMassSubsystemRequirements::IsEmpty();
	}

	const TArray<FMassArchetypeHandle>& GetArchetypes() const
	{ 
		return ValidArchetypes; 
	}

	/**
	 * Calculates a hash of the entity order revisions in all archetypes. If returned value ever differs from
	 * a previous call it indicates that one or more archetypes associated with this query have been updated
	 * by having the entity order changed.
	 */
	UE_API uint64 CalculateArchetypeEntityOrderHash();
	
	/**
	 * Calculates a hash of the version of all archetypes. If returned value ever differs from
	 * a previous call it indicates that one or more archetypes associated with this query have been updated
	 * by having entities added, removed or re-ordered.
	 */
	UE_API uint64 CalculateArchetypeVersionHash();

	/** 
	 * Goes through ValidArchetypes and sums up the number of entities contained in them.
	 * Note that the function is not const because calling it can result in re-caching of ValidArchetypes 
	 * @return the number of entities this given query would process if called "now"
	 */
	UE_API int32 GetNumMatchingEntities();

	/** 
	 * Sums the entity range lengths for each collection in EntityCollections, where the collection's 
	 * archetype matches the query's requirements.
	 * @return the number of entities this given query would process if called "now" for EntityCollections
	 */
	UE_API int32 GetNumMatchingEntities(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections);

	/**
	 * Checks if any of ValidArchetypes has any entities.
	 * Note that the function is not const because calling it can result in re-caching of ValidArchetypes
	 * @return "true" if any of the ValidArchetypes has any entities, "false" otherwise
	 */
	UE_API bool HasMatchingEntities();

	/**
	 * Creates an array of FMassArchetypeEntityCollection instances that identify all the entities
	 * currently matching this query.
	 */
	UE_API TArray<FMassArchetypeEntityCollection> CreateMatchingEntitiesCollection();

	/**
	 * Fetches entity handles of all the entities currently matching this query.
	 */
	UE_API TArray<FMassEntityHandle> GetMatchingEntityHandles();

	/** 
	 * Sets a chunk filter condition that will be applied to each chunk of all valid archetypes. Note 
	 * that this condition won't be applied when a specific entity collection is used (via FMassArchetypeEntityCollection )
	 * The value returned by InFunction controls whether to allow execution (true) or block it (false).
	 */
	void SetChunkFilter(const FMassChunkConditionFunction& InFunction);

	void ClearChunkFilter()
	{
		ChunkCondition.Reset();
	}

	bool HasChunkFilter() const
	{
		return bool(ChunkCondition);
	}

	UE_API void GroupBy(UE::Mass::FArchetypeGroupType GroupType);
	UE_API void GroupBy(UE::Mass::FArchetypeGroupType GroupType, const TFunction<bool(const UE::Mass::FArchetypeGroupID, const UE::Mass::FArchetypeGroupID)>& Predicate);
	UE_API void ResetGrouping();

	/** 
	 * @return whether the query is configured to use archetype group information to group and sort
	 *		archetypes to be processed.
	 */
	bool IsGrouping() const;

#if WITH_ARCHETYPE_MATCH_OVERRIDE
	template<typename T> requires TArchetypeMatchOverrideConcept<T>
	void SetArchetypeMatchOverride(const T& Override);
#endif

	const TSharedPtr<FMassEntityManager>& GetEntityManager() const;

	/** 
	 * If ArchetypeHandle is among ValidArchetypes then the function retrieves requirements mapping cached for it,
	 * otherwise an empty mapping will be returned (and the requirements binding will be done the slow way).
	 */
	UE_API const FMassQueryRequirementIndicesMapping& GetRequirementsMappingForArchetype(const FMassArchetypeHandle ArchetypeHandle) const;

	UE_API void ExportRequirements(FMassExecutionRequirements& OutRequirements) const;

	/** 
	 * Controls whether ParallelForEachEntityChunk creates separate command buffers for each job.
	 * @see bAllowParallelCommands for more details
	 */
	void SetParallelCommandBufferEnabled(const bool bInAllowParallelCommands)
	{
		bAllowParallelCommands = bInAllowParallelCommands;
	}

	/**
	 * Configures the query to support per-entity logging based on their individual UObject "owners",
	 * as declared via debug fragments.
	 */
	UE_API void DebugEnableEntityOwnerLogging();

	void AddCreatedArchetype(const FMassElementBitSet& InComposition)
	{
		EntityCreationRequirements.AddCreatedArchetype(InComposition);
	}

	void AddCreatedArchetype(FMassElementBitSet&& InComposition)
	{
		EntityCreationRequirements.AddCreatedArchetype(InComposition);
	}

	FMassEntityCreationRequirements& GetEntityCreationRequirements()
	{
		return EntityCreationRequirements;
	}

private:
	/**
	 * Incrementally sorts all ValidArchetypes to fill OrderedArchetypeIndices with the expected order of archetype processing.
	 * This function will only ever get called when there are actual sorting steps registered (@see GroupBy)
	 */
	void SortArchetypes(const int32 FirstNewArchetypeIndex = 0);
	/**
	 * An alternative to SortArchetypes that will get called in the absence of archetype sorting steps to maintain OrderedArchetypeIndices
	 * and have it reflect the order of ValidArchetypes.
	 */
	void BuildOrderedArchetypeIndices(const int32 FirstNewArchetypeIndex = 0);
	
	/**
	 * Internal function to hash any value for all archetypes associated with this query.
	 */
	uint64 CalculateArchetypeHash_Internal(TFunctionRef<uint32(const FMassArchetypeData*)> HashValueGetter);

	/** 
	 * This function represents a condition that will be called for every chunk to be processed before the actual 
	 * execution function is called. The chunk fragment requirements are already bound and ready to be used by the time 
	 * ChunkCondition is executed.
	 */
	FMassChunkConditionFunction ChunkCondition;

	uint32 EntitySubsystemHash = 0;
	uint32 LastUpdatedArchetypeDataVersion = 0;

	TArray<FMassArchetypeHandle> ValidArchetypes;
	TArray<int32> OrderedArchetypeIndices;
	TArray<FMassQueryRequirementIndicesMapping> ArchetypeFragmentMapping;

	struct FArchetypeGroupingStep
	{
		FArchetypeGroupingStep() = default;
		FArchetypeGroupingStep(UE::Mass::FArchetypeGroupType InGroupType, const TFunction<bool(const UE::Mass::FArchetypeGroupID, const UE::Mass::FArchetypeGroupID)>& InPredicate)
			: GroupType(InGroupType), Predicate(InPredicate)
		{
		}
		UE::Mass::FArchetypeGroupType GroupType;
		TFunction<bool(const UE::Mass::FArchetypeGroupID, const UE::Mass::FArchetypeGroupID)> Predicate;
	};
	TArray<FArchetypeGroupingStep> GroupSortingSteps;
	TArray<TArray<UE::Mass::FArchetypeGroupID>> CachedGroupIDs;

	FMassEntityCreationRequirements EntityCreationRequirements;

	/** 
	 * Controls whether ParallelForEachEntityChunk created dedicated command buffer for each job. This is required 
	 * to ensure thread safety. Disable by calling SetParallelCommandBufferEnabled(false) if execution function doesn't 
	 * issue commands. Disabling will save some performance since it will avoid dynamic allocation of command buffers.
	 * 
	 * @Note that disabling parallel commands will result in no command buffer getting passed to execution which in turn
	 *	will cause crashes if the underlying code does try to issue commands. 
	 */
	uint8 bAllowParallelCommands : 1 = true;
	uint8 bRequiresMutatingWorldAccess : 1 = false;
#if WITH_ARCHETYPE_MATCH_OVERRIDE
	uint8 bHasArchetypeMatchOverride : 1 = false;
#endif

	EMassExecutionContextType ExpectedContextType = EMassExecutionContextType::Local;

#if WITH_MASSENTITY_DEBUG
	uint8 bRegistered : 1 = false;
#endif // WITH_MASSENTITY_DEBUG

#if WITH_ARCHETYPE_MATCH_OVERRIDE
	FArchetypeMatchOverride ArchetypeMatchOverride;
#endif

public:
	//-----------------------------------------------------------------------------
	// DEPRECATED
	//-----------------------------------------------------------------------------
	UE_DEPRECATED(5.6, "This type of FMassEntityQuery is no longer supported. Use one of the other constructors instead.")
	UE_API FMassEntityQuery(std::initializer_list<UScriptStruct*> InitList);

	UE_DEPRECATED(5.6, "This type of FMassEntityQuery is no longer supported. Use one of the other constructors instead.")
	UE_API FMassEntityQuery(TConstArrayView<const UScriptStruct*> InitList);

	enum EParallelForMode
	{
		Default = static_cast<int32>(EParallelExecutionFlags::Default),
		ForceParallelExecution = static_cast<int32>(EParallelExecutionFlags::Force),
	};

	UE_DEPRECATED(5.6, "ForEachEntityChunk is deprecated. New version doesn't require the FMassEntityManager parameter")
	UE_API void ForEachEntityChunk(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction);
	
	UE_DEPRECATED(5.6, "ForEachEntityChunk is deprecated. New version doesn't require the FMassEntityManager parameter")
	UE_API void ForEachEntityChunk(const FMassArchetypeEntityCollection& EntityCollection, FMassEntityManager& EntityManager
		, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction);

	UE_DEPRECATED(5.6, "ParallelForEachEntityChunk is deprecated. New version doesn't require the FMassEntityManager parameter. Also ParallelMode parameter changed type, usee EParallelExecutionFlags instead.")
	UE_API void ParallelForEachEntityChunk(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext
		, const FMassExecuteFunction& ExecuteFunction, const EParallelForMode ParallelMode = Default);

	UE_DEPRECATED(5.6, "ForEachEntityChunkInCollections is deprecated. New version doesn't require the FMassEntityManager parameter")
	UE_API void ForEachEntityChunkInCollections(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections, FMassEntityManager& EntityManager
		, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction);

	UE_DEPRECATED(5.6, "ParallelForEachEntityChunkInCollection is deprecated. New version doesn't require the FMassEntityManager parameter. Also ParallelMode parameter changed type, usee EParallelExecutionFlags instead.")
	UE_API void ParallelForEachEntityChunkInCollection(TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
		, FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext, const FMassExecuteFunction& ExecuteFunction
		, const EParallelForMode ParallelMode);

	UE_DEPRECATED(5.6, "This flavor of CacheArchetypes is deprecated. EntityQueries are not tied to a specific EntityManager instance and there's no need to pass it in as a parameter")
	UE_API void CacheArchetypes(const FMassEntityManager& InEntityManager);

	UE_DEPRECATED(5.6, "This flavor of GetNumMatchingEntities is deprecated. EntityQueries are not tied to a specific EntityManager instance and there's no need to pass it in as a parameter")
	UE_API int32 GetNumMatchingEntities(FMassEntityManager& InEntityManager);

	UE_DEPRECATED(5.6, "This flavor of HasMatchingEntities is deprecated. EntityQueries are not tied to a specific EntityManager instance and there's no need to pass it in as a parameter")
	UE_API bool HasMatchingEntities(FMassEntityManager& InEntityManager);
};

ENUM_CLASS_FLAGS(FMassEntityQuery::EParallelExecutionFlags);

//-----------------------------------------------------------------------------
// INLINES
//-----------------------------------------------------------------------------
#if WITH_ARCHETYPE_MATCH_OVERRIDE
template <typename T> requires TArchetypeMatchOverrideConcept<T>
void FMassEntityQuery::SetArchetypeMatchOverride(const T& Context)
{
	static_assert(sizeof(T) <= sizeof(FArchetypeMatchOverride));
	static_assert(alignof(T) <= alignof(FArchetypeMatchOverride));
	static_assert(std::is_trivially_copyable_v<T>);
	static_assert(std::is_trivially_destructible_v<T>);

	check(!bHasArchetypeMatchOverride);
	bHasArchetypeMatchOverride = true;

	ArchetypeMatchOverride.Match = [](const void* TypeErasedContext, const FMassArchetypeCompositionDescriptor& Descriptor)->bool
	{
		const T* Context = static_cast<const T*>(TypeErasedContext);
		return Context->Match(Descriptor);	
	};
	FMemory::Memcpy(&ArchetypeMatchOverride.Data, &Context, sizeof(T));
}
#endif

inline void FMassEntityQuery::SetChunkFilter(const FMassChunkConditionFunction& InFunction)
{
	checkf(!HasChunkFilter(), TEXT("Chunk filter needs to be cleared before setting a new one."));
	ChunkCondition = InFunction;
}

inline bool FMassEntityQuery::IsGrouping() const
{
	return !GroupSortingSteps.IsEmpty();
}

inline const TSharedPtr<FMassEntityManager>& FMassEntityQuery::GetEntityManager() const
{
	return CachedEntityManager;
}

#undef MASS_ACHETYPE_OVERRIDE_MAX_SIZE
#undef MASS_ACHETYPE_OVERRIDE_MAX_ALIGNMENT

#undef UE_API 