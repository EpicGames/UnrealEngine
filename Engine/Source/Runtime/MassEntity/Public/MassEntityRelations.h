// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "Mass/EntityElementTypes.h"
#include "Mass/EntityHandle.h"
#include "Mass/ArchetypeGroup.h"
#include "Mass/EntityMacros.h"
#include "MassEntityRelations.generated.h"

#define UE_API MASSENTITY_API

struct FMassEntityManager;
class UMassObserverProcessor;
namespace UE::Mass
{
	struct FRelationManager;
} // namespace UE::Mass

namespace UE::Mass::Relations
{
	enum class ERemovalPolicy : uint8
	{
		/** only update the book-keeping */
		CleanUp,
		/** when a given relation gets removed destroy the source entity (eg destroy the child when ChildOf relation gets removed) */
		Destroy,
		/** external code will be called to patch up what remains off the relation */
		Splice,
		/** the user will provide the observer processor */
		Custom,

		MAX,
		Default = CleanUp,
	};

	enum class EExternalMappingRequired : uint8
	{
		No,
		Yes,
		Default = Yes, 
	};

	enum class ERelationRole : uint8
	{
		Subject,
		Object,
		MAX
	};

	inline FString LexToString(const ERelationRole Value)
	{
		return Value == ERelationRole::MAX
			? TEXT("INVALID")
			: (Value == ERelationRole::Subject
				? TEXT("Subject")
				: TEXT("Object"));
	}

	struct FRoleTraits
	{
		/** 
		 * The element type that will be added to the participating entity. Can remain empty. 
		 * Every entity will get the relation tag as well, the FRelationTypeTraits.RelationTagType
		 */
		const UScriptStruct* Element = nullptr;

		/**
		 * What to do when entities serving this role are destroyed. This value affects the whole relation, including other participants. 
		 */
		ERemovalPolicy DestructionPolicy = ERemovalPolicy::Default;

		/**
		 * "Exclusive" means there can be only one participant like this in a relation.
		 * For example, there can be only one parent in our ChildOf relation (bExclusive = true, on the "parent" participant, i.e. the `Object`)
		 * while there can be multiple children (bExclusive = false for the `Subject`)
		 */
		bool bExclusive = true;

		/**
		 * Declares whether this relation-specific implementation details provide dedicated
		 * mechanism for mapping other roles for entities serving as this role, the role these traits affect.
		 * It essentially answers whether this role needs the system to find other relation participants (`Yes`)
		 * or it can handle the task with custom code (like via the Element, option `No`).
		 * 
		 * Defaulting to `Yes` to provide reliable, out-of-the-box functionality required for
		 * fetching participants of relation instances. Set to `No` if you want to save memory by
		 * not automatically populating and utilizing FRelationData.RoleMap.
		 *
		 * @todo this functionality is not plugged in yet.
		 */
		EExternalMappingRequired RequiresExternalMapping = EExternalMappingRequired::Default;

		bool operator==(const FRoleTraits& Other) const
		{
			return Element == Other.Element
				&& DestructionPolicy == Other.DestructionPolicy
				&& bExclusive == Other.bExclusive
				&& RequiresExternalMapping == Other.RequiresExternalMapping;
		}
	};

	struct FRelationTypeTraits
	{
		UE_API FRelationTypeTraits(TNotNull<const UScriptStruct*> InRelationTagType);
		UE_API FRelationTypeTraits(const FRelationTypeTraits& Other, TNotNull<const UScriptStruct*> NewRelationTagType);
		FRelationTypeTraits(const FRelationTypeTraits& Other) = default;

		TNotNull<const UScriptStruct*> GetRelationTagType() const
		{
			return RelationTagType;
		}

		FName GetFName() const
		{
			return RelationName;
		}

		FName RelationName;

		/** Checks whether the traits are configured properly. */
		bool IsValid() const;

	private:
		/** can only be set during creation since we use the same type to create UE::Mass::FTypeHandle for the relation type */
		TNotNull<const UScriptStruct*> RelationTagType;

	public:
		/**
		 * The fragment type that will be automatically added to each relation entity created to represent relation instances
		 * @todo we can switch this to an array of fragment types or FInstancedStructs, bitsets or even FMassArchetypeCompositionDescriptor,
		 *		but first we need to see how our use-cases shape up.
		 */
		TNotNull<const UScriptStruct*> RelationFragmentType;

		/**
		 * Whether to use hierarchical archetype groups for the relation entities
		 * Set this to true of you want to do any data calculations with data stored in
		 * the relation entities, and them being processed in hierarchy order being a requirements
		 */
		bool bCreateRelationEntitiesInHierarchy = false;

		TStaticArray<FRoleTraits, static_cast<uint8>(ERelationRole::MAX)> RoleTraits;

		bool bHierarchical = false;

		FArchetypeGroupType RegisteredGroupType;

		/**
		 * Is set, gets called upon type's registration to register appropriate observers.
		 * Return value indicates whether the entity manager should register default observes as well.
		 */
		TFunction<bool(FMassEntityManager&)> RegisterObserversDelegate;

		/**
		 * Processor classes to be instantiated when the relation type gets registered.
		 * These will get auto-created only if RegisterObserversDelegate is empty or returns `true`
		 * The default values point to generic observer classes that implement the generic policies and behavior.
		 * Nulling-out any of these member variables will result in the given functionality not being implemented
		 * by the system and makes the user responsible for supplying it. 
		 */
		TWeakObjectPtr<UClass> RelationEntityCreationObserverClass;
		TWeakObjectPtr<UClass> RelationEntityDestructionObserverClass;
		TWeakObjectPtr<UClass> SubjectEntityDestructionObserverClass;
		TWeakObjectPtr<UClass> ObjectEntityDestructionObserverClass;

		void SetDebugInFix(FString&& InFix);
#if WITH_MASSENTITY_DEBUG
		UE_API FString DebugDescribeRelation(FMassEntityHandle A, FMassEntityHandle B) const;
	private:
		FString DebugInFix;
#endif // WITH_MASSENTITY_DEBUG
	};
} // namespace UE::Mass::Relations

//-----------------------------------------------------------------------------
// Relation types
//-----------------------------------------------------------------------------
/**
 * Structs extending FMassRelation represent a "concept" or a relation. These structs are
 * not intended to be stored in Mass. @see FMassRelationFragment for ways of storing relation-specific data
 */
USTRUCT()
struct FMassRelation : public FMassTag
{
	GENERATED_BODY()
};

//-----------------------------------------------------------------------------
// Relation data
//-----------------------------------------------------------------------------
/**
 * Relation fragment base. Every relation entity will get an instance of this type
 * or a type derived from it (as configured via FRelationTypeTraits.RelationFragmentType
 */
USTRUCT()
struct FMassRelationFragment : public FMassFragment
{
	GENERATED_BODY()

	/**
	 * This is the "who" part of the relation. Examples:
	 * - the Child in a ChildOf relation
	 * - the Character in a HasWeapon relation
	 * - the Employee in a WorksFor relation
	 */
	FMassEntityHandle Subject;

	/**
	 * This is the "what" or "target" part of the relation. Examples:
	 * - the Parent in a ChildOf relation
	 * - the Weapon in a HasWeapon relation
	 * - the Company in a WorksFor relation
	 */
	FMassEntityHandle Object;

	FMassEntityHandle GetRole(const int32 Index) const
	{
		return Index == 0 ? Subject : Object;
	}

	FMassEntityHandle GetRole(const UE::Mass::Relations::ERelationRole Role) const
	{
		check(Role != UE::Mass::Relations::ERelationRole::MAX);
		return GetRole(static_cast<int32>(Role));
	}
};

USTRUCT()
struct FMassRelationMappingFragment : public FMassFragment
{
	GENERATED_BODY()
};

/**
 * @todo we might want to promote the internals to FEntityIdentifier and have FMassEntityHandle derive from that.
 */
struct alignas(8) FMassRelationRoleInstanceHandle
{
	static constexpr int32 EntityIndexBits = 30;
	static constexpr int32 EntityIndexMask = (1 << 30) - 1;
	static constexpr int32 TypeMask = ~EntityIndexMask;

	FMassRelationRoleInstanceHandle() = default;

	static FMassRelationRoleInstanceHandle Create(UE::Mass::Relations::ERelationRole Role, const FMassEntityHandle RoleHandle, const FMassEntityHandle RelationEntityHandle)
	{
		check(Role != UE::Mass::Relations::ERelationRole::MAX);

		FMassRelationRoleInstanceHandle ReturnHandle;
		ReturnHandle.SetRoleEntityIndex(RoleHandle.Index);
		ReturnHandle.SetRelationEntityIndex(RelationEntityHandle.Index);
		ReturnHandle.SetRole(Role);
		check(ReturnHandle.GetRoleEntityIndex() == RoleHandle.Index);
		check(ReturnHandle.GetRelationEntityIndex() == RelationEntityHandle.Index);

		return ReturnHandle;
	}

	int32 GetRoleEntityIndex() const 
	{
		return (RoleEntity & EntityIndexMask);
	}
	MASSENTITY_API FMassEntityHandle GetRoleEntityHandle(const FMassEntityManager& EntityManager) const;

	int32 GetRelationEntityIndex() const 
	{
		return (RelationEntity & EntityIndexMask);
	}
	MASSENTITY_API FMassEntityHandle GetRelationEntityHandle(const FMassEntityManager& EntityManager) const;

	UE::Mass::Relations::ERelationRole GetRole() const
	{
		return static_cast<UE::Mass::Relations::ERelationRole>((RoleEntity & TypeMask) >> EntityIndexBits);
	}

	friend FString LexToString(const FMassRelationRoleInstanceHandle Handle)
	{
		return Handle.DebugGetDescription();
	}

	FString DebugGetDescription() const
	{
		return FString::Printf(TEXT("Relation %d role %s %d"), GetRelationEntityIndex(), *LexToString(GetRole()), GetRoleEntityIndex());
	}

	/** We're unable to tell if a given relation instance handle is valid just by looking at a handle. Only the RelationManager can answer this question. Use IsSet as first filter */
	bool IsValid() const = delete;

	struct FMassRelationRoleInstanceHandleFinder
	{
		FMassRelationRoleInstanceHandleFinder(const FMassEntityHandle EntityHandle)
			: EntityIndex(EntityHandle.Index)
		{	
		}

		bool operator()(const FMassRelationRoleInstanceHandle& Element) const
		{
			return Element.GetRoleEntityIndex() == EntityIndex;
		}

		const int32 EntityIndex;
	};

	bool operator==(const FMassRelationRoleInstanceHandle& Other) const
	{
		return RelationEntity == Other.RelationEntity && RoleEntity == Other.RoleEntity;
	}

private:
	uint32 RelationEntity = 0;
	uint32 RoleEntity = 0;

	void SetRoleEntityIndex(const int32 InIndex)
	{
		RoleEntity = (InIndex & EntityIndexMask) | (RoleEntity & TypeMask);
	}

	void SetRelationEntityIndex(const int32 InIndex)
	{
		RelationEntity = (InIndex & EntityIndexMask) | (RelationEntity & TypeMask);
	}

	void SetRole(UE::Mass::Relations::ERelationRole InRole)
	{
		RoleEntity = (RoleEntity & EntityIndexMask) | (static_cast<int32>(InRole) << EntityIndexBits);
	}
};

namespace UE::Mass
{
	template<>
	inline bool IsA<FMassRelation>(const UStruct* Struct)
	{
		return Struct && Struct->IsChildOf(FMassRelation::StaticStruct());
	}

	template<typename T>
	concept CRelation = TIsDerivedFrom<typename TRemoveReference<T>::Type, FMassRelation>::Value;
} // namespace UE::Mass

#undef UE_API
