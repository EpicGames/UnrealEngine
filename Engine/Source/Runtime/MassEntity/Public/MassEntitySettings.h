// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSettings.h"
#include "MassProcessingPhaseManager.h"
#include "MassProcessor.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "StructUtils/InstancedStruct.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntitySettings.generated.h"

#define GET_MASS_CONFIG_VALUE(a) \
	UE_DEPRECATED_MACRO(5.8, "Use GetDefault<UMassEntitySettings>() (or GetMutableDefault) directly.") \
	(GetMutableDefault<UMassEntitySettings>()->a)

/**
 * Implements the settings for the MassEntity module
 */
UCLASS(MinimalAPI, config = Mass, defaultconfig, DisplayName = "Mass Entity")
class UMassEntitySettings : public UMassModuleSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSettingsChange, const FPropertyChangedEvent& /*PropertyChangedEvent*/);
#endif // WITH_EDITORONLY_DATA
	DECLARE_MULTICAST_DELEGATE(FOnInitialized);

	MASSENTITY_API UMassEntitySettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	MASSENTITY_API void BuildProcessorListAndPhases();
	MASSENTITY_API void AddToActiveProcessorsList(TSubclassOf<UMassProcessor> ProcessorClass);

	MASSENTITY_API TConstArrayView<FMassProcessingPhaseConfig> GetProcessingPhasesConfig();
	const FMassProcessingPhaseConfig& GetProcessingPhaseConfig(const EMassProcessingPhase ProcessingPhase) const
	{
		check(ProcessingPhase != EMassProcessingPhase::MAX);
		return ProcessingPhasesConfig[static_cast<int32>(ProcessingPhase)];
	}

	FOnInitialized& GetOnInitialized()
	{
		return OnInitializedEvent;
	}

	UE_DEPRECATED(5.8, "Use the instance method GetOnInitialized")
	static FOnInitialized& GetOnInitializedEvent()
	{
		UMassEntitySettings* Settings = GetMutableDefault<UMassEntitySettings>();
		return Settings->OnInitializedEvent;
	}

#if WITH_EDITOR
	FOnSettingsChange& GetOnSettingsChange()
	{
		return OnSettingsChange;
	}

	bool IsInitialized2() const
	{
		return bInitialized;
	}

	UE_DEPRECATED(5.8, "Use the instance method IsInitialized2")
	static bool IsInitialized()
	{
		const UMassEntitySettings* Settings = GetDefault<UMassEntitySettings>();
		return Settings->bInitialized;
	}
#endif // WITH_EDITOR

protected:
#if WITH_EDITOR
	MASSENTITY_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	MASSENTITY_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	MASSENTITY_API virtual void PostInitProperties() override;
	MASSENTITY_API virtual void BeginDestroy() override;

	MASSENTITY_API void OnPostEngineInit();
	MASSENTITY_API void BuildPhases();
	MASSENTITY_API void BuildProcessorList();

private:
	void OnModulePackagesUnloaded(TConstArrayView<UPackage*> Packages);

public:
	UPROPERTY(EditDefaultsOnly, Category = Mass, config, AdvancedDisplay)
	uint32 ChunkMemorySize = 128 * 1024;

	/**
	 * The name of the file to dump the processor dependency graph. T
	 * The dot file will be put in the project log folder.
	 * To generate a svg out of that file, simply run dot executable with following parameters: -Tsvg -O filename.dot 
	 */
	UPROPERTY(EditDefaultsOnly, Category = Mass, Transient)
	FString DumpDependencyGraphFileName;

	/** Lets users configure processing phases including the composite processor class to be used as a container for the phases' processors. */
	UPROPERTY(EditDefaultsOnly, Category = Mass, config)
	FMassProcessingPhaseConfig ProcessingPhasesConfig[static_cast<uint8>(EMassProcessingPhase::MAX)];

	/** This list contains all the processors available in the given binary (including plugins). The contents are sorted by display name.*/
	UPROPERTY(VisibleAnywhere, Category = Mass, Transient, Instanced, EditFixedSize)
	TArray<TObjectPtr<UMassProcessor>> ProcessorCDOs;

protected:
#if WITH_EDITORONLY_DATA
	FOnSettingsChange OnSettingsChange;
#endif // WITH_EDITORONLY_DATA

	bool bInitialized = false;
	bool bEngineInitialized = false;

	FOnInitialized OnInitializedEvent;
};
