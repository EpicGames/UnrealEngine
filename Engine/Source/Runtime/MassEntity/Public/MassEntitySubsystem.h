// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSubsystemBase.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.generated.h"

#define UE_API MASSENTITY_API


/** 
 * The sole responsibility of this world subsystem class is to host the default instance of FMassEntityManager
 * for a given UWorld. All the gameplay-related use cases of Mass (found in MassGameplay and related plugins) 
 * use this by default. 
 */
UCLASS(MinimalAPI)
class UMassEntitySubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()

	//~USubsystem interface
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void PostInitialize() override;
	UE_API virtual void Deinitialize() override;
	//~End of USubsystem interface

public:
	UE_API UMassEntitySubsystem();

	//~UObject interface
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~End of UObject interface

	const FMassEntityManager& GetEntityManager() const
	{
		check(EntityManager);
		return *EntityManager.Get();
	}
	FMassEntityManager& GetMutableEntityManager()
	{
		check(EntityManager);
		return *EntityManager.Get();
	}

protected:
	TSharedPtr<FMassEntityManager> EntityManager;
};

#undef UE_API
