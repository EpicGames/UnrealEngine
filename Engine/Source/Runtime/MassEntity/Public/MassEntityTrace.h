// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessingTypes.h"
#include "Misc/NotNull.h"
#include "Trace/Config.h"
#include "Trace/Trace.h"

#ifndef UE_MASS_TRACE_ENABLED
#define UE_MASS_TRACE_ENABLED (UE_TRACE_ENABLED && WITH_MASSENTITY_DEBUG && !IS_PROGRAM && !UE_BUILD_SHIPPING)
#endif

#if UE_MASS_TRACE_ENABLED

struct FMassEntityHandle;
struct FMassEntityManager;
struct FMassEntityQuery;
struct FMassArchetypeData;
struct FMassArchetypeHandle;
struct FMassArchetypeCompositionDescriptor;
class UMassProcessor;	

UE_TRACE_CHANNEL_EXTERN(MassChannel, MASSENTITY_API);

class FMassTrace : public FNoncopyable
{
public:
	struct FScopedQueryForEachTrace
	{
		FScopedQueryForEachTrace(const FMassEntityQuery* InQuery);
		~FScopedQueryForEachTrace();

		void ReportArchetype(const FMassArchetypeData& Archetype);
		const FMassEntityQuery* Query = nullptr;
		int32 ArchetypeCount = 0;
		int32 ChunkCount = 0;
		int32 EntityCount = 0;
	};

	/** Set the current world elapsed time (recording time) for subsequent trace events.
	 *  Should be called once per frame/phase from code that has UWorld access.
	 *  This allows Mass trace events to be aligned with the RewindDebugger's
	 *  ElapsedTime-based scrub time, which pauses when the simulation pauses. */
	MASSENTITY_API static void SetRecordingTime(double InRecordingTime);

	MASSENTITY_API static uint64 OutputBeginPhaseWithID(const TCHAR* PhaseName);
	MASSENTITY_API static void OutputBeginPhaseRegion(const TCHAR* PhaseName);
	MASSENTITY_API static void OutputEndPhaseRegion(const TCHAR* PhaseName);
	MASSENTITY_API static void OutputEndPhaseRegion(uint64 PhaseId);

	MASSENTITY_API static void OnPhaseBegin(uint64 PhaseId);
	MASSENTITY_API static void OnPhaseEnd(uint64 PhaseId);

	MASSENTITY_API static uint64 OutputRegisterArchetype(uint64 ArchetypeID, const FMassArchetypeCompositionDescriptor& CompositionDescriptor);
	MASSENTITY_API static uint64 RegisterArchetype(const FMassArchetypeHandle& ArchetypeHandle);
	MASSENTITY_API static uint64 RegisterArchetype(const FMassArchetypeData& Data);

	MASSENTITY_API static void OutputRegisterFragment(const UScriptStruct* Struct);
	MASSENTITY_API static void RegisterFragment(const UScriptStruct* Struct);

	MASSENTITY_API static void EntityCreated(FMassEntityHandle Entity, const FMassArchetypeData& Archetype);
	MASSENTITY_API static void EntityMoved(FMassEntityHandle Entity, const FMassArchetypeData& NewArchetype);
	MASSENTITY_API static void EntityDestroyed(FMassEntityHandle Entity);
	MASSENTITY_API static void EntitiesDestroyed(TConstArrayView<FMassEntityHandle> Entities);

	MASSENTITY_API static void QueryCreated(const FMassEntityQuery* Query);
	MASSENTITY_API static void QueryDestroyed(const FMassEntityQuery* Query);
	MASSENTITY_API static void QueryRegisteredToProcessor(const FMassEntityQuery* Query, TNotNull<const UMassProcessor*> Processor);
	MASSENTITY_API static void QueryArchetypeAdded(const FMassEntityQuery* Query, const FMassArchetypeHandle& Archetype);
};

#define UE_TRACE_MASS_ARCHETYPE_CREATED(Archetype) \
	FMassTrace::RegisterArchetype(Archetype);

#define UE_TRACE_MASS_ENTITY_CREATED(Entity, Archetype) \
	FMassTrace::EntityCreated(Entity, Archetype);

#define UE_TRACE_MASS_ENTITIES_CREATED(EntityHandles, Archetype) \
	for (const FMassEntityHandle& Entity : EntityHandles) \
	{ \
		FMassTrace::EntityCreated(Entity, Archetype); \
	}

#define UE_TRACE_MASS_ENTITY_MOVED(Entity, NewArchetype) \
	FMassTrace::EntityMoved(Entity, NewArchetype);

#define UE_TRACE_MASS_ENTITIES_MOVED(EntityHandles, NewArchetype) \
	for (const FMassEntityHandle& Entity : EntityHandles) \
	{ \
		FMassTrace::EntityMoved(Entity, NewArchetype); \
	}

#define UE_TRACE_MASS_ENTITY_DESTROYED(Entity) \
	FMassTrace::EntityDestroyed(Entity);

#define UE_TRACE_MASS_ENTITIES_DESTROYED(EntityHandles) \
	for (const FMassEntityHandle& Entity : EntityHandles) \
	{ \
		FMassTrace::EntityDestroyed(Entity); \
	}
#define UE_TRACE_MASS_PHASE_BEGIN(PhaseID) \
	FMassTrace::OnPhaseBegin(PhaseID);

#define UE_TRACE_MASS_PHASE_END(PhaseID) \
	FMassTrace::OnPhaseEnd(PhaseID);

#define UE_TRACE_MASS_QUERY_CREATED() \
	FMassTrace::QueryCreated(this);

#define UE_TRACE_MASS_QUERY_DESTROYED() \
	FMassTrace::QueryDestroyed(this);

#define UE_TRACE_MASS_QUERY_REGISTERED_TO_PROCESSOR(Processor) \
	FMassTrace::QueryRegisteredToProcessor(this, Processor);

#define UE_TRACE_MASS_QUERY_ARCHETYPE_ADDED(Archetype) \
	FMassTrace::QueryArchetypeAdded(this, Archetype);

#define UE_TRACE_MASS_SET_RECORDING_TIME(RecordingTime) \
	FMassTrace::SetRecordingTime(RecordingTime);

#define UE_TRACE_SCOPED_MASS_QUERY_FOR_EACH() \
	FMassTrace::FScopedQueryForEachTrace _ScopedQueryForEachTrace(this);

#define UE_TRACE_SCOPED_MASS_QUERY_FOR_EACH_REPORT_ARCHETYPE(Archetype) \
	_ScopedQueryForEachTrace.ReportArchetype(Archetype);

#else //UE_MASS_TRACE_ENABLED

#define UE_TRACE_MASS_ARCHETYPE_CREATED(PhaseID)
#define UE_TRACE_MASS_ENTITY_CREATED(Entity, Archetype)
#define UE_TRACE_MASS_ENTITIES_CREATED(EntityHandles, Archetype)
#define UE_TRACE_MASS_ENTITY_MOVED(Entity, NewArchetype)
#define UE_TRACE_MASS_ENTITIES_MOVED(EntityHandles, NewArchetype)
#define UE_TRACE_MASS_ENTITY_DESTROYED(Entity)
#define UE_TRACE_MASS_ENTITIES_DESTROYED(EntityHandles)
#define UE_TRACE_MASS_PHASE_BEGIN(PhaseID)
#define UE_TRACE_MASS_PHASE_END(PhaseID)
#define UE_TRACE_MASS_QUERY_CREATED()
#define UE_TRACE_MASS_QUERY_DESTROYED()
#define UE_TRACE_MASS_QUERY_REGISTERED_TO_PROCESSOR(Processor)
#define UE_TRACE_MASS_QUERY_ARCHETYPE_ADDED(Archetype)
#define UE_TRACE_MASS_SET_RECORDING_TIME(RecordingTime)
#define UE_TRACE_SCOPED_MASS_QUERY_FOR_EACH()
#define UE_TRACE_SCOPED_MASS_QUERY_FOR_EACH_REPORT_ARCHETYPE(Archetype)

#endif //UE_MASS_TRACE_ENABLED