// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessingTypes.h"
#include "StructUtils/StructArrayView.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Subsystems/Subsystem.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Mass/ExternalSubsystemTraits.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "StructUtils/SharedStruct.h"
#include "MassEntityConcepts.h"
#include "Mass/TestableEnsures.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Mass/ExternalSubsystemTraits.h"
#include "Mass/EntityHandle.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassElement.h"
#include "MassEntityTypes.generated.h"


MASSENTITY_API DECLARE_LOG_CATEGORY_EXTERN(LogMass, Warning, All);

DECLARE_STATS_GROUP(TEXT("Mass"), STATGROUP_Mass, STATCAT_Advanced);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Mass Total Frame Time"), STAT_Mass_Total, STATGROUP_Mass, MASSENTITY_API);

struct FMassArchetypeData;
struct FMassEntityQuery;
struct FMassArchetypeHandle;

namespace UE::Mass
{
	template<typename T>
	static constexpr bool TAlwaysFalse = false;

	/**
	 * FExecutionLimiter is used to limit the execution of a query to a set entity count.
	 */
	struct FExecutionLimiter
	{
		friend struct ::FMassArchetypeData;
		friend struct ::FMassEntityQuery;

		explicit FExecutionLimiter(int32 InEntityLimit)
			: EntityLimit(InEntityLimit)
			, ChunkIndex(INDEX_NONE)
			, ArchetypeIndex(INDEX_NONE)
			, MaxChunkIndex(INDEX_NONE)
			, EntityCountRemaining(0)
		{
		}
		
		int32 EntityLimit;

	private:
		int32 ChunkIndex;
		int32 ArchetypeIndex;
		int32 MaxChunkIndex;
		int32 EntityCountRemaining;
	};

	enum class EIncludeSparseElements : uint8
	{
		No,
		Yes
	};
} // namespace UE::Mass

/** The type summarily describing a composition of an entity or an archetype. It contains information on both the
 *  fragments and tags */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct FMassArchetypeCompositionDescriptor
{
	FMassArchetypeCompositionDescriptor() = default;
	FMassArchetypeCompositionDescriptor(const FMassFragmentBitSet& InFragments,
		const FMassTagBitSet& InTags,
		const FMassChunkFragmentBitSet& InChunkFragments,
		const FMassSharedFragmentBitSet& InSharedFragments,
		const FMassConstSharedFragmentBitSet& InConstSharedFragments)
		: Fragments(InFragments)
		, Tags(InTags)
		, ChunkFragments(InChunkFragments)
		, SharedFragments(InSharedFragments)
		, ConstSharedFragments(InConstSharedFragments)
	{
		SetElementsBitSet();
	}

	FMassArchetypeCompositionDescriptor(TConstArrayView<const UScriptStruct*> InFragments,
		const FMassTagBitSet& InTags,
		const FMassChunkFragmentBitSet& InChunkFragments,
		const FMassSharedFragmentBitSet& InSharedFragments,
		const FMassConstSharedFragmentBitSet& InConstSharedFragments)
		: FMassArchetypeCompositionDescriptor(FMassFragmentBitSet(InFragments), InTags, InChunkFragments, InSharedFragments, InConstSharedFragments)
	{
	}

	FMassArchetypeCompositionDescriptor(TConstArrayView<FInstancedStruct> InFragmentInstances,
		const FMassTagBitSet& InTags,
		const FMassChunkFragmentBitSet& InChunkFragments,
		const FMassSharedFragmentBitSet& InSharedFragments,
		const FMassConstSharedFragmentBitSet& InConstSharedFragments)
		: FMassArchetypeCompositionDescriptor(FMassFragmentBitSet(InFragmentInstances), InTags, InChunkFragments, InSharedFragments, InConstSharedFragments)
	{
	}

	FMassArchetypeCompositionDescriptor(FMassFragmentBitSet&& InFragments,
		FMassTagBitSet&& InTags,
		FMassChunkFragmentBitSet&& InChunkFragments,
		FMassSharedFragmentBitSet&& InSharedFragments,
		FMassConstSharedFragmentBitSet&& InConstSharedFragments)
		: Fragments(MoveTemp(InFragments))
		, Tags(MoveTemp(InTags))
		, ChunkFragments(MoveTemp(InChunkFragments))
		, SharedFragments(MoveTemp(InSharedFragments))
		, ConstSharedFragments(MoveTemp(InConstSharedFragments))
	{
		SetElementsBitSet();
	}

	FMassArchetypeCompositionDescriptor(FMassElementBitSet&& InElements)
		: ElementsBitSet(MoveTemp(InElements))
	{}

	/**
	 * Constructor deliberately not marked as explicit to support near-future change where we get rid of FMassArchetypeCompositionDescriptor
	 * since it's just a wrapper for FMassElementBitSet now.
	 */
	FMassArchetypeCompositionDescriptor(const FMassElementBitSet& InElements)
		: ElementsBitSet(InElements)
	{}

	explicit MASSENTITY_API FMassArchetypeCompositionDescriptor(const FMassArchetypeHandle& ArchetypeHandle);

	void Reset()
	{
		ElementsBitSet.Reset();
	}

	/**
	 * Compares contents of two FMassArchetypeCompositionDescriptor instances, ignoring the trailing empty bits in the bitsets
	 */
	bool IsEquivalent(const FMassArchetypeCompositionDescriptor& OtherDescriptor) const
	{
		return ElementsBitSet.IsEquivalent(OtherDescriptor.GetElementsBitSet());
	}

	/**
	 * Checks whether contents of two FMassArchetypeCompositionDescriptor instances are identical.
	 */
	bool IsIdentical(const FMassArchetypeCompositionDescriptor& OtherDescriptor) const
	{
		return ElementsBitSet == OtherDescriptor.GetElementsBitSet();
	}

	bool IsEmpty() const 
	{
		return ElementsBitSet.IsEmpty();
	}

	bool HasAll(const FMassArchetypeCompositionDescriptor& OtherDescriptor) const
	{
		return ElementsBitSet.HasAll(OtherDescriptor.GetElementsBitSet());
	}

	void Append(const FMassArchetypeCompositionDescriptor& OtherDescriptor)
	{
		ElementsBitSet += OtherDescriptor.ElementsBitSet;
	}

	void Remove(const FMassArchetypeCompositionDescriptor& OtherDescriptor)
	{
		ElementsBitSet -= OtherDescriptor.ElementsBitSet;
	}

	/**
	 * Finds all the elements contained in `this` while missing in `OtherDescriptor` and returns
	 * the data as a FMassArchetypeCompositionDescriptor instance
	 */
	FMassArchetypeCompositionDescriptor CalculateDifference(const FMassArchetypeCompositionDescriptor& OtherDescriptor) const
	{
		return FMassArchetypeCompositionDescriptor(ElementsBitSet - OtherDescriptor.ElementsBitSet);
	}

	uint32 CalculateHash() const 
	{
		return GetTypeHash(ElementsBitSet);
	}

	UE_DEPRECATED(5.8, "This function is deprecated. Use FMassArchetypeCompositionDescriptor::CalculateHash() instead")
	static uint32 CalculateHash(const FMassFragmentBitSet& InFragments, const FMassTagBitSet& InTags
		, const FMassChunkFragmentBitSet& InChunkFragments, const FMassSharedFragmentBitSet& InSharedFragmentBitSet
		, const FMassConstSharedFragmentBitSet& InConstSharedFragmentBitSet)
	{
		return GetTypeHash(InFragments + InTags + InChunkFragments + InSharedFragmentBitSet + InConstSharedFragmentBitSet);
	}

	MASSENTITY_API int32 CountStoredTypes() const;

	MASSENTITY_API void DebugOutputDescription(FOutputDevice& Ar) const;

	template<typename T>
	auto GetContainer() const;

	template<typename T>
	UE_DEPRECATED(5.8, "Mutable GetContainer is deprecated")
	auto& GetContainer()
	{
		return GetElementsBitSet();
	}

	template<typename T>
	bool Contains() const;

	bool Contains(TNotNull<const UScriptStruct*> ElementType) const;

	template<typename T>
	void Add();

	void Add(TNotNull<const UScriptStruct*> ElementType);

	template<typename T>
	void Remove();

	void Remove(TNotNull<const UScriptStruct*> ElementType);

	FMassFragmentBitSet DebugGetFragments() const;
	FMassTagBitSet DebugGetTags() const;
	FMassChunkFragmentBitSet DebugGetChunkFragments() const;
	FMassSharedFragmentBitSet DebugGetSharedFragments() const;
	FMassConstSharedFragmentBitSet DebugGetConstSharedFragments() const;

	UE_DEPRECATED(5.8, "Getting a subset of composition's bitsets is slow now. Consider refactoring your code to work directly on FMassElementBitSet")
	FMassFragmentBitSet GetFragments() const;
	UE_DEPRECATED(5.8, "Getting a subset of composition's bitsets is slow now. Consider refactoring your code to work directly on FMassElementBitSet")
	FMassTagBitSet GetTags() const;
	UE_DEPRECATED(5.8, "Getting a subset of composition's bitsets is slow now. Consider refactoring your code to work directly on FMassElementBitSet")
	FMassChunkFragmentBitSet GetChunkFragments() const;
	UE_DEPRECATED(5.8, "Getting a subset of composition's bitsets is slow now. Consider refactoring your code to work directly on FMassElementBitSet")
	FMassSharedFragmentBitSet GetSharedFragments() const;
	UE_DEPRECATED(5.8, "Getting a subset of composition's bitsets is slow now. Consider refactoring your code to work directly on FMassElementBitSet")
	FMassConstSharedFragmentBitSet GetConstSharedFragments() const;

	/**
	 * Functions setting specific type of bits. These functions are a lot slower now.
	 * Consider changing your code to work directly on FMassElementBitSet
	 */
	UE_DEPRECATED(5.8, "Setting a subset of composition's bitsets is slow now. Consider refactoring your code to work directly on FMassElementBitSet")
	void SetFragments(const FMassFragmentBitSet& InBitSet);
	UE_DEPRECATED(5.8, "Setting a subset of composition's bitsets is slow now. Consider refactoring your code to work directly on FMassElementBitSet")
	void SetTags(const FMassTagBitSet& InBitSet);
	UE_DEPRECATED(5.8, "Setting a subset of composition's bitsets is slow now. Consider refactoring your code to work directly on FMassElementBitSet")
	void SetChunkFragments(const FMassChunkFragmentBitSet& InBitSet);
	UE_DEPRECATED(5.8, "Setting a subset of composition's bitsets is slow now. Consider refactoring your code to work directly on FMassElementBitSet")
	void SetSharedFragments(const FMassSharedFragmentBitSet& InBitSet);
	UE_DEPRECATED(5.8, "Setting a subset of composition's bitsets is slow now. Consider refactoring your code to work directly on FMassElementBitSet")
	void SetConstSharedFragments(const FMassConstSharedFragmentBitSet& InBitSet);
	
	void SetAllSharedElements(const FMassElementBitSet& InBitSet);

	FMassElementBitSet GetAllSharedFragments() const
	{
		return ElementsBitSet & (FMassElementBitSet::GetAllSharedFragments());
	}

	const FMassElementBitSet& GetElementsBitSet() const
	{
		return ElementsBitSet;
	}

	FMassElementBitSet& GetElementsBitSet()
	{
		return ElementsBitSet;
	}

	UE_DEPRECATED(5.7, "Direct access to FMassArchetypeCompositionDescriptor's bitsets is deprecated. Use any of the newly added getters instead.")
	FMassFragmentBitSet Fragments;

	UE_DEPRECATED(5.7, "Direct access to FMassArchetypeCompositionDescriptor's bitsets is deprecated. Use any of the newly added getters instead.")
	FMassTagBitSet Tags;

	UE_DEPRECATED(5.7, "Direct access to FMassArchetypeCompositionDescriptor's bitsets is deprecated. Use any of the newly added getters instead.")
	FMassChunkFragmentBitSet ChunkFragments;

	UE_DEPRECATED(5.7, "Direct access to FMassArchetypeCompositionDescriptor's bitsets is deprecated. Use any of the newly added getters instead.")
	FMassSharedFragmentBitSet SharedFragments;

	UE_DEPRECATED(5.7, "Direct access to FMassArchetypeCompositionDescriptor's bitsets is deprecated. Use any of the newly added getters instead.")
	FMassConstSharedFragmentBitSet ConstSharedFragments;

protected:
	FMassElementBitSet ElementsBitSet;

public:
	template<typename TBitSet>
	TBitSet Get() const
	{
		return ElementsBitSet.Get<TBitSet>();
	}

	void SetElementsBitSet()
	{
		ElementsBitSet = Fragments;
		ElementsBitSet += Tags;
		ElementsBitSet += ChunkFragments;
		ElementsBitSet += SharedFragments;
		ElementsBitSet += ConstSharedFragments;
	}

	template<typename TElementType>
	bool HasAny() const
	{
		return ElementsBitSet.HasAny<TElementType>();
	}

	template<typename TElementType>
	bool HasAll() const
	{
		return ElementsBitSet.HasAll<TElementType>();
	}
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**
 * Wrapper for const and non-const shared fragment containers that tracks which struct types it holds (via a FMassSharedFragmentBitSet).
 * Note that having multiple instances of a given struct type is not supported and Add* functions will fetch the previously
 * added fragment instead of adding a new one.
 *
 * Shared fragment TYPES contribute to archetype composition (two archetypes with different shared fragment type sets are distinct).
 * Shared fragment VALUES are stored per-chunk — entities within the same archetype but with different shared values are placed
 * in separate chunks. Chunk selection uses IsEquivalent() which compares by pointer identity (hash of backing struct memory
 * addresses). When shared fragments are obtained via GetOrCreateConstSharedFragment/GetOrCreateSharedFragment, identical
 * values are deduplicated (same pointer), so IsEquivalent effectively acts as value comparison in the expected usage path.
 */
struct FMassArchetypeSharedFragmentValues
{
	FMassArchetypeSharedFragmentValues() = default;
	
	FMassArchetypeSharedFragmentValues(const FMassArchetypeSharedFragmentValues& Other)
		: HashCache(Other.HashCache)
		, bSorted(Other.bSorted)
		, StoredElementsBitSet(Other.StoredElementsBitSet)
		, ConstSharedFragments(Other.ConstSharedFragments)
		, SharedFragments(Other.SharedFragments)
	{
	}

	FMassArchetypeSharedFragmentValues(FMassArchetypeSharedFragmentValues&& Other)
		: HashCache(Other.HashCache)
		, bSorted(Other.bSorted)
		, StoredElementsBitSet(MoveTemp(Other.StoredElementsBitSet))
		, ConstSharedFragments(MoveTemp(Other.ConstSharedFragments))
		, SharedFragments(MoveTemp(Other.SharedFragments))
	{
	}

	FMassArchetypeSharedFragmentValues& operator=(const FMassArchetypeSharedFragmentValues& Other)
	{
		if (this != &Other)
		{
			UE::TWriteScopeLock WriteLock{ HashLock };

			StoredElementsBitSet = Other.StoredElementsBitSet;
			ConstSharedFragments = Other.ConstSharedFragments;
			SharedFragments = Other.SharedFragments;
			HashCache = Other.HashCache;
			bSorted = Other.bSorted;
		}
		return *this;
	}

	FMassArchetypeSharedFragmentValues& operator=(FMassArchetypeSharedFragmentValues&& Other) noexcept
	{
		if (this != &Other)
		{
			UE::TWriteScopeLock WriteLock{ HashLock };

			StoredElementsBitSet = MoveTemp(Other.StoredElementsBitSet);
			ConstSharedFragments = MoveTemp(Other.ConstSharedFragments);
			SharedFragments = MoveTemp(Other.SharedFragments);
			HashCache = Other.HashCache;
			bSorted = Other.bSorted;
		}
		return *this;
	}

	/**
	 * @param SharedFragmentValuesModification if provided will be used to override or add values already existing in Original.
	 *	Needs to be consistent with NewSharedElementComposition, meaning, not add anything that's not already in the bitset. 
	 * Also, if NewSharedElementComposition contains elements not in Original, then SharedFragmentValuesModification has to provide a value for it.  
	 */
	MASSENTITY_API static FMassArchetypeSharedFragmentValues CreateCombined(const FMassArchetypeSharedFragmentValues& Original
		, const FMassElementBitSet& NewSharedElementComposition, const FMassArchetypeSharedFragmentValues* SharedFragmentValuesModification);

	/**
	 * @return whether the stored shared fragment values exactly match shared fragment types indicated by InDescriptor
	 */
	bool DoesMatchComposition(const FMassArchetypeCompositionDescriptor& InDescriptor) const 
	{
		return DoContainEquivalentSharedFragments(StoredElementsBitSet, InDescriptor.GetElementsBitSet());
	}

	bool DoesMatchComposition(const FMassElementBitSet& CompositionBitSet) const 
	{
		return DoContainEquivalentSharedFragments(StoredElementsBitSet, CompositionBitSet);
	}

	inline bool IsEquivalent(const FMassArchetypeSharedFragmentValues& OtherSharedFragmentValues) const
	{
		return GetTypeHash(*this) == GetTypeHash(OtherSharedFragmentValues);
	}

	/** 
	 * Compares contents of `this` and the Other, and allows different order of elements in both containers.
	 * Note that the function ignores "nulls", i.e. empty FConstSharedStruct and FSharedStruct instances. The function
	 * does care however about matching "mode", meaning ConstSharedFragments and SharedFragments arrays are compared
	 * independently.
	 */
	MASSENTITY_API bool HasSameValues(const FMassArchetypeSharedFragmentValues& Other) const;

	inline bool ContainsType(const UScriptStruct* FragmentType) const
	{
		return (UE::Mass::IsA<FMassSharedFragment>(FragmentType) || UE::Mass::IsA<FMassConstSharedFragment>(FragmentType))
			&& StoredElementsBitSet.Contains(FragmentType);
	}

	template<typename T>
	inline bool ContainsType() const
	{
		if constexpr (UE::Mass::CConstSharedFragment<T> || UE::Mass::CSharedFragment<T>)
		{
			return StoredElementsBitSet.Contains(T::StaticStruct());
		}		
		else
		{
			return false;
		}
	}

	/**
	 * Adds Fragment to the collection.
	 * Method will ensure if a fragment of the given FMassConstSharedFragment subclass has already been added.
	 */
	void Add(const FConstSharedStruct& Fragment)
	{
		(void)Add_GetRef(Fragment);
	}

	/** 
	 * Adds Fragment to the collection.
	 * Method will ensure if a fragment of the given FMassConstSharedFragment subclass has already been added.
	 * In that case the method will return the previously added instance if the given type has been added
	 * as a CONST shared fragment and if not it will return an empty FConstSharedStruct.
	 */
	MASSENTITY_API FConstSharedStruct Add_GetRef(const FConstSharedStruct& Fragment);

	UE_DEPRECATED(5.6, "Use Add or Add_GetRef instead depending on whether you need the return value.")
		FConstSharedStruct AddConstSharedFragment(const FConstSharedStruct& Fragment)
	{
		return Add_GetRef(Fragment);
	}

	/**
	 * Adds Fragment to the collection.
	 * Method will ensure if a fragment of the given FMassSharedFragment subclass has already been added.
	 */
	void Add(const FSharedStruct& Fragment)
	{
		(void)Add_GetRef(Fragment);
	}

	/** 
	 * Adds Fragment to the collection.
	 * Method will ensure if a fragment of the given FMassSharedFragment subclass has already been added.
	 * In that case the method will return the previously added instance if the given type has been added
	 * as a NON-CONST shared fragment and if not it will return an empty FSharedStruct.
	 */
	MASSENTITY_API FSharedStruct Add_GetRef(const FSharedStruct& Fragment);

	UE_DEPRECATED(5.6, "Use Add or Add_GetRef instead depending on whether you need the return value.")
	FSharedStruct AddSharedFragment(const FSharedStruct& Fragment)
	{
		return Add_GetRef(Fragment);
	}

	/**
	 * Finds instances of fragment types given by Fragments and replaces their values with contents of respective
	 * element of Fragments.
	 * Note that it's callers responsibility to ensure every fragment type in Fragments already has an instance in
	 * this FMassArchetypeSharedFragmentValues instance. Failing that assumption will result in ensure failure. 
	 */
	template<typename TSharedStruct>
	void ReplaceSharedFragments(TArrayView<TSharedStruct> Fragments)
	{
		using FDecayedSharedStruct = TDecay<TSharedStruct>::Type;

		DirtyHashCache();
		for (const TSharedStruct& NewFragment : Fragments)
		{
			const UScriptStruct* NewFragScriptStruct = NewFragment.GetScriptStruct();
			check(NewFragScriptStruct);

			bool bEntryFound = false;
			for (FDecayedSharedStruct& MyFragment : GetMutableFragmentsContainer<FDecayedSharedStruct>())
			{
				if (MyFragment.GetScriptStruct() == NewFragScriptStruct)
				{
					MyFragment = NewFragment;
					bEntryFound = true;
					break;
				}
			}
			ensureMsgf(bEntryFound, TEXT("Existing fragment of type %s could not be found"), *GetNameSafe(NewFragScriptStruct));
		}
	}

	/** 
	 * Appends contents of Other to `this` instance. All common fragments will get overridden with values in Other.
	 * Note that changing a fragments "role" (being const or non-const) is not supported and the function will fail an
	 * ensure when that is attempted.
	 * @return number of fragments added or changed
	 */
	MASSENTITY_API int32 Append(const FMassArchetypeSharedFragmentValues& Other);

	MASSENTITY_API int32 Remove(const FMassElementBitSet& ElementBitSet);

	/** 
	 * Note that the function removes the shared fragments by type
	 * @return number of fragments types removed
	 */
	int32 Remove(const FMassSharedFragmentBitSet& SharedFragmentToRemoveBitSet)
	{
		return Remove(FMassElementBitSet(SharedFragmentToRemoveBitSet));
	}

	/** 
	 * Note that the function removes the const shared fragments by type
	 * @return number of fragments types removed
	 */
	int32 Remove(const FMassConstSharedFragmentBitSet& ConstSharedFragmentToRemoveBitSet)
	{
		return Remove(FMassElementBitSet(ConstSharedFragmentToRemoveBitSet));
	}

	/**
	 * Remove all the shared and const shared fragments indicated by InDescriptor
	 * @return number of fragments types removed
	 */
	int32 Remove(const FMassArchetypeCompositionDescriptor& InDescriptor)
	{
		return Remove(InDescriptor.GetAllSharedFragments());
	}

	int32 Remove(const FMassArchetypeSharedFragmentValues& Other)
	{
		return Remove(Other.GetBitSet());
	}

	inline const TArray<FConstSharedStruct>& GetConstSharedFragments() const
	{
		return ConstSharedFragments;
	}

	inline TArray<FSharedStruct>& GetMutableSharedFragments()
	{
		return SharedFragments;
	}

	/** depending on TSharedStruct returns Shared or ConstShared fragment container */
	template<typename TSharedStruct>
	TArray<TSharedStruct>& GetMutableFragmentsContainer();
	
	inline const TArray<FSharedStruct>& GetSharedFragments() const
	{
		return SharedFragments;
	}
	
	FConstSharedStruct GetConstSharedFragmentStruct(const UScriptStruct* StructType) const
	{
		const int32 FragmentIndex = ConstSharedFragments.IndexOfByPredicate(FStructTypeEqualOperator(StructType));
		return FragmentIndex != INDEX_NONE ? ConstSharedFragments[FragmentIndex] : FConstSharedStruct();
	}
		
	FSharedStruct GetSharedFragmentStruct(const UScriptStruct* StructType)
	{
		const int32 FragmentIndex = SharedFragments.IndexOfByPredicate(FStructTypeEqualOperator(StructType));
		return FragmentIndex != INDEX_NONE ? SharedFragments[FragmentIndex] : FSharedStruct();
	}

	FConstSharedStruct GetSharedFragmentStruct(const UScriptStruct* StructType) const
	{
		const int32 FragmentIndex = SharedFragments.IndexOfByPredicate(FStructTypeEqualOperator(StructType));
		return FragmentIndex != INDEX_NONE ? SharedFragments[FragmentIndex] : FSharedStruct();
	}

	const FMassElementBitSet& GetBitSet() const
	{
		return StoredElementsBitSet;
	}

	UE_DEPRECATED(5.8, "This function is deprecated. Use GetBitSet instead")
	FMassSharedFragmentBitSet GetSharedFragmentBitSet() const
	{
		return GetBitSet();
	}

	UE_DEPRECATED(5.8, "This function is deprecated. Use GetBitSet instead")
	FMassConstSharedFragmentBitSet GetConstSharedFragmentBitSet() const
	{
		return GetBitSet();
	}

	inline void DirtyHashCache()
	{
		UE::TWriteScopeLock ScopeLock{ HashLock };
		HashCache = UINT32_MAX;
		// we consider a single shared fragment as being "sorted"
		bSorted = (SharedFragments.Num() + ConstSharedFragments.Num() <= 1);
	}

	/** Returns the cached-or-calculated hash */
	inline uint32 CacheHash() const
	{
		{
			UE::TReadScopeLock ReadLock{ HashLock };
			if (HashCache != UINT32_MAX)
			{
				return HashCache;
			}
		}

		{
			UE::TWriteScopeLock WriteLock{ HashLock };
			HashCache = CalculateHash();
			return HashCache;
		}
	}

	friend inline uint32 GetTypeHash(const FMassArchetypeSharedFragmentValues& SharedFragmentValues)
	{
		return SharedFragmentValues.CacheHash();
	}

	MASSENTITY_API uint32 CalculateHash() const;
	SIZE_T GetAllocatedSize() const;

	void Sort()
	{
		if(!bSorted)
		{
			ConstSharedFragments.Sort(FStructTypeSortOperator());
			SharedFragments.Sort(FStructTypeSortOperator());
			bSorted = true;
		}
	}

	bool IsSorted() const;

	bool IsEmpty() const;

	void Reset();

protected:
	mutable uint32 HashCache = UINT32_MAX;
	/**
	 * We consider empty FMassArchetypeSharedFragmentValues a sorted container.Same goes for a container containing
	 * a single element, @see DirtyHashCache
	 */ 
	mutable bool bSorted = true; 
	
	FMassElementBitSet StoredElementsBitSet;
	TArray<FConstSharedStruct> ConstSharedFragments;
	TArray<FSharedStruct> SharedFragments;

private:
	mutable FTransactionallySafeRWLock HashLock;
};

/**
 * The enum is used to categorize any operation an entity can be a subject to.
 */
UENUM()
enum class EMassObservedOperation : uint8
{
	AddElement,	// when an element (a fragment, tag...) is added to an existing entity
	RemoveElement,	// when an element (a fragment, tag...) is removed from an existing entity
	DestroyEntity,	// when an entity is destroyed, which is a special case of RemoveElement, because the entity gets all of its elements removed
	CreateEntity,	// when an entity is created, which is a special case of AddElement, because the entity gets all of its elements added

	// @todo another planned supported operation type
	// Touch,
	// -- new operations above this line -- //

	MAX,

	// the following values are deprecated. Use one of the values above 
	Add UMETA(Deprecated, DisplayName="DEPRECATED_Add"),
	Remove UMETA(Deprecated, DisplayName="DEPRECATED_Remove")
};

enum class EMassObservedOperationFlags : uint8
{
	None = 0,
	AddElement = 1 << static_cast<uint8>(EMassObservedOperation::AddElement),
	RemoveElement = 1 << static_cast<uint8>(EMassObservedOperation::RemoveElement),
	CreateEntity = 1 << static_cast<uint8>(EMassObservedOperation::CreateEntity),
	DestroyEntity = 1 << static_cast<uint8>(EMassObservedOperation::DestroyEntity),
	
	Add = AddElement | CreateEntity,
	Remove = RemoveElement | DestroyEntity,
	All = Add | Remove,
};
ENUM_CLASS_FLAGS(EMassObservedOperationFlags);
MASSENTITY_API FString LexToString(const EMassObservedOperationFlags Value);

enum class EMassExecutionContextType : uint8
{
	Local,
	Processor,
	MAX
};

/** 
 * Note that this is a view and is valid only as long as the source data is valid. Used when flushing mass commands to
 * wrap different kinds of data into a uniform package so that it can be passed over to a common interface.
 */
struct FMassGenericPayloadView
{
	FMassGenericPayloadView() = default;
	FMassGenericPayloadView(TArray<FStructArrayView>& SourceData)
		: Content(SourceData)
	{}
	FMassGenericPayloadView(TArrayView<FStructArrayView> SourceData)
		: Content(SourceData)
	{}

	int32 Num() const
	{
		return Content.Num();
	}

	void Reset()
	{
		Content = TArrayView<FStructArrayView>();
	}

	inline void Swap(const int32 A, const int32 B)
	{
		for (FStructArrayView& View : Content)
		{
			View.Swap(A, B);
		}
	}

	/** Moves NumToMove elements to the back of the viewed collection. */
	void SwapElementsToEnd(int32 StartIndex, int32 NumToMove);

	TArrayView<FStructArrayView> Content;
};

/**
 * Used to indicate a specific slice of a preexisting FMassGenericPayloadView, it's essentially an access pattern
 * Note: accessing content generates copies of FStructArrayViews stored (still cheap, those are just views). 
 */
struct FMassGenericPayloadViewSlice
{
	FMassGenericPayloadViewSlice() = default;
	FMassGenericPayloadViewSlice(const FMassGenericPayloadView& InSource, const int32 InStartIndex, const int32 InCount)
		: Source(InSource), StartIndex(InStartIndex), Count(InCount)
	{
	}

	FStructArrayView operator[](const int32 Index) const
	{
		return Source.Content[Index].Slice(StartIndex, Count);
	}

	/** @return the number of "layers" (i.e. number of original arrays) this payload has been built from */
	int32 Num() const 
	{
		return Source.Num();
	}

	bool IsEmpty() const
	{
		return !(Source.Num() > 0 && Count > 0);
	}

private:
	FMassGenericPayloadView Source;
	const int32 StartIndex = 0;
	const int32 Count = 0;
};

namespace UE::Mass
{
	/**
	 * A statically-typed list of related types. Used mainly to differentiate type collections at compile-type as well as
	 * efficiently produce TStructTypeBitSet representing given collection.
	 */
	template<typename T, typename... TOthers>
	struct TMultiTypeList : TMultiTypeList<TOthers...>
	{
		using Super = TMultiTypeList<TOthers...>;
		using FType = std::remove_const_t<typename TRemoveReference<T>::Type>;
		enum
		{
			Ordinal = Super::Ordinal + 1
		};

		template<typename TBitSetType>
		constexpr static void PopulateBitSet(TBitSetType& OutBitSet)
		{
			Super::PopulateBitSet(OutBitSet);
			OutBitSet += TBitSetType::template GetTypeBitSet<FType>();
		}
	};
		
	/** Single-type specialization of TMultiTypeList. */
	template<typename T>
	struct TMultiTypeList<T>
	{
		using FType = std::remove_const_t<typename TRemoveReference<T>::Type>;
		enum
		{
			Ordinal = 0
		};

		template<typename TBitSetType>
		constexpr static void PopulateBitSet(TBitSetType& OutBitSet)
		{
			OutBitSet += TBitSetType::template GetTypeBitSet<FType>();
		}
	};

	/** 
	 * The type hosts a statically-typed collection of TArrays, where each TArray is strongly-typed (i.e. it contains 
	 * instances of given structs rather than structs wrapped up in FInstancedStruct). This type lets us do batched 
	 * fragment values setting by simply copying data rather than setting per-instance. 
	 */
	template<typename T, typename... TOthers>
	struct TMultiArray : TMultiArray<TOthers...>
	{
		using FType = std::remove_const_t<typename TRemoveReference<T>::Type>;
		using Super = TMultiArray<TOthers...>;

		enum
		{
			Ordinal = Super::Ordinal + 1
		};

		SIZE_T GetAllocatedSize() const
		{
			return FragmentInstances.GetAllocatedSize() + Super::GetAllocatedSize();
		}

		int GetNumArrays() const
		{
			return Ordinal + 1;
		}

		/** TInstances might be different from TOthers if move semantics were used */
		template<typename... TInstances>
		void Add(const FType& Item, TInstances... Rest)
		{
			FragmentInstances.Add(Item);
			Super::Add(Forward<TInstances>(Rest)...);
		}

		/** TInstances might be different from TOthers if move semantics were used */
		template<typename... TInstances>
		void Add(FType&& Item, TInstances... Rest)
		{
			FragmentInstances.Emplace(Forward<FType>(Item));
			Super::Add(Forward<TInstances>(Rest)...);
		}

		template<typename AllocatorType = FDefaultAllocator>
		void GetAsGenericMultiArray(TArray<FStructArrayView, AllocatorType>& A) /*const*/
		{
			Super::GetAsGenericMultiArray(A);
			A.Add(FStructArrayView(FragmentInstances));
		}

		/** Like GetAsGenericMultiArray but skips FMassTag-derived types (including FMassSparseTag).
		 *  Tags have zero-size storage and must be excluded from payload arrays passed to BatchSetFragmentValues. */
		template<typename AllocatorType = FDefaultAllocator>
		void GetAsNonTagGenericMultiArray(TArray<FStructArrayView, AllocatorType>& A) /*const*/
		{
			Super::GetAsNonTagGenericMultiArray(A);
			if constexpr (!TIsDerivedFrom<FType, FMassTag>::Value)
			{
				A.Add(FStructArrayView(FragmentInstances));
			}
		}

		void GatherAffectedFragments(FMassFragmentBitSet& OutBitSet) const
		{
			Super::GatherAffectedFragments(OutBitSet);
			OutBitSet += FMassFragmentBitSet::GetTypeBitSet<FType>();
		}

		UE_DEPRECATED(5.8, "Function name's spelling has been fixed. Use GatherAffectedFragments now")
		void GetheredAffectedFragments(FMassFragmentBitSet& OutBitSet) const
		{
			GatherAffectedFragments(OutBitSet);
		}

		void Reset()
		{
			Super::Reset();
			FragmentInstances.Reset();
		}

		TArray<FType> FragmentInstances;
	};

	/** TMultiArray single-type specialization */
	template<typename T>
	struct TMultiArray<T>
	{
		using FType = std::remove_const_t<typename TRemoveReference<T>::Type>;
		enum { Ordinal = 0 };

		SIZE_T GetAllocatedSize() const
		{
			return FragmentInstances.GetAllocatedSize();
		}

		int GetNumArrays() const
		{
			return Ordinal + 1;
		}

		void Add(const FType& Item)
		{
			FragmentInstances.Add(Item);
		}

		void Add(FType&& Item)
		{
			FragmentInstances.Emplace(Forward<FType>(Item));
		}

		template<typename AllocatorType = FDefaultAllocator>
		void GetAsGenericMultiArray(TArray<FStructArrayView, AllocatorType>& A) /*const*/
		{
			A.Add(FStructArrayView(FragmentInstances));
		}

		/** Like GetAsGenericMultiArray but skips FMassTag-derived types (including FMassSparseTag). */
		template<typename AllocatorType = FDefaultAllocator>
		void GetAsNonTagGenericMultiArray(TArray<FStructArrayView, AllocatorType>& A) /*const*/
		{
			if constexpr (!TIsDerivedFrom<FType, FMassTag>::Value)
			{
				A.Add(FStructArrayView(FragmentInstances));
			}
		}

		void GatherAffectedFragments(FMassFragmentBitSet& OutBitSet) const
		{
			OutBitSet += FMassFragmentBitSet::GetTypeBitSet<FType>();
		}

		UE_DEPRECATED(5.8, "Function name's spelling has been fixed. Use GatherAffectedFragments now")
		void GetheredAffectedFragments(FMassFragmentBitSet& OutBitSet) const
		{
			GatherAffectedFragments(OutBitSet);
		}

		void Reset()
		{
			FragmentInstances.Reset();
		}

		TArray<FType> FragmentInstances;
	};

} // namespace UE::Mass


struct FMassArchetypeCreationParams
{
	FMassArchetypeCreationParams() = default;
	explicit FMassArchetypeCreationParams(const FName DebugName)
		: DebugName(DebugName)
	{
	}

	explicit FMassArchetypeCreationParams(const struct FMassArchetypeData& Archetype);

	/** Created archetype will have chunks of this size. 0 denotes "use default" (see UE::Mass::ChunkSize) */
	int32 ChunkMemorySize = 0;

	/** Name to identify the archetype while debugging*/
	FName DebugName;

#if WITH_MASSENTITY_DEBUG
	FColor DebugColor{0};
#endif // WITH_MASSENTITY_DEBUG
};

//-----------------------------------------------------------------------------
// INLINES
//-----------------------------------------------------------------------------
PRAGMA_DISABLE_DEPRECATION_WARNINGS
template<typename T>
auto FMassArchetypeCompositionDescriptor::GetContainer() const
{
	if constexpr (std::is_same_v<FMassFragment, T>)
	{
		return ElementsBitSet.Get<FMassFragmentBitSet>();
	}
	else if constexpr (std::is_same_v<FMassTag, T>)
	{
		return ElementsBitSet.Get<FMassTagBitSet>();
	}
	else if constexpr (std::is_same_v<FMassChunkFragment, T>)
	{
		return ElementsBitSet.Get<FMassChunkFragmentBitSet>();
	}
	else if constexpr (std::is_same_v<FMassSharedFragment, T>)
	{
		return ElementsBitSet.Get<FMassSharedFragmentBitSet>();
	}
	else if constexpr (std::is_same_v<FMassConstSharedFragment, T>)
	{
		return ElementsBitSet.Get<FMassConstSharedFragmentBitSet>();
	}
	else
	{
		static_assert(UE::Mass::TAlwaysFalse<T>, "Unknown element type passed to GetContainer.");
	}
}

template<typename T>
bool FMassArchetypeCompositionDescriptor::Contains() const
{
	return ElementsBitSet.Contains(T::StaticStruct());
}

inline bool FMassArchetypeCompositionDescriptor::Contains(TNotNull<const UScriptStruct*> ElementType) const
{
	return ElementsBitSet.Contains(ElementType);
}

template<typename T>
void FMassArchetypeCompositionDescriptor::Add()
{
	return ElementsBitSet.Add(T::StaticStruct());
}

inline void FMassArchetypeCompositionDescriptor::Add(TNotNull<const UScriptStruct*> ElementType)
{
	return ElementsBitSet.Add(ElementType);
}

template<typename T>
void FMassArchetypeCompositionDescriptor::Remove()
{
	return ElementsBitSet.Remove(T::StaticStruct());
}

inline void FMassArchetypeCompositionDescriptor::Remove(TNotNull<const UScriptStruct*> ElementType)
{
	return ElementsBitSet.Remove(ElementType);
}

inline FMassFragmentBitSet FMassArchetypeCompositionDescriptor::DebugGetFragments() const 
{ 
	return ElementsBitSet.Get<FMassFragmentBitSet>();
}

inline FMassTagBitSet FMassArchetypeCompositionDescriptor::DebugGetTags() const 
{ 
	return ElementsBitSet.Get<FMassTagBitSet>();
}

inline FMassChunkFragmentBitSet FMassArchetypeCompositionDescriptor::DebugGetChunkFragments() const 
{ 
	return ElementsBitSet.Get<FMassChunkFragmentBitSet>();
}

inline FMassSharedFragmentBitSet FMassArchetypeCompositionDescriptor::DebugGetSharedFragments() const 
{ 
	return ElementsBitSet.Get<FMassSharedFragmentBitSet>();
}

inline FMassConstSharedFragmentBitSet FMassArchetypeCompositionDescriptor::DebugGetConstSharedFragments() const 
{ 
	return ElementsBitSet.Get<FMassConstSharedFragmentBitSet>();
}

inline FMassFragmentBitSet FMassArchetypeCompositionDescriptor::GetFragments() const 
{ 
	return DebugGetFragments(); 
}

inline FMassTagBitSet FMassArchetypeCompositionDescriptor::GetTags() const 
{ 
	return DebugGetTags(); 
}

inline FMassChunkFragmentBitSet FMassArchetypeCompositionDescriptor::GetChunkFragments() const 
{ 
	return DebugGetChunkFragments(); 
}

inline FMassSharedFragmentBitSet FMassArchetypeCompositionDescriptor::GetSharedFragments() const 
{ 
	return DebugGetSharedFragments(); 
}

inline FMassConstSharedFragmentBitSet FMassArchetypeCompositionDescriptor::GetConstSharedFragments() const 
{ 
	return DebugGetConstSharedFragments(); 
}

inline void FMassArchetypeCompositionDescriptor::SetFragments(const FMassFragmentBitSet& InBitSet)
{ 
	ElementsBitSet = (ElementsBitSet - UE::Mass::FElementBitSet::GetTypeBitArray(UE::Mass::EElementType::Fragment)) + InBitSet;
}

inline void FMassArchetypeCompositionDescriptor::SetTags(const FMassTagBitSet& InBitSet)
{ 
	ElementsBitSet = (ElementsBitSet - UE::Mass::FElementBitSet::GetTypeBitArray(UE::Mass::EElementType::Tag)) + InBitSet;
}

inline void FMassArchetypeCompositionDescriptor::SetChunkFragments(const FMassChunkFragmentBitSet& InBitSet)
{ 
	ElementsBitSet = (ElementsBitSet - UE::Mass::FElementBitSet::GetTypeBitArray(UE::Mass::EElementType::ChunkFragment)) + InBitSet;
}

inline void FMassArchetypeCompositionDescriptor::SetSharedFragments(const FMassSharedFragmentBitSet& InBitSet)
{ 
	ElementsBitSet = (ElementsBitSet - UE::Mass::FElementBitSet::GetTypeBitArray(UE::Mass::EElementType::SharedFragment)) + InBitSet;
}

inline void FMassArchetypeCompositionDescriptor::SetConstSharedFragments(const FMassConstSharedFragmentBitSet& InBitSet)
{ 
	ElementsBitSet = (ElementsBitSet - UE::Mass::FElementBitSet::GetTypeBitArray(UE::Mass::EElementType::ConstSharedFragment)) + InBitSet;
}

inline void FMassArchetypeCompositionDescriptor::SetAllSharedElements(const FMassElementBitSet& InBitSet)
{ 
	ElementsBitSet = (ElementsBitSet - ElementsBitSet.GetAllSharedFragments()) + InBitSet;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

inline SIZE_T FMassArchetypeSharedFragmentValues::GetAllocatedSize() const
{
	return StoredElementsBitSet.GetAllocatedSize()
		+ ConstSharedFragments.GetAllocatedSize()
		+ SharedFragments.GetAllocatedSize();
}

inline bool FMassArchetypeSharedFragmentValues::IsSorted() const
{
	return bSorted;
}

inline bool FMassArchetypeSharedFragmentValues::IsEmpty() const
{
	return ConstSharedFragments.IsEmpty() && SharedFragments.IsEmpty();
}

inline void FMassArchetypeSharedFragmentValues::Reset()
{
	UE::TWriteScopeLock ScopeLock{ HashLock };

	HashCache = UINT32_MAX;
	bSorted = false; 
	StoredElementsBitSet.Reset();
	ConstSharedFragments.Reset();
	SharedFragments.Reset();
}

template<>
inline TArray<FSharedStruct>& FMassArchetypeSharedFragmentValues::GetMutableFragmentsContainer<FSharedStruct>()
{
	return SharedFragments;
}

template<>
inline TArray<FConstSharedStruct>& FMassArchetypeSharedFragmentValues::GetMutableFragmentsContainer<FConstSharedStruct>()
{
	return ConstSharedFragments;
}