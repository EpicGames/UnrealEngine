// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessingTypes.h"
#include "MassArchetypeTypes.h"

class UWorld;
struct FMassEntityManager;
struct FMassEntityHandle;

namespace UE::Mass::Utils
{

/** returns the current execution mode for the processors calculated from the world network mode */
MASSENTITY_API extern EProcessorExecutionFlags GetProcessorExecutionFlagsForWorld(const UWorld& World);

/** based on the given World (which can be null) and 'ExecutionFlagsOverride', the function determines the execution flags to use */
MASSENTITY_API extern EProcessorExecutionFlags DetermineProcessorExecutionFlags(const UWorld* World, EProcessorExecutionFlags ExecutionFlagsOverride = EProcessorExecutionFlags::None);

/** based on the given World (which can be null), the function determines additional level tick types for the processing phases */
uint8 DetermineProcessorSupportedTickTypes(const UWorld* World);

/** 
 * Fills OutEntityCollections with per-archetype FMassArchetypeEntityCollection instances. 
 * @param DuplicatesHandling used to inform the function whether to expect duplicates.
 */
MASSENTITY_API extern void CreateEntityCollections(const FMassEntityManager& EntityManager, const TConstArrayView<FMassEntityHandle> Entities
	, const FMassArchetypeEntityCollection::EDuplicatesHandling DuplicatesHandling, TArray<FMassArchetypeEntityCollection>& OutEntityCollections);

/**
* AbstractSort is a sorting function that only needs to know how many items there are, how to compare items
* at individual locations - where location is in [0, NumElements) - and how to swap two elements at given locations.
* The main use case is to sort multiple arrays while keeping them in sync. For example:
*
* TArray<float> Lead = { 3.1, 0.2, 2.6, 1.0 };
* TArray<UObject*> Payload = { A, B, C, D };
*
* AbstractSort(Lead.Num()											// NumElements
* 	, [&Lead](const int32 LHS, const int32 RHS)					// Predicate
*		{
*			return Lead[LHS] < Lead[RHS];
*		}
* 	, [&Lead, &Payload](const int32 A, const int32 B)			// SwapFunctor
*	 	{
*			Swap(Lead[A], Lead[B]);
* 			Swap(Payload[A], Payload[B]);
*		}
* );
*/
template<typename TPred, typename TSwap>
inline void AbstractSort(const int32 NumElements, TPred&& Predicate, TSwap&& SwapFunctor)
{
	if (NumElements == 0)
	{
		return;
	}

	TArray<int32> Indices;
	Indices.AddUninitialized(NumElements);
	int i = 0;
	do
	{
		Indices[i] = i;
	} while (++i < NumElements);

	Indices.Sort(Predicate);

	for (i = 0; i < NumElements; ++i)
	{
		int32 SwapFromIndex = Indices[i];
		while (SwapFromIndex < i)
		{
			SwapFromIndex = Indices[SwapFromIndex];
		}

		if (SwapFromIndex != i)
		{
			SwapFunctor(i, SwapFromIndex);
		}
	}
}

MASSENTITY_API extern FMassEntityManager* GetEntityManager(const UObject* WorldContextObject);
MASSENTITY_API extern FMassEntityManager* GetEntityManager(const UWorld* World);
MASSENTITY_API extern FMassEntityManager& GetEntityManagerChecked(const UWorld& World);

} // namespace UE::Mass::Utils

