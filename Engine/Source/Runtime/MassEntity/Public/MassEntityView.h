// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityHandle.h"
#include "MassArchetypeTypes.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "StructUtils/InstancedStruct.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityView.generated.h"

#define UE_API MASSENTITY_API

struct FMassEntityManager;
struct FMassArchetypeData;
struct FMassArchetypeHandle;

/** 
 * The type representing a single entity in a single archetype. It's of a very transient nature so we guarantee it's 
 * validity only within the scope it has been created in. Don't store it. 
 */
USTRUCT()
struct FMassEntityView
{
	GENERATED_BODY()

	FMassEntityView() = default;

	/** 
	 *  Resolves Entity against ArchetypeHandle. Note that this approach requires the caller to ensure that Entity
	 *  indeed belongs to ArchetypeHandle. If not the call will fail a check. As a remedy calling the 
	 *  FMassEntityManager-flavored constructor is recommended since it will first find the appropriate archetype for
	 *  Entity. 
	 */
	UE_API FMassEntityView(const FMassArchetypeHandle& ArchetypeHandle, FMassEntityHandle Entity);

	/** 
	 *  Finds the archetype Entity belongs to and then resolves against it. The caller is responsible for ensuring
	 *  that the given Entity is in fact a valid ID tied to any of the archetypes 
	 */
	UE_API FMassEntityView(const FMassEntityManager& EntityManager, FMassEntityHandle Entity);

	/** 
	 * If the given handle represents a valid entity the function will create a FMassEntityView just like a constructor 
	 * would. If the entity is not valid the produced view will be "unset".
	 */
	static UE_API FMassEntityView TryMakeView(const FMassEntityManager& EntityManager, FMassEntityHandle Entity);

	FMassEntityHandle GetEntity() const
	{
		return Entity;
	}

	/** will fail a check if the viewed entity doesn't have the given fragment */
	template<typename T>
	T& GetFragmentData() const
	{
		static_assert(!UE::Mass::CTag<T>,
			"Given struct doesn't represent a valid fragment type but a tag. Use HasTag instead.");
		MASS_STATIC_CHECK_FRAGMENT(T);

		return *static_cast<T*>(GetFragmentPtrChecked(*T::StaticStruct()));
	}

	/** if the viewed entity doesn't have the given fragment the function will return null */
	template<typename T>
	T* GetFragmentDataPtr() const
	{
		static_assert(!UE::Mass::CTag<T>,
			"Given struct doesn't represent a valid fragment type but a tag. Use HasTag instead.");
		MASS_STATIC_CHECK_FRAGMENT(T);

		return static_cast<T*>(GetFragmentPtr(*T::StaticStruct()));
	}

	FStructView GetFragmentDataStruct(const UScriptStruct* FragmentType) const
	{
		check(FragmentType);
		return FStructView(FragmentType, static_cast<uint8*>(GetFragmentPtr(*FragmentType)));
	}

	template<UE::Mass::CSparse T>
	requires UE::Mass::CFragment<T>
	const T* GetSparseFragmentDataPtr() const
	{
		return GetSparseFragmentPtr(T::StaticStruct()).template GetPtr<T>();
	}

	template<UE::Mass::CSparse T>
	requires UE::Mass::CFragment<T>
	T* GetSparseFragmentDataPtr()
	{
		return GetSparseFragmentPtr(T::StaticStruct()).template GetPtr<T>();
	}

	/** if the viewed entity doesn't have the given const shared fragment the function will return null */
	template<typename T>
	const T* GetConstSharedFragmentDataPtr() const
	{
		static_assert(UE::Mass::CConstSharedFragment<T>,
			"Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.");

		return static_cast<const T*>(GetConstSharedFragmentPtr(*T::StaticStruct()));
	}

	/** will fail a check if the viewed entity doesn't have the given const shared fragment */
	template<typename T>
	const T& GetConstSharedFragmentData() const
	{
		static_assert(UE::Mass::CConstSharedFragment<T>,
			"Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.");

		return *static_cast<const T*>(GetConstSharedFragmentPtrChecked(*T::StaticStruct()));
	}

	FConstStructView GetConstSharedFragmentDataStruct(const UScriptStruct* FragmentType) const
	{
		check(UE::Mass::IsA<FMassConstSharedFragment>(FragmentType));
		return FConstStructView(FragmentType, static_cast<const uint8*>(GetConstSharedFragmentPtr(*FragmentType)));
	}

	/** will fail a check if the viewed entity doesn't have the given shared fragment */
	template<UE::Mass::CSharedFragment T>
	T& GetSharedFragmentData() const
	{
		return *static_cast<T*>(GetSharedFragmentPtrChecked(*T::StaticStruct()));
	}

	/** if the viewed entity doesn't have the given shared fragment the function will return null */
	template<UE::Mass::CSharedFragment T>
	T* GetSharedFragmentDataPtr() const
	{
		return static_cast<T*>(GetSharedFragmentPtr(*T::StaticStruct()));
	}

	FStructView GetSharedFragmentDataStruct(const UScriptStruct* FragmentType) const
	{
		check(UE::Mass::IsA<FMassSharedFragment>(FragmentType));
		return FStructView(FragmentType, static_cast<uint8*>(GetSharedFragmentPtr(*FragmentType)));
	}

	template<typename T>
	bool HasTag() const
	{
		return HasTag(*T::StaticStruct());
	}

	UE_API bool HasTag(TNotNull<const UScriptStruct*> TagType) const;
	bool HasTag(const UScriptStruct& TagType) const
	{
		return HasTag(&TagType);
	}

	UE_API bool HasFragment(TNotNull<const UScriptStruct*> FragmentType) const;
	template<UE::Mass::CFragment T>
	bool HasFragment() const
	{
		return HasFragment(T::StaticStruct());
	}

	UE_API bool HasElement(TNotNull<const UScriptStruct*> ElementType, const UE::Mass::EIncludeSparseElements IncludeSparseElements = UE::Mass::EIncludeSparseElements::Yes) const;
	template<UE::Mass::CSparse T>
	bool HasElement(const UE::Mass::EIncludeSparseElements IncludeSparseElements = UE::Mass::EIncludeSparseElements::Yes) const
	{
		return HasElement(T::StaticStruct(), IncludeSparseElements);
	}

	bool IsSet() const;
	bool IsValid() const;
	bool operator==(const FMassEntityView& Other) const;

protected:
	UE_API void* GetFragmentPtr(const UScriptStruct& FragmentType) const;
	UE_API void* GetFragmentPtrChecked(const UScriptStruct& FragmentType) const;
	UE_API const void* GetConstSharedFragmentPtr(const UScriptStruct& FragmentType) const;
	UE_API const void* GetConstSharedFragmentPtrChecked(const UScriptStruct& FragmentType) const;
	UE_API void* GetSharedFragmentPtr(const UScriptStruct& FragmentType) const;
	UE_API void* GetSharedFragmentPtrChecked(const UScriptStruct& FragmentType) const;
	UE_API FStructView GetSparseFragmentPtr(TNotNull<const UScriptStruct*> FragmentType);
	UE_API FConstStructView GetSparseFragmentPtr(TNotNull<const UScriptStruct*> FragmentType) const;

private:
	FMassEntityHandle Entity;
	FMassEntityInChunkDataHandle EntityDataHandle;
	FMassArchetypeData* Archetype = nullptr;
};

//-----------------------------------------------------------------------------
// INLINES
//-----------------------------------------------------------------------------
inline bool FMassEntityView::IsSet() const
{
	return EntityDataHandle.IsValid(Archetype);
}

inline bool FMassEntityView::IsValid() const
{
	return IsSet();
}

inline bool FMassEntityView::operator==(const FMassEntityView& Other) const
{
	return Archetype == Other.Archetype && EntityDataHandle == Other.EntityDataHandle;
}

#undef UE_API
