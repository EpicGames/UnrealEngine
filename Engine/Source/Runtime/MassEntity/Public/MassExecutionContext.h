// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Mass/EntityHandle.h"
#include "MassEntityTypes.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Mass/ExternalSubsystemTraits.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityQuery.h"
#include "MassArchetypeTypes.h"
#include "MassSubsystemAccess.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "MassProcessor.h"
#endif
#include "MassEntityCollection.h"
#include "MassEntityView.h"

#if WITH_MASSENTITY_DEBUG
#include "MassProcessor.h"  // full definition needed for TWeakObjectPtr<UMassProcessor> debug member
#endif

#define MASS_CHECK_IF_VALID(View, Type) \
	checkf(View \
		, TEXT("Requested fragment type not bound, type %s. Make sure it has been listed as required."), *GetNameSafe(Type))

#define MASS_CHECK_IF_READWRITE(View) \
	checkf(View == nullptr || View->Requirement.AccessMode == EMassFragmentAccess::ReadWrite \
		, TEXT("Requested fragment type not bound for writing, type %s. Make sure it has been listed as required in ReadWrite mode.") \
		, View ? *GetNameSafe(View->Requirement.StructType) : TEXT("[Not found]"))

#define MASS_CHECK_IF_SPARSE_REQUIREMENT(ElementType) \
	checkf(RequestedSparseElementsAccess.Contains(ElementType) \
			, TEXT("%hs: %s not requested as requirement by the query"), __FUNCTION__, *ElementType->GetName())

class UWorld;
class UMassProcessor;
struct FMassEntityQuery;
struct FMassEntityLinkFragment;
struct FMassArchetypeChunk;
struct FMassArchetypeData;

struct FMassExecutionContext
{
	struct FSparseEntityIterator;
private:
	friend FMassArchetypeData;

	template< typename ViewType >
	struct TFragmentView 
	{
		FMassFragmentRequirementDescription Requirement;
		ViewType FragmentView;

		TFragmentView()
		{
		}
		explicit TFragmentView(const FMassFragmentRequirementDescription& InRequirement)
			: Requirement(InRequirement)
		{
		}

		bool operator==(const UScriptStruct* FragmentType) const
		{
			return Requirement.StructType == FragmentType;
		}
	};
	using FFragmentView = TFragmentView<TArrayView<FMassFragment>>;
	TArray<FFragmentView, TInlineAllocator<8>> FragmentViews;

	using FChunkFragmentView = TFragmentView<FStructView>;
	TArray<FChunkFragmentView, TInlineAllocator<4>> ChunkFragmentViews;

	using FConstSharedFragmentView = TFragmentView<FConstStructView>;
	TArray<FConstSharedFragmentView, TInlineAllocator<4>> ConstSharedFragmentViews;

	using FSharedFragmentView = TFragmentView<FStructView>;
	TArray<FSharedFragmentView, TInlineAllocator<4>> SharedFragmentViews;

	FMassElementBitSet RequestedSparseElementsAccess;
	// @todo could add a construct similar to TFragmentView that would cache direct access to FSparseElementStorage::FTypePool
	// for requested sparse element types. That would be a perf optimization

	/** linked entity to allow access to fragments on an entity not in the query, must be bound before access */
	FMassEntityView LinkedEntityView;

	/** Used for indirect fragment access */
	FMassEntityView CachedIndirectEntityView;

	TUniquePtr<FMassSubsystemAccess> SubsystemAccessPtr;

	// mz@todo make this shared ptr thread-safe and never auto-flush in MT environment.
	TSharedPtr<FMassCommandBuffer> DeferredCommandBuffer;
	TArrayView<FMassEntityHandle> EntityListView;
	
	/** If set this indicates the exact archetype and its chunks to be processed. 
	 *  @todo this data should live somewhere else, preferably be just a parameter to Query.ForEachEntityChunk function */
	FMassArchetypeEntityCollection EntityCollection;
	
	/** @todo rename to "payload" */
	FInstancedStruct AuxData;
	float DeltaTimeSeconds = 0.0f;
	FMassElementBitSet CurrentArchetypeCompositionBitSet;
#if WITH_MASSENTITY_DEBUG
	FColor DebugColor;
#endif // WITH_MASSENTITY_DEBUG

	TSharedRef<FMassEntityManager> EntityManager;

	struct FQueryTransientRuntime
	{
		TNotNull<FMassEntityQuery*> Query;
		FMassExternalSubsystemBitSet ConstSubsystemsBitSet;
		FMassExternalSubsystemBitSet MutableSubsystemsBitSet;

		/** used to restore creation requirements on query pop, storing the previous value to easily restore processor creation requirements */
		const FMassEntityCreationRequirements* PreviousEntityCreationRequirements = nullptr;

#if WITH_MASSENTITY_DEBUG
		/** MaxBreakFragmentCount needs to be bigger than the greatest number of fragments a query has a write requirement for to that can have a breakpoint set */
		static constexpr uint32 MaxFragmentBreakpointCount = 8;
		TStaticArray<const UScriptStruct*, MaxFragmentBreakpointCount> FragmentTypesToBreakOn;

		bool bCheckProcessorBreaks = false;
		int32 BreakFragmentsCount = 0;
#endif // WITH_MASSENTITY_DEBUG

		/** Serial number to ensure iterator consistency (subsequent calls to CreateEntityIterator should not pass equivalency test) */
		uint32 IteratorSerialNumber = 0;

		/** Helper function to create an empty instance with a valid Query ptr */
		static FQueryTransientRuntime& GetDummyInstance();
	};

	/** We usually expect the queries to go only a single layer deep, so 2 elements here should suffice most of the time */
	TArray<FQueryTransientRuntime, TInlineAllocator<2>> QueriesStack;

	/** Track the serial number for FEntityIterator creation */
	uint32 IteratorSerialNumberGenerator = 0;

	EMassExecutionContextType ExecutionType = EMassExecutionContextType::Local;

	/** Used to control when the context is allowed to flush commands collected in DeferredCommandBuffer. This mechanism 
	 * is mainly utilized to avoid numerous small flushes in favor of fewer larger ones. */
	bool bFlushDeferredCommands = true;

	/** cached value of current query's HasSparseRequirements() */
	bool bHasSparseRequirements = false;

	/** Extremely transient, needs to be kept private. Used for direct access to sparse elements API  */
	FMassArchetypeData* ArchetypeDataPtr = nullptr;
	FMassEntityInChunkDataHandle ChunkEntityIndex;

#if WITH_MASSENTITY_DEBUG
	FString DebugExecutionDescription;

	/** Currently executing processor, used for debugger breakpoint checking */
	// js@todo make this more generic
	TWeakObjectPtr<UMassProcessor> DebugProcessor;
#endif // WITH_MASSENTITY_DEBUG

	/** Active permissions for archetype creation (processor-level when no query is active; query-level otherwise) */
	const FMassEntityCreationRequirements* ActiveEntityCreationRequirements = nullptr;

	TArrayView<FFragmentView> GetMutableRequirements()
	{
		return FragmentViews;
	}
	TArrayView<FChunkFragmentView> GetMutableChunkRequirements()
	{
		return ChunkFragmentViews;
	}
	TArrayView<FConstSharedFragmentView> GetMutableConstSharedRequirements()
	{
		return ConstSharedFragmentViews;
	}
	TArrayView<FSharedFragmentView> GetMutableSharedRequirements()
	{
		return SharedFragmentViews;
	}

	void GetSubsystemRequirementBits(FMassExternalSubsystemBitSet& OutConstSubsystemsBitSet, FMassExternalSubsystemBitSet& OutMutableSubsystemsBitSet)
	{
		SubsystemAccessPtr->GetSubsystemRequirementBits(OutConstSubsystemsBitSet, OutMutableSubsystemsBitSet);
	}

	void SetSubsystemRequirementBits(const FMassExternalSubsystemBitSet& InConstSubsystemsBitSet, const FMassExternalSubsystemBitSet& InMutableSubsystemsBitSet)
	{
		SubsystemAccessPtr->SetSubsystemRequirementBits(InConstSubsystemsBitSet, InMutableSubsystemsBitSet);
	}

	MASSENTITY_API FStructView GetSparseElementInternal(TNotNull<const UScriptStruct*> ElementType, const FMassEntityHandle EntityHandle) const;

	/**
	 * Explicit assignment used by copy ctor and internal helpers. Intentionally does NOT copy
	 * QueriesStack (callers manage it explicitly) or processing callbacks (set by FProcessorWorker
	 * on the copy after construction).
	 */
	MASSENTITY_API FMassExecutionContext& operator=(const FMassExecutionContext& Other);

public:
	MASSENTITY_API explicit FMassExecutionContext(FMassEntityManager& InEntityManager, const float InDeltaTimeSeconds = 0.f, const bool bInFlushDeferredCommands = true);
	MASSENTITY_API FMassExecutionContext(FMassExecutionContext&& Other);
	MASSENTITY_API FMassExecutionContext(const FMassExecutionContext& Other);
	MASSENTITY_API FMassExecutionContext(const FMassExecutionContext& Other, FMassEntityQuery& Query, const TSharedPtr<FMassCommandBuffer>& InCommandBuffer = {});
	MASSENTITY_API ~FMassExecutionContext();

	/** For internal use only, should never be exported as part of API */
	MASSENTITY_API static FMassExecutionContext& GetDummyInstance();

	FMassEntityManager& GetEntityManagerChecked() const;
	const TSharedRef<FMassEntityManager>& GetSharedEntityManager();

#if WITH_MASSENTITY_DEBUG
	const FString& DebugGetExecutionDesc() const
	{
		return DebugExecutionDescription;
	}
	void DebugSetExecutionDesc(const FString& Description)
	{
		DebugExecutionDescription = Description;
	}
	void DebugSetExecutionDesc(FString&& Description)
	{
		DebugExecutionDescription = MoveTemp(Description);
	}

	UMassProcessor* DebugGetProcessor() const
	{
		return DebugProcessor.Get();
	}
	void DebugSetProcessor(UMassProcessor* Processor)
	{
		DebugProcessor = Processor;
	}
#endif

	void PushQuery(FMassEntityQuery& InQuery);
	void PopQuery(const FMassEntityQuery& InQuery);
	const FMassEntityQuery& GetCurrentQuery() const;
	bool IsCurrentQuery(const FMassEntityQuery& Query) const;
	void ApplyFragmentRequirements(const FMassEntityQuery& RequestingQuery);
	void ClearFragmentViews(const FMassEntityQuery& RequestingQuery);

	/**
	 * @return whether the entity indicated via FSparseEntityIterator meets the sparse element requirements.
	 *		Will always return true if there are no sparse requirements
	 */
	bool ExecuteSparseElementsFilter(const FSparseEntityIterator& EntityIt) const;

	/**
	 * Iterator to easily loop through entities in the current chunk.
	 * Supports ranged for and can be used directly as an entity index for the current chunk.
	 */
	struct FEntityIterator
	{
		inline int32 operator*() const
		{
			return EntityIndex;
		}

		inline bool operator!=(const int& Other) const
		{
			return EntityIndex != Other;
		}

		inline operator int32() const
		{
			return EntityIndex;
		}

		inline operator bool() const
		{
			return SerialNumber && EntityIndex < NumEntities;
		}

		inline bool operator<(const int32 Other) const
		{
			return SerialNumber && EntityIndex != INDEX_NONE && EntityIndex < Other;
		}

		inline FEntityIterator& operator++()
		{
			++EntityIndex;
#if WITH_MASSENTITY_DEBUG
			if (UNLIKELY(QueryRuntime.bCheckProcessorBreaks || QueryRuntime.BreakFragmentsCount != 0) 
				&& EntityIndex < NumEntities)
			{
				TestBreakpoints();
			}
#endif //WITH_MASSENTITY_DEBUG
			return *this;
		}

		inline void operator++(int)
		{
			++(*this);
		}

		FEntityIterator&& begin()
		{
			return MoveTemp(*this);
		}

		FEntityIterator end() const
		{
			FEntityIterator End;
			End.EntityIndex = NumEntities;
			return End;
		}

		MASSENTITY_API FEntityIterator();
		FEntityIterator(FEntityIterator&&) = default;

		FEntityIterator& operator=(const FEntityIterator&) = delete;
		FEntityIterator& operator=(FEntityIterator&&) = delete;
		/**
		 * Iterator copying is disabled to avoid additional checks to detect if entity chunk being iterated on changed.
		 * This decision is to be reconsidered when valid iterator-copying scenarios emerge. 
		 */
		FEntityIterator(const FEntityIterator&) = delete;

		FMassEntityHandle GetEntityHandle() const
		{
			return bool(*this) ? ExecutionContext.GetEntities()[EntityIndex] : FMassEntityHandle();
		}

	protected:
		friend FMassExecutionContext;
		FEntityIterator(FMassExecutionContext& InExecutionContext);
		FEntityIterator(FMassExecutionContext& InExecutionContext, FQueryTransientRuntime& InQueryRuntime);

#if WITH_MASSENTITY_DEBUG
		MASSENTITY_API void TestBreakpoints();
#endif //!WITH_MASSENTITY_DEBUG

		const FMassExecutionContext& ExecutionContext;
		const FQueryTransientRuntime& QueryRuntime;
		int32 EntityIndex = INDEX_NONE;
		const int32 NumEntities = INDEX_NONE;
		const uint32 SerialNumber = 0;
	};

	/**
	 * Creates an Entity Iterator for the current chunk.
	 * Supports range-based for loop and can be used directly as an entity index for the current chunk.
	 */
	MASSENTITY_API FEntityIterator CreateEntityIterator();

	/**
	 * A flavor of FEntityIterator that skips all the entities that don't match current query's sparse element requirements.
	 * This means that if there are no sparse requirements then FSparseEntityIterator will behave like a regular FEntityIterator
	 * but less efficiently so.
	 */
	struct FSparseEntityIterator : FEntityIterator
	{
		/** Note that the default constructor results in an "invalid" (i.e. useless) iterator. */
		FSparseEntityIterator() = default;

		explicit FSparseEntityIterator(FEntityIterator&& Source)
			: FEntityIterator(MoveTemp(Source))
		{
			if (EntityIndex < NumEntities && ExecutionContext.ExecuteSparseElementsFilter(*this) == false)
			{
				++(*this);
			}
		}

		FSparseEntityIterator& operator++()
		{
			do
			{
				++EntityIndex;
			} while (EntityIndex < NumEntities && ExecutionContext.ExecuteSparseElementsFilter(*this) == false);

#if WITH_MASSENTITY_DEBUG
			if (UNLIKELY(QueryRuntime.bCheckProcessorBreaks || QueryRuntime.BreakFragmentsCount != 0) 
				&& EntityIndex < NumEntities)
			{
				TestBreakpoints();
			}
#endif //WITH_MASSENTITY_DEBUG
			return *this;
		}

		void operator++(int)
		{
			++(*this);
		}

		FSparseEntityIterator&& begin()
		{
			return MoveTemp(*this);
		}

		FSparseEntityIterator end() const
		{
			FSparseEntityIterator End;
			End.EntityIndex = NumEntities;
			return End;
		}
	};

	/**
	 * Creates an entity iterator for the current chunk that's configured to skip entities that are not matching
	 * the current query's sparse element requirements.
	 */
	MASSENTITY_API FSparseEntityIterator CreateSparseEntityIterator();
	// @todo we need AnySparseElementsBitSet and NoneSparseElementsBitSet

	/** Sets bFlushDeferredCommands. Note that setting to True while the system is being executed doesn't result in
	 *  immediate commands flushing */
	void SetFlushDeferredCommands(const bool bNewFlushDeferredCommands);
	void SetDeferredCommandBuffer(const TSharedPtr<FMassCommandBuffer>& InDeferredCommandBuffer);
	MASSENTITY_API void SetEntityCollection(const FMassArchetypeEntityCollection& InEntityCollection);
	MASSENTITY_API void SetEntityCollection(FMassArchetypeEntityCollection&& InEntityCollection);
	void ClearEntityCollection();
	void SetAuxData(const FInstancedStruct& InAuxData);

	void SetExecutionType(EMassExecutionContextType InExecutionType);
	EMassExecutionContextType GetExecutionType() const;

	/** Binds a linked entity with a specific link type, use this if you already have the link fragment you wish to bind */
	MASSENTITY_API void BindLinkedEntity(const FMassEntityLinkFragment& EntityLinkFragment);
	/** Binds a linked entity with the link type specified in the current query */
	MASSENTITY_API void BindLinkedEntity();

	float GetDeltaTimeSeconds() const;

	MASSENTITY_API UWorld* GetWorld() const;

	TSharedPtr<FMassCommandBuffer> GetSharedDeferredCommandBuffer() const
	{
		return DeferredCommandBuffer;
	}
	FMassCommandBuffer& Defer() const
	{
		checkSlow(DeferredCommandBuffer.IsValid());
		return *DeferredCommandBuffer.Get();
	}

	TConstArrayView<FMassEntityHandle> GetEntities() const
	{
		return EntityListView;
	}
	int32 GetNumEntities() const
	{
		return EntityListView.Num();
	}

	FMassEntityHandle GetEntity(const int32 Index) const
	{
		return EntityListView[Index];
	}

	void ForEachEntityInChunk(const FMassEntityExecuteFunction& EntityExecuteFunction)
	{
		for (FEntityIterator EntityIterator = CreateEntityIterator(); EntityIterator; ++EntityIterator)
		{
			EntityExecuteFunction(*this, EntityIterator);
		}
	}

	bool DoesArchetypeHaveElement(TNotNull<const UScriptStruct*> ElementType) const
	{
		return CurrentArchetypeCompositionBitSet.Contains(ElementType);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	bool DoesArchetypeHaveElement(const UScriptStruct& ElementType) const
	{
		return DoesArchetypeHaveElement(&ElementType);
	}

	template<UE::Mass::CElement T>
	bool DoesArchetypeHaveElement() const
	{
		return CurrentArchetypeCompositionBitSet.Contains<T>();
	}

	bool DoesArchetypeHaveFragment(TNotNull<const UScriptStruct*> FragmentType) const
	{
		return CurrentArchetypeCompositionBitSet.Contains(FragmentType);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	bool DoesArchetypeHaveFragment(const UScriptStruct& ElementType) const
	{
		return DoesArchetypeHaveFragment(&ElementType);
	}

	template<UE::Mass::CFragment T>
	bool DoesArchetypeHaveFragment() const
	{
		return CurrentArchetypeCompositionBitSet.Contains<T>();
	}

	bool DoesArchetypeHaveTag(TNotNull<const UScriptStruct*> TagType) const
	{
		return CurrentArchetypeCompositionBitSet.Contains(TagType);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	bool DoesArchetypeHaveTag(const UScriptStruct& ElementType) const
	{
		return DoesArchetypeHaveTag(&ElementType);
	}

	template<UE::Mass::CTag T>
	bool DoesArchetypeHaveTag() const
	{
		return CurrentArchetypeCompositionBitSet.Contains<T>();
	}

#if WITH_MASSENTITY_DEBUG
	FColor DebugGetArchetypeColor() const
	{
		return DebugColor;
	}
#endif // WITH_MASSENTITY_DEBUG

	//-----------------------------------------------------------------------------
	// Chunk related operations
	//-----------------------------------------------------------------------------

	/** @return whether the given chunk is fine to be processed, i.e. `false` indicates the chunk should be skipped */
	bool SetCurrentChunk(int32 ChunkIndex, const FMassArchetypeChunk& Chunk);

	int32 GetChunkSerialModificationNumber() const;

	template<typename T>
	T* GetMutableChunkFragmentPtr()
	{
		static_assert(UE::Mass::CChunkFragment<T>, "Given struct doesn't represent a valid chunk fragment type. Make sure to inherit from FMassChunkFragment or one of its child-types.");

		const UScriptStruct* Type = T::StaticStruct();
		FChunkFragmentView* FoundChunkFragmentData = ChunkFragmentViews.FindByPredicate([Type](const FChunkFragmentView& Element) { return Element.Requirement.StructType == Type; } );
		MASS_CHECK_IF_READWRITE(FoundChunkFragmentData);
		return FoundChunkFragmentData ? FoundChunkFragmentData->FragmentView.GetPtr<T>() : static_cast<T*>(nullptr);
	}
	
	template<typename T>
	T& GetMutableChunkFragment()
	{
		T* ChunkFragment = GetMutableChunkFragmentPtr<T>();
		MASS_CHECK_IF_VALID(ChunkFragment, T::StaticStruct());
		return *ChunkFragment;
	}

	template<typename T>
	const T* GetChunkFragmentPtr() const
	{
		static_assert(UE::Mass::CChunkFragment<T>, "Given struct doesn't represent a valid chunk fragment type. Make sure to inherit from FMassChunkFragment or one of its child-types.");

		const UScriptStruct* Type = T::StaticStruct();
		const FChunkFragmentView* FoundChunkFragmentData = ChunkFragmentViews.FindByPredicate([Type](const FChunkFragmentView& Element) { return Element.Requirement.StructType == Type; } );
		return FoundChunkFragmentData ? FoundChunkFragmentData->FragmentView.GetPtr<T>() : static_cast<const T*>(nullptr);
	}
	
	template<typename T>
	const T& GetChunkFragment() const
	{
		const T* ChunkFragment = GetChunkFragmentPtr<T>();
		MASS_CHECK_IF_VALID(ChunkFragment, T::StaticStruct());
		return *ChunkFragment;
	}

	/** Shared fragment related operations */
	const void* GetConstSharedFragmentPtr(const UScriptStruct& SharedFragmentType) const
	{
		const FConstSharedFragmentView* FoundSharedFragmentData = ConstSharedFragmentViews.FindByPredicate([&SharedFragmentType](const FConstSharedFragmentView& Element) { return Element.Requirement.StructType == &SharedFragmentType; });
		return FoundSharedFragmentData ? FoundSharedFragmentData->FragmentView.GetMemory() : nullptr;
	}
	
	template<typename T>
	const T* GetConstSharedFragmentPtr() const
	{
		static_assert(UE::Mass::CConstSharedFragment<T>, "Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.");

		const FConstSharedFragmentView* FoundSharedFragmentData = ConstSharedFragmentViews.FindByPredicate([](const FConstSharedFragmentView& Element) { return Element.Requirement.StructType == T::StaticStruct(); });
		return FoundSharedFragmentData ? FoundSharedFragmentData->FragmentView.GetPtr<const T>() : static_cast<const T*>(nullptr);
	}

	template<typename T>
	const T& GetConstSharedFragment() const
	{
		const T* SharedFragment = GetConstSharedFragmentPtr<const T>();
		MASS_CHECK_IF_VALID(SharedFragment, T::StaticStruct());
		return *SharedFragment;
	}

	template<typename T>
	T* GetMutableSharedFragmentPtr()
	{
		static_assert(UE::Mass::CSharedFragment<T>, "Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types.");

		FSharedFragmentView* FoundSharedFragmentData = SharedFragmentViews.FindByPredicate([](const FSharedFragmentView& Element) { return Element.Requirement.StructType == T::StaticStruct(); });
		MASS_CHECK_IF_READWRITE(FoundSharedFragmentData);
		return FoundSharedFragmentData ? FoundSharedFragmentData->FragmentView.GetPtr<T>() : static_cast<T*>(nullptr);
	}

	template<typename T>
	const T* GetSharedFragmentPtr() const
	{
		static_assert(UE::Mass::CSharedFragment<T>, "Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types.");

		const FSharedFragmentView* FoundSharedFragmentData = SharedFragmentViews.FindByPredicate([](const FSharedFragmentView& Element) { return Element.Requirement.StructType == T::StaticStruct(); });
		return FoundSharedFragmentData ? FoundSharedFragmentData->FragmentView.GetPtr<T>() : static_cast<const T*>(nullptr);
	}

	template<typename T>
	T& GetMutableSharedFragment()
	{
		T* SharedFragment = GetMutableSharedFragmentPtr<T>();
		MASS_CHECK_IF_VALID(SharedFragment, T::StaticStruct());
		return *SharedFragment;
	}

	template<typename T>
	const T& GetSharedFragment() const
	{
		const T* SharedFragment = GetSharedFragmentPtr<T>();
		MASS_CHECK_IF_VALID(SharedFragment, T::StaticStruct());
		return *SharedFragment;
	}

	/* Fragments related operations */
	template<typename TFragment>
	TArrayView<TFragment> GetMutableFragmentView()
	{
		const UScriptStruct* FragmentType = TFragment::StaticStruct();
		const FFragmentView* View = FragmentViews.FindByPredicate([FragmentType](const FFragmentView& Element) { return Element.Requirement.StructType == FragmentType; });
		MASS_CHECK_IF_VALID(View, FragmentType);
		MASS_CHECK_IF_READWRITE(View);
		return MakeArrayView<TFragment>(static_cast<TFragment*>(View->FragmentView.GetData()), View->FragmentView.Num());
	}

	template<typename TFragment>
	TConstArrayView<TFragment> GetFragmentView() const
	{
		const UScriptStruct* FragmentType = TFragment::StaticStruct();
		const FFragmentView* View = FragmentViews.FindByPredicate([FragmentType](const FFragmentView& Element) { return Element.Requirement.StructType == FragmentType; });
		MASS_CHECK_IF_VALID(View, TFragment::StaticStruct());
		return TConstArrayView<TFragment>(static_cast<const TFragment*>(View->FragmentView.GetData()), View->FragmentView.Num());
	}

	TConstArrayView<FMassFragment> GetFragmentView(const UScriptStruct* FragmentType) const
	{
		const FFragmentView* View = FragmentViews.FindByPredicate([FragmentType](const FFragmentView& Element) { return Element.Requirement.StructType == FragmentType; });
		MASS_CHECK_IF_VALID(View, FragmentType);
		return TConstArrayView<FMassFragment>(static_cast<const FMassFragment*>(View->FragmentView.GetData()), View->FragmentView.Num());
	}

	TArrayView<FMassFragment> GetMutableFragmentView(const UScriptStruct* FragmentType) 
	{
		const FFragmentView* View = FragmentViews.FindByPredicate([FragmentType](const FFragmentView& Element) { return Element.Requirement.StructType == FragmentType; });
		MASS_CHECK_IF_VALID(View, FragmentType);
		MASS_CHECK_IF_READWRITE(View);
		return View->FragmentView;
	}

	template<typename TFragmentBase>
	TConstArrayView<TFragmentBase> GetFragmentView(TNotNull<const UScriptStruct*> FragmentType) const
	{
		check(FragmentType->IsChildOf(TFragmentBase::StaticStruct()));
		TConstArrayView<FMassFragment> View = GetFragmentView(FragmentType);
		return TConstArrayView<TFragmentBase>(reinterpret_cast<const TFragmentBase*>(View.GetData()), View.Num());
	}

	template<typename TFragmentBase>
	TArrayView<TFragmentBase> GetMutableFragmentView(TNotNull<const UScriptStruct*> FragmentType)
	{
		check(FragmentType->IsChildOf(TFragmentBase::StaticStruct()));
		TArrayView<FMassFragment> View = GetMutableFragmentView(FragmentType);
		return TArrayView<TFragmentBase>(reinterpret_cast<TFragmentBase*>(View.GetData()), View.Num());
	}

	template<typename T>
	T* GetLinkedEntityFragmentPtr()
	{
		checkf(QueriesStack.Num(), TEXT("There must be an active query to access fragments on the linked entity"));
		checkf(LinkedEntityView.IsValid(), TEXT("No entity has been linked. Add a FMassEntityLinkFragment to entities in this query to access fragments on a linked entity that isn't in this query."));
		checkfSlow(QueriesStack.Last().Query->HasMutableIndirectEntityRequirement<T>()
			, TEXT("Attempting to access a fragment from a linked entity without first adding the LinkedEntityFragment requirement to the query"));

		return LinkedEntityView.GetFragmentDataPtr<T>();
	}

	template<typename T>
	const T* GetLinkedEntityConstFragmentPtr()
	{
		checkf(QueriesStack.Num(), TEXT("There must be an active query to access fragments on the linked entity"));
		checkf(LinkedEntityView.IsValid(), TEXT("No entity has been linked. Add a FMassEntityLinkFragment to entities in this query to access fragments on a linked entity that isn't in this query."));
		checkfSlow(QueriesStack.Last().Query->HasConstIndirectEntityRequirement<T>()
			, TEXT("Attempting to access a fragment from a linked entity without first adding the LinkedEntityFragment requirement to the query"));

		return LinkedEntityView.GetFragmentDataPtr<T>();
	}

	template<typename T>
	T* GetIndirectFragmentPtr(const FMassEntityHandle& FragmentOwner)
	{
		checkf(QueriesStack.Num(), TEXT("There must be an active query to access indirect fragments, otherwise just use direct access with a query"));
		checkfSlow(QueriesStack.Last().Query->HasMutableIndirectEntityRequirement<T>()
			, TEXT("Attempting indirect fragment access without first adding and IndirectFragment requirement to the query"));

		if (CachedIndirectEntityView.GetEntity() != FragmentOwner)
		{
			CachedIndirectEntityView = FMassEntityView(EntityManager.Get(), FragmentOwner);
		}

		return CachedIndirectEntityView.GetFragmentDataPtr<T>();
	}

	template<typename T>
	const T* GetIndirectConstFragmentPtr(const FMassEntityHandle& FragmentOwner)
	{
		checkf(QueriesStack.Num(), TEXT("There must be an active query to access indirect fragments, otherwise just use direct access with a query"));
		checkfSlow(QueriesStack.Last().Query->HasConstIndirectEntityRequirement<T>()
			, TEXT("Attempting indirect fragment access without first adding and IndirectFragment requirement to the query"));

		if (CachedIndirectEntityView.GetEntity() != FragmentOwner)
		{
			CachedIndirectEntityView = FMassEntityView(EntityManager.Get(), FragmentOwner);
		}

		return CachedIndirectEntityView.GetFragmentDataPtr<T>();
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	T* GetMutableSubsystem()
	{
		return SubsystemAccessPtr->GetMutableSubsystem<T>();
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	T& GetMutableSubsystemChecked()
	{
		return SubsystemAccessPtr->GetMutableSubsystemChecked<T>();
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	const T* GetSubsystem()
	{
		return SubsystemAccessPtr->GetSubsystem<T>();
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	const T& GetSubsystemChecked()
	{
		return SubsystemAccessPtr->GetSubsystemChecked<T>();
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	T* GetMutableSubsystem(const TSubclassOf<USubsystem> SubsystemClass)
	{
		return SubsystemAccessPtr->GetMutableSubsystem<T>(SubsystemClass);
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	T& GetMutableSubsystemChecked(const TSubclassOf<USubsystem> SubsystemClass)
	{
		return SubsystemAccessPtr->GetMutableSubsystemChecked<T>(SubsystemClass);
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	const T* GetSubsystem(const TSubclassOf<USubsystem> SubsystemClass)
	{
		return SubsystemAccessPtr->GetSubsystem<T>(SubsystemClass);
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	const T& GetSubsystemChecked(const TSubclassOf<USubsystem> SubsystemClass)
	{
		return SubsystemAccessPtr->GetSubsystemChecked<T>(SubsystemClass);
	}

	//-----------------------------------------------------------------------------
	// Sparse elements querying
	//-----------------------------------------------------------------------------
	template<UE::Mass::CSparse TFragment>
	requires UE::Mass::CFragment<TFragment>
	TFragment* GetMutableSparseElement(const FEntityIterator& EntityIterator) const
	{
		MASS_CHECK_IF_SPARSE_REQUIREMENT(TFragment::StaticStruct());

		FStructView SparseElementView = GetMutableSparseElement(TFragment::StaticStruct(), EntityIterator);
		return SparseElementView.GetPtr<TFragment>();
	}

	template<UE::Mass::CSparse TFragmentBase>
	requires UE::Mass::CFragment<TFragmentBase>
	TFragmentBase* GetMutableSparseElement(TNotNull<const UScriptStruct*> ElementType, const FEntityIterator& EntityIterator) const
	{
		checkf(ElementType->IsChildOf(TFragmentBase::StaticStruct()), TEXT("%s is not derived from %s")
			, *ElementType->GetName(), *TFragmentBase::StaticStruct()->GetName());
		MASS_CHECK_IF_SPARSE_REQUIREMENT(ElementType);

		FStructView SparseElementView = GetSparseElementInternal(ElementType, EntityIterator.GetEntityHandle());
		return SparseElementView.GetPtr<TFragmentBase>();
	}	

	FStructView GetMutableSparseElement(TNotNull<const UScriptStruct*> ElementType, const FEntityIterator& EntityIterator) const
	{
		checkf(UE::Mass::IsSparse(ElementType), TEXT("%s doesn't represent a sparse element type"), *ElementType->GetName());
		MASS_CHECK_IF_SPARSE_REQUIREMENT(ElementType);

		return UE::Mass::IsA<FMassFragment>(ElementType)
			? GetSparseElementInternal(ElementType, EntityIterator.GetEntityHandle())
			: FStructView();
	}

	template<UE::Mass::CSparse TFragment>
	requires UE::Mass::CFragment<TFragment>
	const TFragment* GetSparseElement(const FEntityIterator& EntityIterator) const
	{
		MASS_CHECK_IF_SPARSE_REQUIREMENT(TFragment::StaticStruct());
		FConstStructView SparseElementView = GetSparseElement(TFragment::StaticStruct(), EntityIterator);
		return SparseElementView.GetPtr<const TFragment>();
	}

	template<UE::Mass::CSparse TFragmentBase>
	requires UE::Mass::CFragment<TFragmentBase>
	const TFragmentBase* GetSparseElement(TNotNull<const UScriptStruct*> ElementType, const FEntityIterator& EntityIterator) const
	{
		checkf(ElementType->IsChildOf(TFragmentBase::StaticStruct()), TEXT("%s is not derived from %s")
			, *ElementType->GetName(), *TFragmentBase::StaticStruct()->GetName());
		MASS_CHECK_IF_SPARSE_REQUIREMENT(ElementType);

		FConstStructView SparseElementView = GetSparseElementInternal(ElementType, EntityIterator.GetEntityHandle());
		return SparseElementView.GetPtr<const TFragmentBase>();
	}

	FConstStructView GetSparseElement(TNotNull<const UScriptStruct*> ElementType, const FEntityIterator& EntityIterator) const
	{
		checkf(UE::Mass::IsSparse(ElementType), TEXT("%s doesn't represent a sparse element type"), *ElementType->GetName());
		MASS_CHECK_IF_SPARSE_REQUIREMENT(ElementType);

		return UE::Mass::IsA<FMassFragment>(ElementType)
			? GetSparseElementInternal(ElementType, EntityIterator.GetEntityHandle())
			: FConstStructView();
	}

	MASSENTITY_API bool HasSparseElement(TNotNull<const UScriptStruct*> ElementType, const FEntityIterator& EntityIterator) const;

	/** Sparse chunk related operation */
	const FMassArchetypeEntityCollection& GetEntityCollection() const
	{
		return EntityCollection;
	}

	const FInstancedStruct& GetAuxData() const
	{
		return AuxData;
	}
	FInstancedStruct& GetMutableAuxData()
	{
		return AuxData;
	}
	
	template<typename TFragment>
	bool ValidateAuxDataType() const
	{
		const UScriptStruct* FragmentType = GetAuxData().GetScriptStruct();
		return FragmentType != nullptr && FragmentType == TFragment::StaticStruct();
	}

	MASSENTITY_API void FlushDeferred();

	void ClearExecutionData();

	void SetCurrentArchetype(FMassArchetypeData& Archetype);

	void SetCurrentArchetypeCompositionDescriptor(const FMassArchetypeCompositionDescriptor& Descriptor)
	{
		CurrentArchetypeCompositionBitSet = Descriptor.GetElementsBitSet();
	}
	void StoreCurrentArchetypeComposition(const FMassElementBitSet& CompositionBitSet)
	{
		CurrentArchetypeCompositionBitSet = CompositionBitSet;
	}

	/**
	* Validates creation request by the active FMassEntityCreationRequirements and creates an entity
	* of the declared archetype. Will assert on validation failure. 
	*/
	MASSENTITY_API FMassEntityHandle CreateEntity(const FMassArchetypeHandle& ArchetypeHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues = {});
	MASSENTITY_API void BuildEntity(const FMassArchetypeHandle& ArchetypeHandle, const FMassEntityHandle& EntityHandle, const FMassArchetypeSharedFragmentValues& SharedFragmentValues = {});

	MASSENTITY_API void BatchCreateEntities(const FMassArchetypeHandle& ArchetypeHandle,
		const FMassArchetypeSharedFragmentValues& SharedFragmentValues, const int32 Count, TArray<FMassEntityHandle>& InOutEntities);
	void BatchCreateEntities(const FMassArchetypeHandle& ArchetypeHandle, const int32 Count, TArray<FMassEntityHandle>& InOutEntities)
	{
		BatchCreateEntities(ArchetypeHandle, FMassArchetypeSharedFragmentValues(), Count, InOutEntities);

	}

	/** Set/restore by processor and query push/pop */
	void SetActiveArchetypeRequirements(const FMassEntityCreationRequirements* InRequirements)
	{
		ActiveEntityCreationRequirements = InRequirements;
	}

#if WITH_MASSENTITY_DEBUG
	void DebugSetColor(FColor InColor)
	{
		DebugColor = InColor;
	}
#endif // WITH_MASSENTITY_DEBUG

	/** 
	 * Processes SubsystemRequirements to fetch and cache all the indicated subsystems. If a UWorld is required to fetch
	 * a specific subsystem then the one associated with the stored EntityManager will be used.
	 *
	 * @param SubsystemRequirements indicates all the subsystems that are expected to be accessed. Requesting a subsystem 
	 *	not indicated by the SubsystemRequirements will result in a failure.
	 * 
	 * @return `true` if all required subsystems have been found, `false` otherwise.
	 */
	bool CacheSubsystemRequirements(const FMassSubsystemRequirements& SubsystemRequirements);

protected:
	void SetSubsystemRequirements(const FMassSubsystemRequirements& SubsystemRequirements)
	{
		SubsystemAccessPtr->SetSubsystemRequirements(SubsystemRequirements);
	}

	void SetFragmentRequirements(const FMassFragmentRequirements& FragmentRequirements);

	void ClearFragmentViews()
	{
		for (FFragmentView& View : FragmentViews)
		{
			View.FragmentView = TArrayView<FMassFragment>();
		}
		for (FChunkFragmentView& View : ChunkFragmentViews)
		{
			View.FragmentView.Reset();
		}
		for (FConstSharedFragmentView& View : ConstSharedFragmentViews)
		{
			View.FragmentView.Reset();
		}
		for (FSharedFragmentView& View : SharedFragmentViews)
		{
			View.FragmentView.Reset();
		}
	}

	MASSENTITY_API bool ExecuteSparseElementsFilterImpl(const FSparseEntityIterator& EntityIt) const;
	bool ExecuteSparseChunkFilterImpl(const FMassArchetypeChunk& Chunk) const;

protected:
	TFunction<void(const FMassEntityQuery*)> PreQueryCallbackFunc;
	TFunction<void(const FMassEntityQuery*)> PostQueryCallbackFunc;

	FMassQueueChunkFunction ChunkQueueFunc;

public:
	void SetPreQueryCallback(TFunction<void(const FMassEntityQuery*)> Function)
	{
		PreQueryCallbackFunc = Function;
	}

	void PreQueryCallback(const FMassEntityQuery* SourceQuery)
	{
		if (PreQueryCallbackFunc)
		{
			PreQueryCallbackFunc(SourceQuery);
		}
	}

	void SetPostQueryCallback(TFunction<void(const FMassEntityQuery*)> Function)
	{
		PostQueryCallbackFunc = Function;
	}

	bool HasChunkQueueFunc() const
	{
		return ChunkQueueFunc.IsSet();
	}

	void QueueChunkChecked(FMassChunkProcessingQueueParams&& Params)
	{
		check(ChunkQueueFunc.IsSet());
		Params.SourceContext = this;
		ChunkQueueFunc(MoveTemp(Params));
	}

	void SetChunkQueueFunc(FMassQueueChunkFunction&& InChunkQueueFunc)
	{
		ChunkQueueFunc = MoveTemp(InChunkQueueFunc);
	}

	void PostQueryCallback(const FMassEntityQuery* SourceQuery)
	{
		if (PostQueryCallbackFunc)
		{
			PostQueryCallbackFunc(SourceQuery);
		}
	}
};

//-----------------------------------------------------------------------------
// Inlines
//-----------------------------------------------------------------------------
inline FMassEntityManager& FMassExecutionContext::GetEntityManagerChecked() const
{
	return EntityManager.Get();
}

inline const TSharedRef<FMassEntityManager>& FMassExecutionContext::GetSharedEntityManager()
{
	return EntityManager;
}

inline void FMassExecutionContext::SetFlushDeferredCommands(const bool bNewFlushDeferredCommands)
{
	bFlushDeferredCommands = bNewFlushDeferredCommands;
}

inline void FMassExecutionContext::SetDeferredCommandBuffer(const TSharedPtr<FMassCommandBuffer>& InDeferredCommandBuffer)
{
	DeferredCommandBuffer = InDeferredCommandBuffer;
}

inline void FMassExecutionContext::ClearEntityCollection()
{
	EntityCollection.Reset();
}

inline void FMassExecutionContext::SetAuxData(const FInstancedStruct& InAuxData)
{
	AuxData = InAuxData;
}

inline float FMassExecutionContext::GetDeltaTimeSeconds() const
{
	return DeltaTimeSeconds;
}

inline void FMassExecutionContext::SetExecutionType(EMassExecutionContextType InExecutionType)
{
	check(InExecutionType != EMassExecutionContextType::MAX);
	ExecutionType = InExecutionType;
}

inline EMassExecutionContextType FMassExecutionContext::GetExecutionType() const
{
	return ExecutionType;
}

inline const FMassEntityQuery& FMassExecutionContext::GetCurrentQuery() const
{
	check(QueriesStack.Num());
	return *QueriesStack.Last().Query;
}

inline bool FMassExecutionContext::IsCurrentQuery(const FMassEntityQuery& Query) const
{
	return QueriesStack.Num() && QueriesStack.Last().Query == &Query;
}

inline void FMassExecutionContext::ApplyFragmentRequirements(const FMassEntityQuery& RequestingQuery)
{
	check(IsCurrentQuery(RequestingQuery));
	SetFragmentRequirements(RequestingQuery);
}

inline void FMassExecutionContext::ClearFragmentViews(const FMassEntityQuery& RequestingQuery)
{
	check(IsCurrentQuery(RequestingQuery));
	ClearFragmentViews();
}

inline bool FMassExecutionContext::ExecuteSparseElementsFilter(const FSparseEntityIterator& EntityIt) const
{
	return (bHasSparseRequirements == false) || ExecuteSparseElementsFilterImpl(EntityIt);
}

inline int32 FMassExecutionContext::GetChunkSerialModificationNumber() const
{
	return ChunkEntityIndex.ChunkSerialNumber;
}

using FMassSparseEntityIterator = FMassExecutionContext::FSparseEntityIterator;

#undef MASS_CHECK_IF_VALID
#undef MASS_CHECK_IF_READWRITE
#undef MASS_CHECK_IF_SPARSE_REQUIREMENT
