// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassArchetypeTypes.h"
#include "Async/TaskGraphInterfaces.h"


struct FMassRuntimePipeline;
namespace UE::Mass
{
	struct FProcessingContext;
} // namespace UE::Mass
struct FMassEntityHandle;
struct FMassArchetypeEntityCollection;
class UMassProcessor;


namespace UE::Mass::Executor
{
	/** Executes processors in a given RuntimePipeline */
	MASSENTITY_API void Run(FMassRuntimePipeline& RuntimePipeline, FProcessingContext& ProcessingContext);

	/** Executes given Processor. Used mainly for triggering calculations via MassCompositeProcessors, e.g processing phases */
	MASSENTITY_API void Run(UMassProcessor& Processor, FProcessingContext& ProcessingContext);

	/** Similar to the Run function, but instead of using all the entities hosted by ProcessingContext.EntitySubsystem
	 *  it is processing only the entities given by EntityID via the Entities input parameter.
	 *  Note that all the entities need to be of Archetype archetype.
	 *  Under the hood the function converts Archetype-Entities pair to FMassArchetypeEntityCollection and calls RunWithCollection.
	 */
	MASSENTITY_API void RunSparseEntities(FMassRuntimePipeline& RuntimePipeline, FProcessingContext& ProcessingContext, FMassArchetypeHandle Archetype, TConstArrayView<FMassEntityHandle> Entities);

	UE_DEPRECATED(5.8, "Use RunSparseEntities instead.")
	inline void RunSparse(FMassRuntimePipeline& RuntimePipeline, FProcessingContext& ProcessingContext, FMassArchetypeHandle Archetype, TConstArrayView<FMassEntityHandle> Entities)
	{
		RunSparseEntities(RuntimePipeline, ProcessingContext, Archetype, Entities);
	}

	/** Similar to the Run function, but instead of using all the entities hosted by ProcessingContext.EntitySubsystem
	 *  it is processing only the entities given via the EntityCollection input parameter.
	 */
	MASSENTITY_API void RunWithCollection(FMassRuntimePipeline& RuntimePipeline, FProcessingContext& ProcessingContext, const FMassArchetypeEntityCollection& EntityCollection);

	UE_DEPRECATED(5.8, "Use RunWithCollection instead.")
	inline void RunSparse(FMassRuntimePipeline& RuntimePipeline, FProcessingContext& ProcessingContext, const FMassArchetypeEntityCollection& EntityCollection)
	{
		RunWithCollection(RuntimePipeline, ProcessingContext, EntityCollection);
	}

	/** Executes given Processors array view. This function gets called under the hood by the rest of Run* functions */
	MASSENTITY_API void RunProcessorsView(TArrayView<UMassProcessor* const> Processors, FProcessingContext& ProcessingContext, TConstArrayView<FMassArchetypeEntityCollection> EntityCollections = {});

	/** 
	 *  Triggers tasks executing Processor (and potentially it's children) and returns the task graph event representing 
	 *  the task (the event will be "completed" once all the processors finish running). 
	 *  @param OnDoneNotification will be called after all the processors are done, just after flushing the command buffer.
	 *  @param OnPreCommandFlushNotification will be called after all the processors are done, just before flushing the command buffer.
	 */
	MASSENTITY_API FGraphEventRef TriggerParallelTasks(UMassProcessor& Processor, FProcessingContext&& ProcessingContext, TFunction<void()> OnDoneNotification
		, ENamedThreads::Type CurrentThread = ENamedThreads::GameThread
		, TFunction<void()> OnPreCommandFlushNotification = TFunction<void()>());

	UE_DEPRECATED(5.6, "lvalue flavor of TriggerParallelTasks has been deprecated. Use the rvalue version.")
	MASSENTITY_API FGraphEventRef TriggerParallelTasks(UMassProcessor& Processor, FProcessingContext& ProcessingContext, TFunction<void()> OnDoneNotification
		, ENamedThreads::Type CurrentThread = ENamedThreads::GameThread);
} // namespace UE::Mass::Executor
