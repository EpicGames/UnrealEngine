// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
// HEADER_UNIT_SKIP - Deprecated
#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.8, "MassExternalSubsystemTraits.h has moved to the MassCore module and been renamed to Mass/ExternalSubsystemTraits.h. "
    "Include Mass/ExternalSubsystemTraits.h from MassCore and add a MassCore dependency to your .Build.cs.")
#include "Mass/ExternalSubsystemTraits.h"
