// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityHandle.h"
#include "MassProcessingTypes.h"
#include "MassEntityTypes.h"
#include "MassArchetypeTypes.h"
#include "Misc/Fork.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
// the following is a new header, but it contains some types moved over from other places, and including
// said header ties everything together. Engine code has been updated to not require it
#include "MassObserverNotificationTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassObserverManager.generated.h"


struct FMassObserverExecutionContext;
struct FMassEntityManager;
struct FMassArchetypeEntityCollection;
class UMassProcessor;
class UMassObserverProcessor;
namespace UE::Mass
{
	struct FProcessingContext;
	namespace ObserverManager
	{
		struct FDeprecationHelper;
		struct FBufferedNotificationExecutioner;
		struct FBufferedCreationNotificationExecutioner;

		struct FBufferedNotification;
		struct FCreationNotificationHandle;
		struct FObserverLock;
		struct FCreationContext;
	} // namespace ObserverManager
} // namespace UE::Mass

/**
 * A wrapper type for a TMap to support having array-of-maps UPROPERTY members in FMassObserverManager
 */
USTRUCT()
struct FMassObserversMap
{
	GENERATED_BODY()

	// a helper accessor simplifying access while still keeping Container private
	TMap<TObjectPtr<const UScriptStruct>, FMassRuntimePipeline>& operator*()
	{
		return Container;
	}

	void DebugAddUniqueProcessors(TArray<const UMassProcessor*>& OutProcessors) const;

private:
	UPROPERTY()
	TMap<TObjectPtr<const UScriptStruct>, FMassRuntimePipeline> Container;
};

/** 
 * A type that encapsulates logic related to notifying interested parties of entity composition changes. Upon creation it
 * reads information from UMassObserverRegistry and instantiates processors interested in handling given fragment
 * type addition or removal.
 */
USTRUCT()
struct FMassObserverManager
{
	GENERATED_BODY()

	/** convenience aliases */
	using FObserverLock = UE::Mass::ObserverManager::FObserverLock;
	using FBufferedNotification = UE::Mass::ObserverManager::FBufferedNotification;
	using FCreationNotificationHandle = UE::Mass::ObserverManager::FCreationNotificationHandle;
	using FCreationContext = UE::Mass::ObserverManager::FCreationContext;

	MASSENTITY_API FMassObserverManager();

	FMassEntityManager& GetEntityManager();

	bool HasObserversForBitSet(const FMassElementBitSet& InQueriedBitSet, const EMassObservedOperation Operation) const
	{
		return (InQueriedBitSet.IsEmpty() == false) && ObservedElements[static_cast<uint8>(Operation)].HasAny(InQueriedBitSet);
	}

	bool HasObserversForComposition(const FMassElementBitSet& InQueriedBitSet, const EMassObservedOperation Operation) const
	{
		return HasObserversForBitSet(InQueriedBitSet, Operation);
	}

	bool HasObserversForComposition(const FMassArchetypeCompositionDescriptor& Composition, const EMassObservedOperation Operation) const
	{
		return HasObserversForBitSet(Composition.GetElementsBitSet(), Operation);
	}

	/** @return whether there are observers watching affected elements */
	MASSENTITY_API bool OnPostEntitiesCreated(const FMassArchetypeEntityCollection& EntityCollection);

	/** @return whether there are observers watching affected elements */
	MASSENTITY_API bool OnPostEntityCreated(const FMassEntityHandle Entity, const FMassElementBitSet& ElementsBitSet);

	bool OnPostEntityCreated(const FMassEntityHandle Entity, const FMassArchetypeCompositionDescriptor& Composition)
	{
		return OnPostEntityCreated(Entity, Composition.GetElementsBitSet());
	}

	/** @return whether there are observers watching affected elements */
	MASSENTITY_API bool OnPreEntitiesDestroyed(const FMassArchetypeEntityCollection& EntityCollection);

	/** @return whether there are observers watching affected elements */
	MASSENTITY_API bool OnPreEntitiesDestroyed(UE::Mass::FProcessingContext& ProcessingContext, const FMassArchetypeEntityCollection& EntityCollection);

	/** @return whether there are observers watching affected elements */
	MASSENTITY_API bool OnPreEntityDestroyed(const FMassEntityHandle Entity, const FMassElementBitSet& ElementsBitSet);

	bool OnPreEntityDestroyed(const FMassArchetypeCompositionDescriptor& ArchetypeComposition, const FMassEntityHandle Entity)
	{
		return OnPreEntityDestroyed(Entity, ArchetypeComposition.GetElementsBitSet());
	}

	/** @return whether there are observers watching affected elements */
	bool OnPostCompositionAdded(const FMassEntityHandle Entity, const FMassElementBitSet& CompositionBitSet)
	{
		return OnCompositionChanged(Entity, CompositionBitSet, EMassObservedOperation::AddElement);
	}

	/** @return whether there are observers watching affected elements */
	bool OnPreCompositionRemoved(const FMassEntityHandle Entity, const FMassElementBitSet& CompositionBitSet)
	{
		return OnCompositionChanged(Entity, CompositionBitSet, EMassObservedOperation::RemoveElement);
	}

	/** @return whether there are observers watching affected elements */
	bool OnPostCompositionAdded(const FMassEntityHandle Entity, const FMassArchetypeCompositionDescriptor& Composition)
	{
		return OnPostCompositionAdded(Entity, Composition.GetElementsBitSet());
	}

	/** @return whether there are observers watching affected elements */
	bool OnPreCompositionRemoved(const FMassEntityHandle Entity, const FMassArchetypeCompositionDescriptor& Composition)
	{
		return OnPreCompositionRemoved(Entity, Composition.GetElementsBitSet()); 
	}

protected:
	/**
	 * helper struct for holding either a single entity handle or an archetype collection reference.
	 * The type is highly transient, do not store instances of it. Its only function is to 
	 * allow us to have a single OnCompositionChange implementation rather than having two separate
	 * implementations, one for entity handle, the other for archetype collection.
	 */
	struct FCollectionRefOrHandle
	{
		UE_NONCOPYABLE(FCollectionRefOrHandle);

		static FMassArchetypeEntityCollection DummyCollection;
		explicit FCollectionRefOrHandle(FMassEntityHandle InEntityHandle)
			: EntityHandle(InEntityHandle)
			, EntityCollection(DummyCollection)
		{	
		}

		explicit FCollectionRefOrHandle(const FMassArchetypeEntityCollection& InEntityCollection)
			: EntityCollection(InEntityCollection)
		{	
		}

		FMassEntityHandle EntityHandle;
		const FMassArchetypeEntityCollection& EntityCollection;
	};
	MASSENTITY_API bool OnCompositionChanged(FCollectionRefOrHandle&& EntityCollection, const FMassElementBitSet& Composition
		, const EMassObservedOperation Operation, UE::Mass::FProcessingContext* ProcessingContext = nullptr);

public:
	bool OnCompositionChanged(const FMassArchetypeEntityCollection& EntityCollection, const FMassArchetypeCompositionDescriptor& Composition
		, const EMassObservedOperation Operation, UE::Mass::FProcessingContext* ProcessingContext = nullptr)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassObserver_OnCompositionChanged_Collection);
		return OnCompositionChanged(FCollectionRefOrHandle(EntityCollection), Composition.GetElementsBitSet(), Operation, ProcessingContext);
	}

	bool OnCompositionChanged(FMassEntityHandle EntityHandle, const FMassArchetypeCompositionDescriptor& Composition
		, const EMassObservedOperation Operation, UE::Mass::FProcessingContext* ProcessingContext = nullptr)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassObserver_OnCompositionChanged_Entity);
		return OnCompositionChanged(FCollectionRefOrHandle(EntityHandle), Composition.GetElementsBitSet(), Operation, ProcessingContext);
	}

	template<typename T, typename U = std::decay_t<T>> requires (std::is_same_v<U, FMassFragmentBitSet> || std::is_same_v<U, FMassTagBitSet>)
	bool OnCompositionChanged(const FMassArchetypeEntityCollection& EntityCollection, T&& BitSet
		, const EMassObservedOperation Operation, UE::Mass::FProcessingContext* ProcessingContext = nullptr)
	{
		return OnCompositionChanged(EntityCollection, FMassElementBitSet(MoveTemp(BitSet)), Operation, ProcessingContext);
	}

	template<typename T, typename U = std::decay_t<T>> requires (std::is_same_v<U, FMassFragmentBitSet> || std::is_same_v<U, FMassTagBitSet>)
	bool OnCompositionChanged(const FMassEntityHandle Entity, T&& BitSet
		, const EMassObservedOperation Operation, UE::Mass::FProcessingContext* ProcessingContext = nullptr)
	{
		return OnCompositionChanged(Entity, FMassElementBitSet(MoveTemp(BitSet)), Operation, ProcessingContext);
	}

	MASSENTITY_API void AddObserverInstance(TNotNull<const UScriptStruct*> ElementType, EMassObservedOperationFlags OperationFlags, TNotNull<UMassProcessor*> ObserverProcessor);
	MASSENTITY_API void AddObserverInstance(const UScriptStruct& ElementType, EMassObservedOperation Operation, UMassProcessor& ObserverProcessor);
	MASSENTITY_API void AddObserverInstance(TNotNull<UMassObserverProcessor*> ObserverProcessor);
	MASSENTITY_API void RemoveObserverInstance(TNotNull<UMassObserverProcessor*> ObserverProcessor);
	MASSENTITY_API void RemoveObserverInstance(TNotNull<const UScriptStruct*> ElementType, EMassObservedOperationFlags OperationFlags, TNotNull<UMassProcessor*> ObserverProcessor);
	MASSENTITY_API void RemoveObserverInstance(const UScriptStruct& ElementType, EMassObservedOperation Operation, UMassProcessor& ObserverProcessor);

	MASSENTITY_API void ReleaseCreationHandle(UE::Mass::ObserverManager::FCreationNotificationHandle InCreationNotificationHandle);

	MASSENTITY_API void DebugGatherUniqueProcessors(TArray<const UMassProcessor*>& OutProcessors) const;

protected:
	friend FMassEntityManager;
	MASSENTITY_API explicit FMassObserverManager(FMassEntityManager& Owner);

	MASSENTITY_API void Initialize();
	MASSENTITY_API void DeInitialize();

	friend UE::Mass::ObserverManager::FBufferedNotificationExecutioner;
	friend UE::Mass::ObserverManager::FBufferedCreationNotificationExecutioner;

	static MASSENTITY_API void HandleElementsImpl(UE::Mass::FProcessingContext& ProcessingContext, TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
		, FMassObserverExecutionContext&& ObserverContext, FMassObserversMap& HandlersContainer);

	/** Coalesces all the elements observed in all the collections and executes all the observers at once */
	MASSENTITY_API bool OnCollectionsCreatedImpl(UE::Mass::FProcessingContext& ProcessingContext, TConstArrayView<FMassArchetypeEntityCollection> EntityCollections);

	MASSENTITY_API void OnPostFork(EForkProcessRole);

	MASSENTITY_API void OnModulePackagesUnloaded(TConstArrayView<UPackage*> Packages);

	MASSENTITY_API TSharedRef<FObserverLock> GetOrMakeObserverLock();
	TSharedPtr<FObserverLock> GetObserverLock() const;
	bool IsLocked() const
	{
		return ActiveObserverLock.IsValid();
	}
	MASSENTITY_API TSharedRef<FCreationContext> GetOrMakeCreationContext();
	MASSENTITY_API TSharedRef<FCreationContext> GetOrMakeCreationContext(TConstArrayView<FMassEntityHandle> ReservedEntities, FMassArchetypeEntityCollection&& EntityCollection);
	TSharedPtr<FCreationContext> GetCreationContext() const;
	
	//-----------------------------------------------------------------------------
	// Observers locking logic
	//-----------------------------------------------------------------------------
	friend FObserverLock;

	/**
	 * Resumes observer triggering. All notifications collected in lock's BufferedNotifications will be processed at this point.
	 *
	 * Note that due to all the notifications being sent our are being sent post-factum the "OnPreRemove" 
	 * observers won't be able to access the data being removed, since the remove operation has already been performed.
	 * All the instances of removal-observers being triggered will be logged.
	 * 
	 * Intended to be called automatically by ~FObserverLock
	 */
	MASSENTITY_API void ResumeExecution(FObserverLock& LockBeingReleased);

	/**
	 * Never access directly, use GetOrMakeObserverLock or GetOrMakeCreationContext instead.
	 * The Mass Processing Phase synchronously creates a lock before dispatching parallel mass tasks
	 * Lock should be thread-safe for creation, fetching, and adding items.
	 */
	TWeakPtr<FObserverLock> ActiveObserverLock;
	int32 LocksCount = 0;
	TWeakPtr<FCreationContext> ActiveCreationContext;

	FMassElementBitSet ObservedElements[static_cast<uint8>(EMassObservedOperation::MAX)];

	UPROPERTY()
	FMassObserversMap ElementObservers[static_cast<uint8>(EMassObservedOperation::MAX)];

	FDelegateHandle ModulesUnloadedHandle;

	/** 
	 * The owning EntityManager. No need for it to be a UPROPERTY since by design we don't support creation of 
	 * FMassObserverManager outside a FMassEntityManager instance 
	 */
	FMassEntityManager& EntityManager;

#if WITH_MASSENTITY_DEBUG
	uint32 LockedNotificationSerialNumber = 1;
	uint32 DebugNonTrivialResumeExecutionCount = 0;
#endif // WITH_MASSENTITY_DEBUG

public:
	UE_DEPRECATED(5.6, "*FragmentOrTag* functions have been deprecated. Use OnCompositionChanged instead.")
	MASSENTITY_API void OnPostFragmentOrTagAdded(const UScriptStruct& FragmentOrTagType, const FMassArchetypeEntityCollection& EntityCollection);
	UE_DEPRECATED(5.6, "*FragmentOrTag* functions have been deprecated. Use OnCompositionChanged instead.")
	MASSENTITY_API void OnPreFragmentOrTagRemoved(const UScriptStruct& FragmentOrTagType, const FMassArchetypeEntityCollection& EntityCollection);
	UE_DEPRECATED(5.6, "*FragmentOrTag* functions have been deprecated. Use OnCompositionChanged instead.")
	MASSENTITY_API void OnFragmentOrTagOperation(const UScriptStruct& FragmentOrTagType, const FMassArchetypeEntityCollection& EntityCollection, const EMassObservedOperation Operation);
	UE_DEPRECATED(5.6, "Use the other OnPostEntitiesCreated implementation.")
	MASSENTITY_API bool OnPostEntitiesCreated(UE::Mass::FProcessingContext& InProcessingContext, TConstArrayView<FMassArchetypeEntityCollection> EntityCollections);
	UE_DEPRECATED(5.6, "Use the other OnCompositionChanged implementation.")
	MASSENTITY_API bool OnCompositionChanged(UE::Mass::FProcessingContext& InProcessingContext, const FMassArchetypeEntityCollection& EntityCollection
		, const FMassArchetypeCompositionDescriptor& Composition, const EMassObservedOperation Operation);
protected:
	UE_DEPRECATED(5.6, "Use HandleElementsImpl instead.")
	MASSENTITY_API void HandleSingleEntityImpl(const UScriptStruct& FragmentType, const FMassArchetypeEntityCollection& EntityCollection, FMassObserversMap& HandlersContainer);
	UE_DEPRECATED(5.7, "Use the implementation with the FMassObserverExecutionContext parameter")
	static void HandleElementsImpl(UE::Mass::FProcessingContext& ProcessingContext, TConstArrayView<FMassArchetypeEntityCollection> EntityCollections
		, TArrayView<const UScriptStruct*> ObservedTypes, FMassObserversMap& HandlersContainer);
	UE_DEPRECATED(5.7, "Use the HandleElementsImpl implementation with the FMassObserverExecutionContext parameter")
	static void HandleFragmentsImpl(UE::Mass::FProcessingContext& ProcessingContext, const FMassArchetypeEntityCollection& EntityCollection
		, TArrayView<const UScriptStruct*> ObservedTypes, FMassObserversMap& HandlersContainer);

	UE_DEPRECATED(5.7, "Rvalue ref implementation is no longer available, it didn't do anything special. Please use the other OnCollectionsCreatedImpl implementation instead.")
	MASSENTITY_API bool OnCollectionsCreatedImpl(UE::Mass::FProcessingContext& ProcessingContext, TArray<FMassArchetypeEntityCollection>&& EntityCollections);

	friend UE::Mass::ObserverManager::FDeprecationHelper;
};

template<>
struct TStructOpsTypeTraits<FMassObserverManager> : public TStructOpsTypeTraitsBase2<FMassObserverManager>
{
	enum
	{
		WithCopy = false,
	};
};

//----------------------------------------------------------------------//
// inlines
//----------------------------------------------------------------------//
inline FMassEntityManager& FMassObserverManager::GetEntityManager()
{
	return EntityManager;
}

inline TSharedPtr<FMassObserverManager::FObserverLock> FMassObserverManager::GetObserverLock() const
{
	return ActiveObserverLock.Pin();
}

inline TSharedPtr<FMassObserverManager::FCreationContext> FMassObserverManager::GetCreationContext() const
{
	return ActiveCreationContext.Pin();
}
