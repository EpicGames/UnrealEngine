// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityHandle.h"
#include "Misc/TVariant.h"
#include "MassEntityCollection.h"
#include "MassEntityTypes.h"

#define UE_API MASSENTITY_API

#define UE_CHECK_OWNER_THREADID() checkf(OwnerThreadId == FPlatformTLS::GetCurrentThreadId(), TEXT("%hs: all FObserverLock operations are expected to be run in a single thread"), __FUNCTION__)

struct FMassObserverManager;
namespace UE::Mass
{
	struct FProcessingContext;
	namespace ObserverManager
	{
		enum class
		UE_DEPRECATED(5.7, "The type is no longer being used, replaced by EMassObservedOperation")
		EObservedOperationNotification : uint8
		{
			Add = static_cast<uint8>(EMassObservedOperation::Add),
			Remove = static_cast<uint8>(EMassObservedOperation::Remove),
			Create
		};

		/**
		 * The type represents a single "operation", as observed by the registered observers, an operation that has been performed
		 * while the FObserverLock was active. Instances of FBufferedNotification contain all the information needed
		 * to send out the necessary notification once the observers lock gets released.
		 *
		 * Note that the type contains information necessary to sent our notification. In case of "Remove" notifications the
		 * operations has already been performed, and the data being removed is no longer available to the observers, and
		 * instances of FBufferedNotification do not host this information either. 
		 */
		struct FBufferedNotification
		{
			struct FEmptyComposition
			{
			};
			EMassObservedOperation Type;
			FMassElementBitSet CompositionChange;
			using FEntitiesContainer = TVariant<FEntityCollection, FMassEntityHandle>;
			FEntitiesContainer AffectedEntities;

			FBufferedNotification(const EMassObservedOperation InType, FMassElementBitSet&& Composition, FEntitiesContainer&& Entities)
				: Type(InType)
				, CompositionChange(Forward<FMassElementBitSet>(Composition))
				, AffectedEntities(MoveTemp(Entities))
			{
			}

			template<typename TEntities>
			FBufferedNotification(const EMassObservedOperation InType, FMassElementBitSet&& Composition, TEntities&& Entities)
				: Type(InType)
				, CompositionChange(MoveTemp(Composition))
				, AffectedEntities(TInPlaceType<typename TDecay<TEntities>::Type>(), Forward<TEntities>(Entities))
			{
			}

			FBufferedNotification(const EMassObservedOperation InType, FMassElementBitSet&& Composition, const FMassArchetypeEntityCollection& Entities)
				: Type(InType)
				, CompositionChange(MoveTemp(Composition))
				, AffectedEntities(TInPlaceType<typename TDecay<FEntityCollection>::Type>(), Entities)
			{
			}

			template<typename TEntities>
			FBufferedNotification(const EMassObservedOperation InType, TEntities&& Entities)
				: Type(InType)
				, AffectedEntities(TInPlaceType<typename TDecay<TEntities>::Type>(), Forward<TEntities>(Entities))
			{
			}

			FBufferedNotification(const EMassObservedOperation InType, const FMassArchetypeEntityCollection& Entities)
				: Type(InType)
				, AffectedEntities(TInPlaceType<typename TDecay<FEntityCollection>::Type>(), Entities)
			{
			}

			bool IsCreationNotification() const
			{
				return Type == EMassObservedOperation::CreateEntity;
			}

			void AddHandle(const FMassEntityHandle EntityHandle)
			{
				FEntityCollection* StoredCollection = AffectedEntities.TryGet<FEntityCollection>();
				if (StoredCollection == nullptr)
				{
					StoredCollection = &ConvertStoredHandleToCollection(TConstArrayView<FMassEntityHandle>());
				}
				StoredCollection->AddHandle(EntityHandle);
			}

			inline void AppendEntities(const FMassEntityHandle EntityHandle)
			{
				AddHandle(EntityHandle);
			}

			void AppendEntities(const TConstArrayView<FMassEntityHandle> InEntityHandles)
			{
				if (FEntityCollection* StoredCollection = AffectedEntities.TryGet<FEntityCollection>())
				{
					StoredCollection->AppendHandles(InEntityHandles);
				}
				else
				{
					ConvertStoredHandleToCollection(InEntityHandles);
				}
			}

			void AppendEntities(const TConstArrayView<FMassEntityHandle> InEntityHandles, FMassArchetypeEntityCollection&& EntityCollection)
			{
				if (FEntityCollection* StoredCollection = AffectedEntities.TryGet<FEntityCollection>())
				{
					StoredCollection->AppendHandles(InEntityHandles, Forward<FMassArchetypeEntityCollection>(EntityCollection));
				}
				else
				{
					ConvertStoredHandleToCollection(InEntityHandles);
					// we're ignoring EntityCollection since the collections will need to be rebuilt anyway,
					// due to AffectedEntities already containing some data before this call
				}
			}

			template<typename T> requires std::is_same_v<typename TDecay<T>::Type, FMassArchetypeEntityCollection>
			void AppendEntities(T&& InEntityCollection)
			{
				if (FEntityCollection* StoredCollection = AffectedEntities.TryGet<FEntityCollection>())
				{
					StoredCollection->AppendCollection(Forward<T>(InEntityCollection));
				}
				else
				{
					ConvertStoredHandleToCollection(Forward<T>(InEntityCollection));
				}
			}

			void DirtyAffectedEntities()
			{
				if (FEntityCollection* StoredCollection = AffectedEntities.TryGet<FEntityCollection>())
				{
					StoredCollection->MarkDirty();
				}
			}

			static bool AreCompositionsEqual(const FMassElementBitSet& A, const FMassElementBitSet& B)
			{
				return A.IsEquivalent(B);
			}

		private:
			template<typename TEntities> 
			FEntityCollection& ConvertStoredHandleToCollection(TEntities&& InEntities)
			{
				// AffectedEntities holds a single handle. We need to extract it and emplace a FEntityCollection instance 
				const FMassEntityHandle StoredHandle = AffectedEntities.Get<FMassEntityHandle>();
				AffectedEntities.Emplace<FEntityCollection>(Forward<TEntities>(InEntities));
				FEntityCollection& StoredEntities = AffectedEntities.Get<FEntityCollection>();
				StoredEntities.AddHandle(StoredHandle);
				return StoredEntities;
			}
		};

		/** Simple handle type representing a entity creation notification as stored by FObserverLock */
		struct FCreationNotificationHandle
		{
			bool IsSet() const
			{
				return OpIndex != INDEX_NONE;
			}

			operator int() const
			{
				return OpIndex;
			}

		private:
			friend FMassObserverManager;

			/**
			 * set upon creation to the value of LockedNotificationSerialNumber.
			 * This property's value is checked when the creation handle gets "released" via
			 * FMassObserverManager::ReleaseCreationHandle
			 */ 
			uint32 SerialNumber = 0;

			int32 OpIndex = INDEX_NONE;
		};

		/**
		 * Once created with MassObserverManager.GetOrMakeObserverLock will prevent triggering
		 * observers and instead buffer all the notifications to be sent.
		 * Once the FObserverLock gets released it will call MassObserverManager.ResumeExecution
		 * that will send out all the buffered notifications.
		 * 
		 * @note that due to the buffering, all the "Remove" operation observers will be sent out later
		 * than usually - without locking those observers get triggered before the removal operation is
		 * performed, and as such have access to the data "about to be removed". Removal observers sent out
		 * after lock release won't have access to that information.
		 * 
		 * There's a special path for freshly created entities, see FCreationContext for more details.
		 */
		struct FObserverLock
		{
			FObserverLock() = default;
			UE_API ~FObserverLock();

			TWeakPtr<FMassEntityManager> GetWeakEntityManager() const
			{
				return WeakEntityManager;
			}

			void MarkCreationNotificationDirty(FCreationNotificationHandle CreationHandle)
			{
				ensureMsgf(CreationHandle == CreationNotificationIndex, TEXT("Given creation handle doesn't match this Lock's data"));
				checkf(BufferedNotifications.IsValidIndex(CreationHandle), TEXT("Given CreationHandle doesn't match stored notifications"));
				BufferedNotifications[CreationHandle].DirtyAffectedEntities();
			}

			const FBufferedNotification& GetCreationNotification(FCreationNotificationHandle CreationHandle) const
			{
				ensureMsgf(CreationHandle == CreationNotificationIndex, TEXT("Given creation handle doesn't match this Lock's data"));
				checkf(BufferedNotifications.IsValidIndex(CreationHandle), TEXT("Given CreationHandle doesn't match stored notifications"));

				return BufferedNotifications[CreationHandle];
			}

			FTransactionallySafeCriticalSection& GetNotificationLock()
			{
				return NotificationLock;
			}
			
		private:
			UE_API explicit FObserverLock(FMassObserverManager& ObserverManager);

			int32 GetOrCreateCreationNotification()
			{
				UE::TScopeLock ScopeLock(NotificationLock);

				if (CreationNotificationIndex == INDEX_NONE)
				{
					CreationNotificationIndex = BufferedNotifications.Num();
					BufferedNotifications.Emplace(EMassObservedOperation::CreateEntity, UE::Mass::FEntityCollection());
				}
				return CreationNotificationIndex;
			}

			bool ReleaseCreationNotification(FCreationNotificationHandle CreationHandle)
			{
				UE::TScopeLock ScopeLock(NotificationLock);

				checkf(BufferedNotifications.IsValidIndex(CreationHandle), TEXT("Given CreationHandle doesn't match stored notifications"));

				int32 CreationHandleOpIndex = CreationHandle;
				if (CreationNotificationIndex == CreationHandleOpIndex)
				{
					CreationNotificationIndex = INDEX_NONE;
					return true;
				}
				// else 
				ensureMsgf(CreationHandle == CreationNotificationIndex, TEXT("Given creation handle doesn't match this Lock's data"));
				return false;
			}

			int32 AddCreatedEntity(FMassEntityHandle CreatedEntity)
			{
				UE::TScopeLock ScopeLock(NotificationLock);

				if (CreationNotificationIndex == INDEX_NONE)
				{
					CreationNotificationIndex = BufferedNotifications.Num();
					BufferedNotifications.Emplace(EMassObservedOperation::CreateEntity, CreatedEntity);
				}
				else
				{
					BufferedNotifications[CreationNotificationIndex].AddHandle(CreatedEntity);
				}

				return CreationNotificationIndex;
			}

			int32 AddCreatedEntities(const TConstArrayView<FMassEntityHandle> InCreatedEntities)
			{
				UE::TScopeLock ScopeLock(NotificationLock);

				if (CreationNotificationIndex == INDEX_NONE)
				{
					CreationNotificationIndex = BufferedNotifications.Num();
					BufferedNotifications.Emplace(EMassObservedOperation::CreateEntity, UE::Mass::FEntityCollection(InCreatedEntities));
				}
				else
				{
					BufferedNotifications[CreationNotificationIndex].AppendEntities(InCreatedEntities);
				}

				return CreationNotificationIndex;
			}

			int32 AddCreatedEntities(const TConstArrayView<FMassEntityHandle> InCreatedEntities, FMassArchetypeEntityCollection&& InEntityCollection)
			{
				UE::TScopeLock ScopeLock(NotificationLock);

				if (CreationNotificationIndex == INDEX_NONE)
				{
					CreationNotificationIndex = BufferedNotifications.Num();
					BufferedNotifications.Emplace(EMassObservedOperation::CreateEntity
						, UE::Mass::FEntityCollection(InCreatedEntities, Forward<FMassArchetypeEntityCollection>(InEntityCollection)));
				}
				else
				{
					BufferedNotifications[CreationNotificationIndex].AppendEntities(InCreatedEntities, Forward<FMassArchetypeEntityCollection>(InEntityCollection));
				}

				return CreationNotificationIndex;
			}

			template<typename T> requires std::is_same_v<typename TDecay<T>::Type, FMassArchetypeEntityCollection>
			int32 AddCreatedEntitiesCollection(T&& InEntityCollection)
			{
				UE::TScopeLock ScopeLock(NotificationLock);

				if (CreationNotificationIndex == INDEX_NONE)
				{
					CreationNotificationIndex = BufferedNotifications.Num();
					BufferedNotifications.Emplace(EMassObservedOperation::CreateEntity, UE::Mass::FEntityCollection(Forward<T>(InEntityCollection)));
				}
				else
				{
					BufferedNotifications[CreationNotificationIndex].AppendEntities<T>(InEntityCollection);
				}

				return CreationNotificationIndex;
			}

			template<typename TEntities>
			void AddNotification(const EMassObservedOperation OperationType, TEntities&& Entities, FMassElementBitSet&& OverlapBitSet)
			{
				UE::TScopeLock ScopeLock(NotificationLock);

				if (BufferedNotifications.Num()
					&& BufferedNotifications.Last().Type == OperationType
					&& FBufferedNotification::AreCompositionsEqual(BufferedNotifications.Last().CompositionChange, OverlapBitSet))
				{
					BufferedNotifications.Last().AppendEntities(Forward<TEntities>(Entities));
				}
				else
				{
					BufferedNotifications.Emplace(OperationType, MoveTemp(OverlapBitSet), Forward<TEntities>(Entities));
				}
			}

			friend FMassObserverManager;
			/** To be called in case of processor forking. */
			UE_API void ForceUpdateCurrentThreadID();

			/**
			 * Identifies the thread where given FObserverLock instance was created. All subsequent operations are 
			 * expected to be run in the same thread.
			 */
			uint32 OwnerThreadId;

			int32 CreationNotificationIndex = INDEX_NONE;

			TArray<FBufferedNotification> BufferedNotifications;

			FMassArchetypeEntityCollection::EDuplicatesHandling CollectionCreationDuplicatesHandling = FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates;

			/** Point to outer EntityManager. Used to obtain ObserverManager in type's destructor*/
			TWeakPtr<FMassEntityManager> WeakEntityManager;

	#if WITH_MASSENTITY_DEBUG
			uint32 LockSerialNumber = 0;
	#endif // WITH_MASSENTITY_DEBUG
			FTransactionallySafeCriticalSection NotificationLock;
		};

		/**
		 * A dedicated structure for ensuring the "on entities creation" observers get notified only once all other 
		 * initialization operations are done and this creation context instance gets released. 
		 */
		struct FCreationContext
		{
			UE_API TArray<FMassArchetypeEntityCollection> GetEntityCollections(const FMassEntityManager& EntityManager) const;

			/** Function for debugging/testing purposes. We don't expect users to ever call it, always get collections via GetEntityCollections */
			UE_API bool DebugAreEntityCollectionsUpToDate() const;

			UE_API ~FCreationContext();

			static UE_API TSharedRef<FCreationContext> DebugCreateDummyCreationContext();

			UE_DEPRECATED(5.6, "Use the other GetEntityCollections flavor insteand")
			UE_API TConstArrayView<FMassArchetypeEntityCollection> GetEntityCollections() const;
			UE_DEPRECATED(5.6, "Functionality no longer available")
			UE_API int32 GetSpawnedNum() const;
			UE_DEPRECATED(5.6, "Do not use, internal use only")
			UE_API bool IsDirty() const;
			UE_DEPRECATED(5.6, "Manually adding entities directly to the creation context is not longer supported and is not taking place automatically")
			UE_API void AppendEntities(const TConstArrayView<FMassEntityHandle>);
			UE_DEPRECATED(5.6, "Manually adding entities directly to the creation context is not longer supported and is not taking place automatically")
			UE_API void AppendEntities(const TConstArrayView<FMassEntityHandle>, FMassArchetypeEntityCollection&&);
			
			/**
			 *	Called in response to composition mutating operation - these operations invalidate stored collections
			 */
			UE_DEPRECATED_FORGAME(5.6, "Do not use, internal use only")
			void MarkDirty()
			{
				Lock->MarkCreationNotificationDirty(CreationHandle);
			}

		private:
			UE_API FCreationContext();

			FCreationContext(TSharedRef<FObserverLock>&& InLock)
				: Lock(Forward<TSharedRef<FObserverLock>>(InLock))
			{	
			}
			FCreationContext(const TSharedRef<FObserverLock>& InLock)
				: Lock(InLock)
			{
			}

			TSharedRef<FObserverLock> GetObserverLock() const
			{
				return Lock;
			}

			bool IsValid() const
			{
				return CreationHandle.IsSet();
			}

			friend FMassObserverManager;
			TSharedRef<FObserverLock> Lock;
			FCreationNotificationHandle CreationHandle;
		};
	} // namespace UE::Mass::ObserverManager
} // namespace UE::Mass

#undef UE_CHECK_OWNER_THREADID

#undef UE_API
