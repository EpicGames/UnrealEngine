// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassObserverProcessor.generated.h"

#define UE_API MASSENTITY_API


namespace UE::Mass::ObserverManager
{
	struct FObserverContextIterator;
} // namespace UE::Mass::ObserverManager

/**
 * Instances of the type will be fed into FMassRuntimePipeline.AuxData and at execution time will
 * be available to observer processors via FMassExecutionContext.GetAuxData() 
 */
USTRUCT()
struct FMassObserverExecutionContext
{
	GENERATED_BODY()

	FMassObserverExecutionContext() = default;
	FMassObserverExecutionContext(const EMassObservedOperation InOperation, const TArrayView<const UScriptStruct*> InTypesInOperation)
		: Operation(InOperation), TypesInOperation(InTypesInOperation)
	{	
	}

	EMassObservedOperation GetOperationType() const
	{
		return Operation;
	}

	TConstArrayView<const UScriptStruct*> GetTypesInOperation() const
	{
		return TypesInOperation;
	}

	const UScriptStruct* GetCurrentType() const
	{
		return CurrentTypeIndex != INDEX_NONE ? TypesInOperation[CurrentTypeIndex] : nullptr;
	}

	bool IsValid() const
	{
		return Operation < EMassObservedOperation::MAX
			&& TypesInOperation.IsValidIndex(CurrentTypeIndex);
	}

private:
	friend UE::Mass::ObserverManager::FObserverContextIterator;
	EMassObservedOperation Operation = EMassObservedOperation::MAX;
	TArrayView<const UScriptStruct*> TypesInOperation;
	int32 CurrentTypeIndex = INDEX_NONE;
};

/**
 * Base class for Processors that are used as "observers" of entity operations.
 * An observer declares the types of Mass elements it cares about (Fragments and Tags supported) - via
 * the ObservedTypes array - and the types of
 * operations it wants to be notified of - via ObservedOperations.
 *
 * When an observed operation takes place the processor's regular execution will take place, with
 * ExecutionContext's "auxiliary data" (obtained by calling GetAuxData) being filled with an instance of FMassObserverExecutionContext,
 * that can be used to get information about the type being handled and the kind of operation.
 *
 * To observe multiple types, populate the ObservedTypes array in the constructor:
 * @code
 *   UMyObserver::UMyObserver()
 *   {
 *       ObservedTypes = { FMyFragment::StaticStruct(), FOtherFragment::StaticStruct() };
 *       ObservedOperations = EMassObservedOperationFlags::Add;
 *   }
 * @endcode
 *
 * When multiple observed types are added simultaneously, Execute() fires exactly once per entity batch
 * thanks to deduplication in FMassObserverManager::HandleElementsImpl.
 */
UCLASS(MinimalAPI, abstract)
class UMassObserverProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassObserverProcessor();
	UE_API explicit UMassObserverProcessor(const FObjectInitializer& ObjectInitializer);

	EMassObservedOperationFlags GetObservedOperations() const;
	/**
	 * Observers support observing multiple fragments now, but if you know there's only one you
	 * can fetch it with this function. The function will assert if the processor is configured to observe more types.
	 */
	TNotNull<const UScriptStruct*> GetSingleObservedTypeChecked() const;
	const UScriptStruct* GetSingleObservedType() const;
	UE_DEPRECATED(5.8, "UMassObserverProcessor::GetObservedTypeChecked is deprecated now. Use GetSingleObservedTypeChecked")
	TNotNull<const UScriptStruct*> GetObservedTypeChecked() const;
	TConstArrayView<const UScriptStruct*> GetObservedTypes() const;

protected:
	UE_API virtual void PostInitProperties() override;

	/**
	 * Called from PostInitProperties on the CDO when ObservedOperations is set.
	 * By default, registers this class for all entries in ObservedTypes. Override to customize registration.
	 */
	UE_API virtual void Register();

private:
	void InitObserverDefaults();

protected:
	/** Determines which Fragment or Tag type this given UMassObserverProcessor will be observing (single-type legacy). */
	UE_DEPRECATED(5.8, "UMassObserverProcessor::ObservedType is deprecated. Use ObservedTypes instead")
	UPROPERTY()
	TObjectPtr<const UScriptStruct> ObservedType;

	/** Fragment or Tag types this observer watches. Populate this array in the constructor. */
	UPROPERTY()
	TArray<TObjectPtr<const UScriptStruct>> ObservedTypes;

	UPROPERTY(EditDefaultsOnly, Category = Processor, config)
	bool bAutoRegisterWithObserverRegistry = true;

	UE_DEPRECATED(5.7, "UMassObserverProcessor::Operation is deprecated. Use ObservedOperations instead")
	EMassObservedOperation Operation = EMassObservedOperation::MAX;

	EMassObservedOperationFlags ObservedOperations = EMassObservedOperationFlags::None;
};


//----------------------------------------------------------------------//
// inlines
//----------------------------------------------------------------------//
inline EMassObservedOperationFlags UMassObserverProcessor::GetObservedOperations() const
{
	return ObservedOperations;
}

inline TNotNull<const UScriptStruct*> UMassObserverProcessor::GetSingleObservedTypeChecked() const
{
	checkf(!ObservedTypes.IsEmpty(), TEXT("%hs called but ObservedTypes is empty"), __FUNCTION__);
	ensureMsgf(ObservedTypes.Num() == 1, TEXT("%hs called there are multiple types being observed. Fetching just one doesn't give you the full picture."), __FUNCTION__);
	return ObservedTypes[0];
}

inline const UScriptStruct* UMassObserverProcessor::GetSingleObservedType() const
{
	ensureMsgf(ObservedTypes.Num() <= 1, TEXT("%hs called there are multiple types being observed. Fetching just one doesn't give you the full picture."), __FUNCTION__);
	return ObservedTypes.Num() ? ObservedTypes[0].Get() : static_cast<const UScriptStruct*>(nullptr);
}

inline TNotNull<const UScriptStruct*> UMassObserverProcessor::GetObservedTypeChecked() const
{
	return GetSingleObservedTypeChecked();
}

inline TConstArrayView<const UScriptStruct*> UMassObserverProcessor::GetObservedTypes() const
{
	return ObservedTypes;
}

#undef UE_API
