// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "MassProcessor.h"
#include "UObject/ObjectKey.h"
#include "MassObserverRegistry.generated.h"

#define UE_API MASSENTITY_API


struct FMassObserverManager;

/**
 * A wrapper type for a TArray to support having map-of-arrays UPROPERTY members in FMassEntityObserverClassesMap
 */
USTRUCT(meta = (Deprecated = "5.7", DeprecationMessage = "FMassProcessorClassCollection is no longer being used and will be removed in the upcoming engine release"))
struct FMassProcessorClassCollection
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<TSubclassOf<UMassProcessor>> ClassCollection;
};

/**
 * A wrapper type for a TMap to support having array-of-maps UPROPERTY members in UMassObserverRegistry
 */
USTRUCT(meta = (Deprecated = "5.7", DeprecationMessage = "FMassEntityObserverClassesMap is no longer being used and will be removed in the upcoming engine release"))
struct FMassEntityObserverClassesMap
{
	GENERATED_BODY()

	/** a helper accessor simplifying access while still keeping Container private */
	const TMap<TObjectPtr<const UScriptStruct>, FMassProcessorClassCollection>& operator*() const
	{
		return Container;
	}

	TMap<TObjectPtr<const UScriptStruct>, FMassProcessorClassCollection>& operator*()
	{
		return Container;
	}

private:
	UPROPERTY()
	TMap<TObjectPtr<const UScriptStruct>, FMassProcessorClassCollection> Container;
};

UCLASS(MinimalAPI)
class UMassObserverRegistry : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMassObserverRegistry();

	static UMassObserverRegistry& GetMutable()
	{
		return *GetMutableDefault<UMassObserverRegistry>();
	}
	static const UMassObserverRegistry& Get()
	{
		return *GetDefault<UMassObserverRegistry>();
	}

	UE_API void RegisterObserver(TNotNull<const UScriptStruct*> ObservedType, uint8 OperationFlags, const TSubclassOf<UMassProcessor>& ObserverClass);

	void RegisterObserver(TNotNull<const UScriptStruct*> ObservedType, EMassObservedOperationFlags OperationFlags, const TSubclassOf<UMassProcessor>& ObserverClass)
	{
		RegisterObserver(ObservedType, static_cast<uint8>(OperationFlags), ObserverClass);
	}

	UE_API void RegisterObserver(const UScriptStruct& ObservedType, EMassObservedOperation Operation, const TSubclassOf<UMassProcessor>& ObserverClass);

	using FObserverClassesMap = TMap<TObjectKey<const UScriptStruct>, TArray<FSoftClassPath>>;
protected:

	void OnModulePackagesUnloaded(TConstArrayView<UPackage*> Packages);

	FDelegateHandle ModulesUnloadedHandle;

	friend FMassObserverManager;
	FObserverClassesMap ElementObserverMaps[static_cast<uint8>(EMassObservedOperation::MAX)];

	UE_DEPRECATED(5.7, "Use ElementObserverMaps instead")
	UPROPERTY()
	FMassEntityObserverClassesMap FragmentObservers[static_cast<uint8>(EMassObservedOperation::MAX)];

	UE_DEPRECATED(5.7, "Use ElementObserverMaps instead")
	UPROPERTY()
	FMassEntityObserverClassesMap TagObservers[static_cast<uint8>(EMassObservedOperation::MAX)];
};

#undef UE_API
