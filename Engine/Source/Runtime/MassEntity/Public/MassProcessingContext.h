// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassExecutionContext.h"
#include "MassCommandBuffer.h"
#include "MassEntityManager.h"
#include "StructUtils/InstancedStruct.h"


namespace UE::Mass
{
	struct FProcessingContext
	{
		UE_DEPRECATED_FORGAME(5.6, "This constructor is deprecated. Use one of the other ones.")
		FProcessingContext() = default;
		UE_DEPRECATED_FORGAME(5.6, "This constructor is deprecated. Use one of the other ones.")
		FProcessingContext(const FProcessingContext&) = default;

		FProcessingContext(FProcessingContext&&) = default;

		explicit FProcessingContext(FMassEntityManager& InEntityManager, const float InDeltaSeconds = 0.f, const bool bInFlushCommandBuffer = true);
		explicit FProcessingContext(const TSharedRef<FMassEntityManager>& InEntityManager, const float InDeltaSeconds = 0.f, const bool bInFlushCommandBuffer = true);
		explicit FProcessingContext(TSharedRef<FMassEntityManager>&& InEntityManager, const float InDeltaSeconds = 0.f, const bool bInFlushCommandBuffer = true);
		explicit FProcessingContext(const TSharedPtr<FMassEntityManager>& InEntityManager, const float InDeltaSeconds = 0.f, const bool bInFlushCommandBuffer = true);
		MASSENTITY_API ~FProcessingContext();
		
		[[nodiscard]] FMassExecutionContext& GetExecutionContext() &;
		[[nodiscard]] FMassExecutionContext&& GetExecutionContext() &&;
		bool GetWillFlushCommands() const;
		float GetDeltaSeconds() const;

		void SetCommandBuffer(TSharedPtr<FMassCommandBuffer>&& InCommandBuffer);
		void SetCommandBuffer(const TSharedPtr<FMassCommandBuffer>& InCommandBuffer);

		const TSharedRef<FMassEntityManager>& GetEntityManager() const;

		UE_DEPRECATED_FORGAME(5.6, "Direct access to FProcessingContext.EntityManager has been deprecated. Use GetEntityManager instead.")
		TSharedRef<FMassEntityManager> EntityManager;

		UE_DEPRECATED_FORGAME(5.6, "Direct access to FProcessingContext.DeltaSeconds has been deprecated. Set it via the constructor instead, read via the getter.")
		float DeltaSeconds = 0.f;

		FInstancedStruct AuxData;

		/** 
		 * If set to "true" the MassExecutor will flush commands at the end of given execution function. 
		 * If "false" the caller is responsible for manually flushing the commands.
		 */
		UE_DEPRECATED_FORGAME(5.6, "Direct access to FProcessingContext.bFlushCommandBuffer has been deprecated. Set it via the constructor instead, read via the GetWillFlushCommands getter.")
		bool bFlushCommandBuffer = true; 

		UE_DEPRECATED_FORGAME(5.6, "Direct access to FProcessingContext's CommandBuffer has been deprecated. Use SetCommandBuffer()")
		TSharedPtr<FMassCommandBuffer> CommandBuffer;

	protected:
		alignas(FMassExecutionContext) uint8 ExecutionContextBuffer[sizeof(FMassExecutionContext)];
		FMassExecutionContext* ExecutionContextPtr = nullptr;
	};

	//----------------------------------------------------------------------//
	// INLINES
	//----------------------------------------------------------------------//
	// @todo remove the deprecation disabling once CommandBuffer and EntityManager are moved to `protected` and un-deprecated (around UE5.8)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	inline FProcessingContext::FProcessingContext(FMassEntityManager& InEntityManager, const float InDeltaSeconds, const bool bInFlushCommandBuffer)
		: FProcessingContext(/*MoveTemp*/(InEntityManager.AsShared()), InDeltaSeconds, bInFlushCommandBuffer)
	{

	}

	inline FProcessingContext::FProcessingContext(const TSharedRef<FMassEntityManager>& InEntityManager, const float InDeltaSeconds, const bool bInFlushCommandBuffer)
		: EntityManager(InEntityManager)
		, DeltaSeconds(InDeltaSeconds)
		, bFlushCommandBuffer(bInFlushCommandBuffer)
	{
		
	}

	inline FProcessingContext::FProcessingContext(TSharedRef<FMassEntityManager>&& InEntityManager, const float InDeltaSeconds, const bool bInFlushCommandBuffer)
		: EntityManager(MoveTemp(InEntityManager))
		, DeltaSeconds(InDeltaSeconds)
		, bFlushCommandBuffer(bInFlushCommandBuffer)
	{
		
	}

	inline FProcessingContext::FProcessingContext(const TSharedPtr<FMassEntityManager>& InEntityManager, const float InDeltaSeconds, const bool bInFlushCommandBuffer)
		: FProcessingContext(InEntityManager.ToSharedRef(), InDeltaSeconds, bInFlushCommandBuffer)
	{
		
	}

	inline FMassExecutionContext& FProcessingContext::GetExecutionContext() &
	{
		if (ExecutionContextPtr == nullptr)
		{
			ExecutionContextPtr = new(&ExecutionContextBuffer) FMassExecutionContext(*EntityManager, DeltaSeconds);

			if (CommandBuffer.IsValid() == false)
			{
				CommandBuffer = MakeShareable(new FMassCommandBuffer());
			}

			ExecutionContextPtr->SetDeferredCommandBuffer(CommandBuffer);
			
			ExecutionContextPtr->SetFlushDeferredCommands(false);
			ExecutionContextPtr->SetAuxData(AuxData);
			ExecutionContextPtr->SetExecutionType(EMassExecutionContextType::Processor);
		}
		return *ExecutionContextPtr;
	}

	inline FMassExecutionContext&& FProcessingContext::GetExecutionContext() &&
	{
		// Note: it's fine to store a reference to created execution context
		// while nulling-out the pointer, since the FMassExecutionContext data
		// lives in the ExecutionContextBuffer buffer. Nulling out ExecutionContextPtr
		// only signals that the execution context has been moved out.

		FMassExecutionContext& LocalExecutionContext = GetExecutionContext();
		ExecutionContextPtr = nullptr;
		CommandBuffer.Reset();

		return MoveTemp(LocalExecutionContext);
	}

	inline void FProcessingContext::SetCommandBuffer(TSharedPtr<FMassCommandBuffer>&& InCommandBuffer)
	{
		checkf(ExecutionContextPtr == nullptr, TEXT("Setting command buffer after ExecutionContext creation is not supported"));
		if (ExecutionContextPtr == nullptr)
		{
			CommandBuffer = MoveTemp(InCommandBuffer);
		}
	}

	inline void FProcessingContext::SetCommandBuffer(const TSharedPtr<FMassCommandBuffer>& InCommandBuffer)
	{
		checkf(ExecutionContextPtr == nullptr, TEXT("Setting command buffer after ExecutionContext creation is not supported"));
		if (ExecutionContextPtr == nullptr)
		{
			CommandBuffer = InCommandBuffer;
		}
	}

	inline const TSharedRef<FMassEntityManager>& FProcessingContext::GetEntityManager() const
	{
		return EntityManager;
	}

	inline bool FProcessingContext::GetWillFlushCommands() const
	{
		return bFlushCommandBuffer;
	}

	inline float FProcessingContext::GetDeltaSeconds() const
	{
		return DeltaSeconds;
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
} // namespace UE::Mass
