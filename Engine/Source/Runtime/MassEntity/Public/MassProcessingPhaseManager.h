// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#include "UObject/UObjectGlobals.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "UObject/Object.h"
#include "UObject/GCObject.h"
#include "Containers/MpscQueue.h"
#include "Engine/EngineBaseTypes.h"
#include "MassProcessingTypes.h"
#include "MassProcessor.h"
#include "MassProcessorDependencySolver.h"
#include "MassProcessingQueue.h"
#include "MassProcessingPhaseManager.generated.h"


struct FMassProcessingPhaseManager;
class UMassProcessor;
class UMassCompositeProcessor;
struct FMassEntityManager;
struct FMassCommandBuffer;
struct FMassProcessingPhaseConfig;


USTRUCT()
struct FMassProcessingPhaseConfig
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Mass, config)
	FName PhaseName;

	UPROPERTY(EditAnywhere, Category = Mass, config, NoClear)
	TSubclassOf<UMassCompositeProcessor> PhaseGroupClass = UMassCompositeProcessor::StaticClass();

	UPROPERTY(Transient)
	TArray<TObjectPtr<UMassProcessor>> ProcessorCDOs;

#if WITH_EDITORONLY_DATA
	// this processor is available only in editor since it's used to present the user the order in which processors
	// will be executed when given processing phase gets triggered
	UPROPERTY(Transient)
	TObjectPtr<UMassCompositeProcessor> PhaseProcessor = nullptr;

	UPROPERTY(VisibleAnywhere, Category = Mass, Transient)
	FText Description;
#endif //WITH_EDITORONLY_DATA
};


struct FMassProcessingPhase : public FTickFunction
{
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPhaseEvent, const float /*DeltaSeconds*/);

	MASSENTITY_API FMassProcessingPhase();
	FMassProcessingPhase(const FMassProcessingPhase& Other) = delete;
	FMassProcessingPhase& operator=(const FMassProcessingPhase& Other) = delete;

protected:
	// FTickFunction interface
	MASSENTITY_API virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	MASSENTITY_API virtual FString DiagnosticMessage() override;
	MASSENTITY_API virtual FName DiagnosticContext(bool bDetailed) override;
	// End of FTickFunction interface

	MASSENTITY_API void OnParallelExecutionDone(const float DeltaTime);

	bool IsConfiguredForParallelMode() const
	{
		return bRunInParallelMode;
	}
	void ConfigureForParallelMode()
	{
		bRunInParallelMode = true;
	}
	void ConfigureForSingleThreadMode()
	{
		bRunInParallelMode = false;
	}

	bool ShouldTick(const ELevelTick TickType) const
	{
		return SupportedTickTypes & (1 << TickType);
	}

public:
	MASSENTITY_API void Initialize(FMassProcessingPhaseManager& InPhaseManager, const EMassProcessingPhase InPhase, const ETickingGroup InTickGroup, UMassCompositeProcessor& InPhaseProcessor);
	void AddSupportedTickType(const ELevelTick TickType)
	{
		SupportedTickTypes |= (1 << TickType);
	}
	void RemoveSupportedTickType(const ELevelTick TickType)
	{
		SupportedTickTypes &= ~(1 << TickType);
	}

#if WITH_MASSENTITY_DEBUG
	const UMassCompositeProcessor* DebugGetPhaseProcessor() const;
#endif // WITH_MASSENTITY_DEBUG

protected:
	friend FMassProcessingPhaseManager;

	// composite processor representing work to be performed. GC-referenced via AddReferencedObjects
	TObjectPtr<UMassCompositeProcessor> PhaseProcessor = nullptr;

	UE::Mass::FProcessingQueue ProcessingQueue;

	EMassProcessingPhase Phase = EMassProcessingPhase::MAX;
	FOnPhaseEvent OnPhaseStart;
	FOnPhaseEvent OnPhaseEnd;

private:
	FMassProcessingPhaseManager* PhaseManager = nullptr;
	std::atomic<bool> bIsDuringMassProcessing = false;
	TSharedPtr<FMassObserverManager::FObserverLock> ObserversLockPtr;

	bool bRunInParallelMode = true;
	uint8 SupportedTickTypes = 0;
};


struct FMassPhaseProcessorConfigurationHelper
{
	FMassPhaseProcessorConfigurationHelper(UMassCompositeProcessor& InOutPhaseProcessor, const FMassProcessingPhaseConfig& InPhaseConfig, UObject& InProcessorOuter, EMassProcessingPhase InPhase)
		: PhaseProcessor(InOutPhaseProcessor), PhaseConfig(InPhaseConfig), ProcessorOuter(InProcessorOuter), Phase(InPhase)
	{
	}

	/** 
	 * @param InWorldExecutionFlags - provide EProcessorExecutionFlags::None to let underlying code decide
	 */
	MASSENTITY_API void Configure(TArrayView<UMassProcessor* const> DynamicProcessors, TArray<TWeakObjectPtr<UMassProcessor>>& InOutRemovedDynamicProcessors
		, EProcessorExecutionFlags InWorldExecutionFlags, const TSharedRef<FMassEntityManager>& EntityManager
		, FMassProcessorDependencySolver::FResult& InOutOptionalResult);

	UMassCompositeProcessor& PhaseProcessor;
	const FMassProcessingPhaseConfig& PhaseConfig;
	UObject& ProcessorOuter;
	EMassProcessingPhase Phase;
	bool bInitializeCreatedProcessors = true;
	bool bIsGameRuntime = true;

	//-----------------------------------------------------------------------------
	// DEPRECATED
	//-----------------------------------------------------------------------------
	UE_DEPRECATED(5.6, "This flavor of Configure is deprecated. Please use the one using a TSharedRef<FMassEntityManager> parameter instead")
	MASSENTITY_API void Configure(TArrayView<UMassProcessor* const> DynamicProcessors, EProcessorExecutionFlags InWorldExecutionFlags
		, const TSharedPtr<FMassEntityManager>& EntityManager = TSharedPtr<FMassEntityManager>()
		, FMassProcessorDependencySolver::FResult* OutOptionalResult = nullptr);
};

/** 
 * MassProcessingPhaseManager owns separate FMassProcessingPhase instances for every ETickingGroup. When activated
 * via Start function it registers and enables the FMassProcessingPhase instances which themselves are tick functions 
 * that host UMassCompositeProcessor which they trigger as part of their Tick function. 
 * MassProcessingPhaseManager serves as an interface to said FMassProcessingPhase instances and allows initialization
 * with collections of processors (via Initialize function) as well as registering arbitrary functions to be called 
 * when a particular phase starts or ends (via GetOnPhaseStart and GetOnPhaseEnd functions). 
 */
struct FMassProcessingPhaseManager : public FGCObject, public TSharedFromThis<FMassProcessingPhaseManager>
{
public:
	MASSENTITY_API explicit FMassProcessingPhaseManager(EProcessorExecutionFlags InProcessorExecutionFlags = EProcessorExecutionFlags::None);
	FMassProcessingPhaseManager(const FMassProcessingPhaseManager& Other) = delete;
	FMassProcessingPhaseManager& operator=(const FMassProcessingPhaseManager& Other) = delete;

	const TSharedPtr<FMassEntityManager>& GetEntityManager() const
	{
		return EntityManager;
	}
	FMassEntityManager& GetEntityManagerRef()
	{
		check(EntityManager);
		return *EntityManager.Get();
	}

	/** Retrieves OnPhaseStart multicast delegate's reference for a given Phase */
	FMassProcessingPhase::FOnPhaseEvent& GetOnPhaseStart(const EMassProcessingPhase Phase)
	{
		return ProcessingPhases[static_cast<uint8>(Phase)].OnPhaseStart;
	}
	/** Retrieves OnPhaseEnd multicast delegate's reference for a given Phase */
	FMassProcessingPhase::FOnPhaseEvent& GetOnPhaseEnd(const EMassProcessingPhase Phase)
	{
		return ProcessingPhases[static_cast<uint8>(Phase)].OnPhaseEnd;
	}

	/** 
	 *  Populates hosted FMassProcessingPhase instances with Processors read from MassEntitySettings configuration.
	 *  Calling this function overrides previous configuration of Phases.
	 */
	MASSENTITY_API void Initialize(UObject& InOwner, TConstArrayView<FMassProcessingPhaseConfig> ProcessingPhasesConfig, const FString& DependencyGraphFileName = TEXT(""));

	/** Needs to be called before destruction, ideally before owner's BeginDestroy (a FGCObject's limitation) */
	MASSENTITY_API void Deinitialize();

	MASSENTITY_API const FGraphEventRef& TriggerPhase(const EMassProcessingPhase Phase, const float DeltaTime, const FGraphEventRef& MyCompletionGraphEvent
		, ENamedThreads::Type CurrentThread = ENamedThreads::GameThread);

	/** 
	 *  Stores EntityManager associated with given world's MassEntitySubsystem and kicks off phase ticking.
	 */
	MASSENTITY_API void Start(UWorld& World);
	
	/**
	 *  Stores InEntityManager as the entity manager. It also kicks off phase ticking if the given InEntityManager is tied to a UWorld.
	 */
	MASSENTITY_API void Start(const TSharedRef<FMassEntityManager>& InEntityManager);
	MASSENTITY_API void Stop();
	bool IsRunning() const
	{
		return EntityManager.IsValid();
	}

	/**
	 * Determine if this Phase Manager is currently paused.
	 * 
	 * While paused, phases will transition as usual, but processors
	 * will not be executed.
	 * 
	 * @return True if this PhaseManager is currently paused; else False
	 */
	bool IsPaused() const;

	/**
	 * Pause this phase manager at the earliest opportunity (on next FrameEnd phase end).
	 * This allows the current phase cycle to complete before the pause takes effect.
	 */
	MASSENTITY_API void Pause();

	/**
	 * Unpause this phase manager at the earliest opportunity (on next PrePhysics phase start).
	 */
	MASSENTITY_API void Resume();

	MASSENTITY_API FString GetName() const;

	/** Registers a dynamic processor. This needs to be a fully formed processor and will be slotted in during the next tick. */
	MASSENTITY_API void RegisterDynamicProcessor(UMassProcessor& Processor);
	/** Removes a previously registered dynamic processor of throws an assert if not found. */
	MASSENTITY_API void UnregisterDynamicProcessor(UMassProcessor& Processor);

	struct FPhaseGraphBuildState
	{
		FMassProcessorDependencySolver::FResult LastResult;
		bool bNewArchetypes = true;
		bool bProcessorsNeedRebuild = true;
		bool bInitialized = false;

		void Reset();
	};

	FTickFunction& GetProcessingPhaseTickFunction(EMassProcessingPhase Phase)
	{
		check(Phase < EMassProcessingPhase::MAX);
		check(static_cast<int32>(Phase) >= 0);
		return ProcessingPhases[static_cast<int32>(Phase)];
	}

	const FTickFunction& GetProcessingPhaseTickFunction(EMassProcessingPhase Phase) const
	{
		check(Phase < EMassProcessingPhase::MAX);
		check(static_cast<int32>(Phase) >= 0);
		return ProcessingPhases[static_cast<int32>(Phase)];
	}

#if WITH_MASSENTITY_DEBUG
	TConstArrayView<FMassProcessingPhase>  DebugGetProcessingPhases() const;
	TConstArrayView<FPhaseGraphBuildState> DebugGetProcessingGraphBuildStates() const;
#endif // WITH_MASSENTITY_DEBUG

protected:
	// FGCObject interface
	MASSENTITY_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMassProcessingPhaseManager");
	}
	// End of FGCObject interface

	MASSENTITY_API void RegisterDynamicProcessorInternal(TNotNull<UMassProcessor*> Processor);
	MASSENTITY_API void UnregisterDynamicProcessorInternal(TNotNull<UMassProcessor*> Processor);
	MASSENTITY_API void HandlePendingDynamicProcessorOperations(const int32 PhaseIndex);

	/** Override this function if you want to modify how the phase tick functions get executed. */
	MASSENTITY_API virtual void EnableTickFunctions(const UWorld& World);

	/** Creates phase processors instances for each declared phase name, based on MassEntitySettings */
	MASSENTITY_API void CreatePhases();

	friend FMassProcessingPhase;

	/** 
	 *  Called by the given Phase at the very start of its execution function (the FMassProcessingPhase::ExecuteTick),
	 *  even before the FMassProcessingPhase.OnPhaseStart broadcast delegate
	 */
	MASSENTITY_API void OnPhaseStart(FMassProcessingPhase& Phase);

	/**
	 *  Called by the given Phase at the very end of its execution function (the FMassProcessingPhase::ExecuteTick),
	 *  after the FMassProcessingPhase.OnPhaseEnd broadcast delegate
	 */
	MASSENTITY_API void OnPhaseEnd(FMassProcessingPhase& Phase);

	MASSENTITY_API void OnNewArchetype(const FMassArchetypeHandle& NewArchetype);

protected:
	FMassProcessingPhase ProcessingPhases[static_cast<uint8>(EMassProcessingPhase::MAX)];
	FPhaseGraphBuildState ProcessingGraphBuildStates[static_cast<uint8>(EMassProcessingPhase::MAX)];
	TArray<FMassProcessingPhaseConfig> ProcessingPhasesConfig;
	TArray<TObjectPtr<UMassProcessor>> DynamicProcessors;
	TArray<TWeakObjectPtr<UMassProcessor>> RemovedDynamicProcessors;
	enum class EDynamicProcessorOperationType : uint8
	{
		Add,
		Remove
	};
	/** using TStrongObjectPtr to not worry about GC while the processor instances are waiting in PendingDynamicProcessors */
	using FDynamicProcessorOperation = TPair<TStrongObjectPtr<UMassProcessor>, EDynamicProcessorOperationType>;
	TMpscQueue<FDynamicProcessorOperation> PendingDynamicProcessors[static_cast<uint8>(EMassProcessingPhase::MAX)];

	TSharedPtr<FMassEntityManager> EntityManager;

	EMassProcessingPhase CurrentPhase = EMassProcessingPhase::MAX;

	TWeakObjectPtr<UObject> Owner;

	FDelegateHandle OnNewArchetypeHandle;

	EProcessorExecutionFlags ProcessorExecutionFlags = EProcessorExecutionFlags::None;
	bool bIsAllowedToTick = false;

	bool bIsPaused = false;
	bool bIsPauseTogglePending = false;

#if WITH_MASSENTITY_DEBUG
	FDelegateHandle OnDebugEntityManagerInitializedHandle;
	FDelegateHandle OnDebugEntityManagerDeinitializedHandle;

	MASSENTITY_API void OnDebugEntityManagerInitialized(const FMassEntityManager&);
	MASSENTITY_API void OnDebugEntityManagerDeinitialized(const FMassEntityManager&);
#endif // WITH_MASSENTITY_DEBUG

public:
	UE_DEPRECATED(5.6, "This flavor of Start is deprecated. Please use the one using a TSharedRef<FMassEntityManager> parameter instead")
	MASSENTITY_API void Start(const TSharedPtr<FMassEntityManager>& InEntityManager);
};

//----------------------------------------------------------------------//
// inlines
//----------------------------------------------------------------------//
#if WITH_MASSENTITY_DEBUG
inline const UMassCompositeProcessor* FMassProcessingPhase::DebugGetPhaseProcessor() const
{
	return PhaseProcessor;
}

inline TConstArrayView<FMassProcessingPhase>  FMassProcessingPhaseManager::DebugGetProcessingPhases() const
{
	return MakeArrayView(ProcessingPhases, static_cast<uint8>(EMassProcessingPhase::MAX));
}

inline TConstArrayView<FMassProcessingPhaseManager::FPhaseGraphBuildState> FMassProcessingPhaseManager::DebugGetProcessingGraphBuildStates() const
{
	return MakeArrayView(ProcessingGraphBuildStates, static_cast<uint8>(EMassProcessingPhase::MAX));
}
#endif // WITH_MASSENTITY_DEBUG

inline bool FMassProcessingPhaseManager::IsPaused() const 
{
	return bIsPaused; 
}
