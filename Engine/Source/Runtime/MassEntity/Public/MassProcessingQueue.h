// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "MassProcessingTypes.h"
#include "MassProcessingQueueTypes.h"

class UMassProcessor;
struct FMassExecutionContext;

#define UE_API MASSENTITY_API

#define MASS_CONCURRENCY_STATS (!UE_BUILD_SHIPPING) && MASS_DO_PARALLEL

namespace UE::Mass
{
	struct FProcessingQueue
	{
	public:
		/** Add a processor to the queue and resolve dependencies */
		UE_API void AddProcessor(TNotNull<UMassProcessor*> InProcessor);

		/** Add processors from a pipeline to the queue and resolve dependencies */
		UE_API void AddProcessors(FMassRuntimePipeline& InPipeline);

		/** Add processors to the queue and resolve dependencies */
		UE_API void AddProcessors(TConstArrayView<UMassProcessor*> InProcessors);

		/** Clear the processing queue and re-initialize with the provided array */
		UE_API void SetProcessors(TConstArrayView<UMassProcessor*> InProcessors);

		/**
		 * Immediately execute the processors in the queue.
		 * Blocks the calling thread until completion.
		 */
		UE_API void Execute(FProcessingContext&& ProcessingContext, TFunction<void()> OnDoneNotification, ENamedThreads::Type CurrentThread, TFunction<void()> OnPreCommandFlushNotification);

		/**
		 * Immediately execute the processors in the queue.
		 * Will block the calling threa when bBlockMainThread is true.
		 * 
		 * Returns a graph event the caller can chain on. 
		 */
		FGraphEventRef Execute_GetEventRef(FProcessingContext&& ProcessingContext, TFunction<void()> OnDoneNotification, ENamedThreads::Type CurrentThread, TFunction<void()> OnPreCommandFlushNotification);

		/**  Resolve dependencies and build the contention matrices for the current processor list */
		UE_API void InitQueue();

		/** Should Execute(...) await the completion of all processors before returning */
		bool bBlockMainThread = false;

	private:
		friend FProcessorWorker;
		friend FChunkWorker;

		void RunMainThread(FMassExecutionContext& ExecutionContext);
		void Run(FMassExecutionContext& ExecutionContext);

		/**
		 * Post-runner cleanup shared by blocking and non-blocking paths: spin on the entity manager's
		 * processing flag, merge worker command buffers into the execution context, fire callbacks,
		 * and flush deferred commands. Callers must ensure any FScopedProcessing they own is destroyed
		 * before invoking this (the IsProcessing spin assumes the only outstanding scope was their own).
		 */
		void FinalizeAsyncCleanup(FMassExecutionContext& ExecutionContext,
			const TFunction<void()>& OnDoneNotification,
			const TFunction<void()>& OnPreCommandFlushNotification);

		enum class EThreadFlag : uint8
		{
			MainThread = 1 << 0,
			OffMainThread = 1 << 1,

			Any = MainThread | OffMainThread
		};

		/** Gets the next processor that is ready to execute, sets its state to running and updates contention tracking and other bookkeeping */
		FQueuedProcessor* GetNextReadyProcessor(TArray<int32>& InOutRemainingIndices);

		bool IsComplete() const;
		std::atomic<int32> PendingMainThreadProcessorCount;
		std::atomic<int32> PendingProcessorCount;
		std::atomic<int32> CompletedProcessorCount;
#if MASS_CONCURRENCY_STATS
		std::atomic<int32> ActiveProcessorCount{0};
		std::atomic<int32> ExecutionConcurrencyHWM{0};
		std::atomic<int32> ActiveChunkCount{0};
		std::atomic<int32> ExecutionChunkHWM{0};
#endif

		/** Removes RequestorIndex from ActiveItems */
		void Release(int32 RequestorIndex);

		/** Checks if bits in Contention collide with bits in ActiveItems, if not, adds RequestorIndex to ActiveItems */
		bool TryReserve(int32 RequestorIndex);
		/** TryReserve then if that fails, wait until contending items are released and try again */
		void Reserve(int32 RequestorIndex);

		/** Release a reservation and immediately attempt TryReserve on a different index */
		bool ReleaseAndTryReserve(int32 IndexToRelease, int32 IndexToReserve);
		/** ReleaseAndReserve, then if that fails immediately Reserve (which will wait) */
		void ReleaseAndReserve(int32 IndexToRelease, int32 IndexToReserve);

		bool TryActivateChunk(FQueuedChunk& InChunk, int32 DirectChunkContentionIndex);
		void DeactivateChunk(FQueuedChunk& InChunk);

		/** Report that a processor is finished, set its internal state and update queue bookkeeping */
		void ProcessorFinished(FQueuedProcessor& CompletedProcessor);

		/**
		 * Make a processor worker and immediately execute the assigned processor
		 * If bInline is true then the processor is executed immediately on this thread and this method won't return until it's complete
		 * Otherwise, a task is created to run the processor worker and execute the processor asynchronously.
		 */
		void MakeProcessorWorker(FMassExecutionContext& ExecutionContext, FQueuedProcessor* AssignedProcessor = nullptr, bool bInLine = false);
		void MakeChunkWorker(FQueuedChunk* AssignedChunk = nullptr, bool bInline = false);
		TSharedPtr<FMassCommandBuffer> GetOrCreateCommandBuffer();
		void ReturnCommandBufferToPool(TSharedPtr<FMassCommandBuffer>& InCommandBuffer);
		void InitializeExplicitDependencies();
		bool CheckForValidDependencies();

		/** Recursively computes dependency depth */
		int32 ComputeGraphDepthRecursive(int32 ProcessorIndex, TArray<int32>& Memo);

		/** Tracks all global contention */
		FContentionMatrix ContentionMatrix;

		/** Tracks active global contention */
		FActiveContentionTracker ActiveContentionTracker{ ContentionMatrix };
		
		FContentionMatrix ChunkContentionMatrix;
		/** Only tracks possible contention of direct chunk access (i.e. contention that can only occur accessing the same chunk) */
		FActiveChunkTracker ActiveChunkContentionTracker;

		TArray<FQueuedProcessor> ProcessorQueue;

		/** array of indices in ProcessorQueue that still need to execute on the main thread */
		TArray<int32> RemainingMainThreadProcessorIndices;

		/** array of indices in ProcessorQueue that still need to execute */
		TArray<int32> RemainingProcessorIndices;

		bool bProcessorListDirty = false;
		FMassRuntimePipeline Processors;
		int32 MainThreadProcessorCount = 0;

		FRWLock CommandBuffersLock;
		std::atomic<uint32> QueueChangeTracker{ 0 };
		TArray<TSharedPtr<FMassCommandBuffer>> CommandBufferPool;
	};

} // namespace UE::Mass



#undef UE_API
