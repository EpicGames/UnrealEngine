// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassExecutionContext.h"
#include "Async/Fundamental/Oversubscription.h"
#include "Tasks/Task.h"

#define UE_API MASSENTITY_API

// Default cap on concurrently-active Mass chunks tracked by FActiveChunkTracker.
// Can be overridden via *.Build.cs
#ifndef UE_MASS_MAX_ACTIVE_CHUNKS
#define UE_MASS_MAX_ACTIVE_CHUNKS 512
#endif

class UMassProcessor;

namespace UE::Mass
{
	struct FProcessingQueue;
	struct FDataAccess;
	struct FQueuedProcessor;
	struct FQueuedChunk;
	struct FQueryInfo;
	class FChunkWorker;
	class FProcessorWorker;

	/**
	 * Dynamically sized bit-matrix to track contention between items.
	 */
	struct FContentionMatrix
	{
	public:
		using WordType = uint64;
		static constexpr int32 BitsPerWord = 64;
		static constexpr int32 BitsPerWordLog2 = 6;

		FContentionMatrix() = default;

		explicit FContentionMatrix(int32 InNumElements)
		{
			Init(InNumElements);
		}

		/**
		 * Resizes the matrix to hold InNumElements.
		 * This clears all existing data.
		 */
		void Init(int32 InNumElements)
		{
			check(InNumElements >= 0);
			NumElements = InNumElements;

			// Calculate how many 64-bit words we need per row
			WordsPerRow = (NumElements + (BitsPerWord - 1)) / BitsPerWord;

			// Total size = Rows * Width
			const int32 TotalWords = NumElements * WordsPerRow;

			MatrixBits.Reset();
			MatrixBits.SetNumZeroed(TotalWords);
		}

		FORCEINLINE int32 GetSize() const
		{
			return NumElements;
		}

		FORCEINLINE int32 GetWordsPerRow() const
		{
			return WordsPerRow;
		}

		/** Zero the entire matrix. */
		FORCEINLINE void Clear()
		{
			FMemory::Memzero(MatrixBits.GetData(), MatrixBits.Num() * sizeof(WordType));
		}

		/** Set contends(i,j) = true */
		void Set(int32 RowIndex, int32 ColumnIndex)
		{
			check(IsValidIndex(RowIndex) && IsValidIndex(ColumnIndex));
			check(RowIndex != ColumnIndex);
			const int32 WordIndex = (ColumnIndex >> BitsPerWordLog2);
			const int32 BitOffset = (ColumnIndex & (BitsPerWord - 1));

			// Calculate flattened index: Row * Stride + ColWord
			const int32 FlatIndex = (RowIndex * WordsPerRow) + WordIndex;

			MatrixBits[FlatIndex] |= (WordType(1) << BitOffset);
		}

		/** Set contends(i,j) = true and contends(j,i) = true*/
		void SetMutual(int32 RowIndex, int32 ColumnIndex)
		{
			Set(RowIndex, ColumnIndex);
			Set(ColumnIndex, RowIndex);
		}

		/** Test contention(i,j) */
		FORCEINLINE bool Test(int32 RowIndex, int32 ColumnIndex) const
		{
			check(IsValidIndex(RowIndex) && IsValidIndex(ColumnIndex));

			const int32 WordIndex = (ColumnIndex >> BitsPerWordLog2);
			const int32 BitOffset = (ColumnIndex & (BitsPerWord-1));
			const int32 FlatIndex = (RowIndex * WordsPerRow) + WordIndex;

			return ((MatrixBits[FlatIndex] >> BitOffset) & 1u) != 0;
		}

		/** raw pointer to the start of the row's words. */
		FORCEINLINE const WordType* GetRowWordPointer(int32 RowIndex) const
		{
			check(IsValidIndex(RowIndex));
			return &MatrixBits[RowIndex * WordsPerRow];
		}

	private:
		int32 NumElements = 0;
		int32 WordsPerRow = 0;

		TArray<WordType> MatrixBits;

		FORCEINLINE bool IsValidIndex(int32 Index) const
		{
			return Index >= 0 && Index < NumElements;
		}
	};

	/**
	 * A single chunk processing callback generated by FMassEntityQuery::ForEachEntityChunk.
	 */
	struct FQueuedChunk
	{
		enum class EState : uint8
		{
			Pending,
			Running,
			Complete,
			MAX
		};

		FQueuedChunk() = default;
		FQueuedChunk(FProcessorWorker* InOwningProcessorWorker, FMassChunkProcessingQueueParams&& InParams);
		FQueuedChunk(FQueuedChunk&& Other) noexcept;

		FQueuedChunk(const FQueuedChunk&) = delete;
		FQueuedChunk& operator=(const FQueuedChunk&) = delete;
		FQueuedChunk& operator=(FQueuedChunk&& Other) = delete;

		uint64 GetChunkID() const
		{
			return (reinterpret_cast<uint64>(Params.Archetype) << 16) ^ static_cast<uint64>(Params.EntityRange.ChunkIndex);
		}

		bool IsSameArchetypeChunk(const FQueuedChunk& Other) const
		{
			return GetChunkID() == Other.GetChunkID();
		}

		EState State = EState::Pending;
		int32 ActiveChunkSlotIndex = INDEX_NONE;

		/** Count of how many "ProcessAfter" chunks remain (includes "ProcessBefore" from other chunks/processors) */
		int32 RemainingDependencyCount = 0;
		FProcessorWorker* OwningProcessorWorker = nullptr;
		const FQueryInfo* QueryInfo = nullptr;
		FMassChunkProcessingQueueParams Params;
	};

	/** Tracks which FQueuedChunks are active to manage contention */
	class FActiveChunkTracker
	{
	public:
		using WordType = uint64;
		static constexpr int32 MaxActiveChunks = UE_MASS_MAX_ACTIVE_CHUNKS;
		static constexpr uint64 InvalidChunkID = 0;
		FActiveChunkTracker() = default;

		void Init(const FContentionMatrix& InMatrix)
		{
			MatrixRef = &InMatrix;
			WordsPerRow = InMatrix.GetWordsPerRow();

			// Initialize IDs to 0
			for (int32 i = 0; i < MaxActiveChunks; ++i)
			{
				ChunkIDs[i].store(InvalidChunkID, std::memory_order_relaxed);
				SlotRefCounts[i] = 0;
				SlotLocks[i].clear();
			}

			SlotAssignmentLock.clear();

			// Resize bitmask buffer: MaxActiveChunks slots * WordsPerRow
			BitmaskBuffer.Reset();
			BitmaskBuffer.SetNumZeroed(MaxActiveChunks * WordsPerRow);
		}

		/**
		 * Checks active contention and reserves the chunk's contention index if safe
		 */
		bool TryAcquireAccess(FQueuedChunk& Chunk, int32 DirectChunkContentionIndex)
		{
			const uint64 ChunkID = Chunk.GetChunkID();
			checkSlow(ChunkID != InvalidChunkID);
			checkSlow(DirectChunkContentionIndex >= 0);
			checkSlow(Chunk.ActiveChunkSlotIndex == -1);

			// Fast scan (relaxed)
			int32 FoundIndex = -1;
			for (int32 i = 0; i < MaxActiveChunks; ++i)
			{
				if (ChunkIDs[i].load(std::memory_order_relaxed) == ChunkID)
				{
					FoundIndex = i;
					break;
				}
			}

			if (FoundIndex != -1)
			{
				// Existing slot
				LockSlot(FoundIndex);

				// Validate slot ownership
				if (ChunkIDs[FoundIndex].load(std::memory_order_acquire) != ChunkID)
				{
					UnlockSlot(FoundIndex);
					// Retry from scratch (extremely unlikely)
					return TryAcquireAccess(Chunk, DirectChunkContentionIndex);
				}

				WordType* SlotBits = GetSlotBits(FoundIndex);
				const WordType* JobRow = MatrixRef->GetRowWordPointer(DirectChunkContentionIndex);

				// Contention check
				for (int32 w = 0; w < WordsPerRow; ++w)
				{
					if ((SlotBits[w] & JobRow[w]) != 0)
					{
						UnlockSlot(FoundIndex);
						return false;
					}
				}

				// Mark active
				AddJobToBitmask(SlotBits, DirectChunkContentionIndex);
				++SlotRefCounts[FoundIndex];

				UnlockSlot(FoundIndex);

				Chunk.ActiveChunkSlotIndex = FoundIndex;
				return true;
			}

			// assign slot for new ChunkID
			while (SlotAssignmentLock.test_and_set(std::memory_order_acquire))
			{
				FPlatformProcess::YieldCycles(10);
			}

			// Double-check under AllocationLock (someone may have added it)
			for (int32 i = 0; i < MaxActiveChunks; ++i)
			{
				if (ChunkIDs[i].load(std::memory_order_relaxed) == ChunkID)
				{
					SlotAssignmentLock.clear(std::memory_order_release);
					// Re-run the found path (will lock+validate)
					return TryAcquireAccess(Chunk, DirectChunkContentionIndex);
				}
			}

			int32 AssignedSlotIndex = -1;
			// find an empty slot
			for (int32 i = 0; i < MaxActiveChunks; ++i)
			{
				if (ChunkIDs[i].load(std::memory_order_acquire) == InvalidChunkID)
				{
					AssignedSlotIndex = i;
					break;
				}
			}

			if (AssignedSlotIndex == -1)
			{
				SlotAssignmentLock.clear(std::memory_order_release);
				// Soft overflow: every slot is in use. Caller (FProcessingQueue::TryActivateChunk)
				// already handles `false` as "couldn't acquire right now"; the chunk is left in
				// the queue and retried later. If hit regularly, raise MaxActiveChunks via
				// Target.MassMaxActiveChunks in the .Target.cs.
				OnSlotExhausted();
				return false;
			}

			// Initialize under slot lock BEFORE setting ChunkID
			LockSlot(AssignedSlotIndex);

#if DO_CHECK
			check(ChunkIDs[AssignedSlotIndex].load(std::memory_order_relaxed) == InvalidChunkID);
#endif

			WordType* SlotBits = GetSlotBits(AssignedSlotIndex);
			FMemory::Memzero(SlotBits, WordsPerRow * sizeof(WordType));

#if DO_CHECK
			const int32 WordIdx = (DirectChunkContentionIndex >> FContentionMatrix::BitsPerWordLog2);
			check(WordIdx >= 0 && WordIdx < WordsPerRow);
#endif

			AddJobToBitmask(SlotBits, DirectChunkContentionIndex);
			SlotRefCounts[AssignedSlotIndex] = 1;

			// Publish LAST
			ChunkIDs[AssignedSlotIndex].store(ChunkID, std::memory_order_release);

			UnlockSlot(AssignedSlotIndex);
			SlotAssignmentLock.clear(std::memory_order_release);

			Chunk.ActiveChunkSlotIndex = AssignedSlotIndex;
			return true;
		}

		void ReleaseAccess(FQueuedChunk& Chunk, int32 DirectChunkContentionIndex)
		{
			const int32 SlotIndex = static_cast<int32>(Chunk.ActiveChunkSlotIndex);
			checkSlow(SlotIndex >= 0 && SlotIndex < MaxActiveChunks);

			const uint64 ChunkID = Chunk.GetChunkID();
			checkSlow(ChunkID != InvalidChunkID);
			checkSlow(DirectChunkContentionIndex >= 0);

			LockSlot(SlotIndex);

			// Validate slot still belongs to this chunk
			check(ChunkIDs[SlotIndex].load(std::memory_order_acquire) == ChunkID);

			WordType* SlotBits = GetSlotBits(SlotIndex);

			const int32 WordIdx = (DirectChunkContentionIndex >> FContentionMatrix::BitsPerWordLog2);

#if DO_CHECK
			check(WordIdx >= 0 && WordIdx < WordsPerRow);
#endif
			const int32 BitOffset = (DirectChunkContentionIndex & (FContentionMatrix::BitsPerWord - 1));
			SlotBits[WordIdx] &= ~(WordType(1) << BitOffset);

			--SlotRefCounts[SlotIndex];
#if DO_CHECK
			check(SlotRefCounts[SlotIndex] >= 0);
#endif

			if (SlotRefCounts[SlotIndex] == 0)
			{
				// release so prior writes are visible before anyone observes Invalid and reuses slot
				ChunkIDs[SlotIndex].store(InvalidChunkID, std::memory_order_release);
			}

			UnlockSlot(SlotIndex);

			Chunk.ActiveChunkSlotIndex = INDEX_NONE;
		}


	private:
		/** Logs a once-per-process warning when the tracker runs out of slots. Defined out-of-line to keep logging deps out of this header. */
		UE_API void OnSlotExhausted();

		const FContentionMatrix* MatrixRef = nullptr;
		int32 WordsPerRow = 0;

		alignas(64) std::atomic<uint64> ChunkIDs[MaxActiveChunks];
		std::atomic_flag SlotLocks[MaxActiveChunks];
		int32 SlotRefCounts[MaxActiveChunks];

		std::atomic_flag SlotAssignmentLock;
		TArray<WordType> BitmaskBuffer;

		FORCEINLINE WordType* GetSlotBits(int32 SlotIndex)
		{
			return &BitmaskBuffer[SlotIndex * WordsPerRow];
		}

		FORCEINLINE void LockSlot(int32 SlotIndex)
		{
			while (SlotLocks[SlotIndex].test_and_set(std::memory_order_acquire))
			{
				FPlatformProcess::YieldCycles(10);
			}
		}

		FORCEINLINE void UnlockSlot(int32 SlotIndex)
		{
			SlotLocks[SlotIndex].clear(std::memory_order_release);
		}

		FORCEINLINE void AddJobToBitmask(WordType* SlotBits, int32 JobIndex)
		{
			const int32 WordIdx = (JobIndex >> FContentionMatrix::BitsPerWordLog2);
			const int32 BitOffset = (JobIndex & (FContentionMatrix::BitsPerWord - 1));
			SlotBits[WordIdx] |= (WordType(1) << BitOffset);
		}
	};

	/**
	 * Uses a pre-calculated FContentionMatrix to help dynamically track active contending items
	 */
	struct FActiveContentionTracker
	{
	public:
		using WordType = uint64;
		static constexpr int32 BitsPerWord = 64;
		static constexpr int32 BitsPerWordLog2 = 6;

		FActiveContentionTracker(const FContentionMatrix& InMatrix)
			: Matrix(InMatrix)
		{
			Init();
		}

		void Init()
		{
			const int32 Size = Matrix.GetSize();
			ActiveContenderCount.SetNumZeroed(Size);
		}

		bool TryActivate(int32 ItemIndex)
		{
			check(IsValidIndex(ItemIndex));

			if (HasAnyActiveContention(ItemIndex))
			{
				return false;
			}

			// assume success and update contention counts first
			UpdateContentionCounts(ItemIndex, 1);

			// see if a contender activated while this thread was in UpdateContentionCounts
			if (HasAnyActiveContention(ItemIndex))
			{
				FScopeLock Lock(&ActiveContenderCriticalSection);
				// check one more time inside the critical section to eliminate all possible races
				if (HasAnyActiveContention(ItemIndex))
				{
					// undo the change to the contention counts
					UpdateContentionCounts(ItemIndex, -1);
					return false;
				}
			}

			return true;
		}

		FORCEINLINE bool Deactivate(int32 ItemIndex)
		{
			check(IsValidIndex(ItemIndex));
			UpdateContentionCounts(ItemIndex, -1);
			return true;
		}

		FORCEINLINE bool HasAnyActiveContention(int32 ItemIndex)
		{
			check(IsValidIndex(ItemIndex));

			// not all platforms support atomic std::atomic_ref, but it's typically faster so do it when we can
#if defined(__cpp_lib_atomic_ref)
			std::atomic_ref<int32> AtomicCount(ActiveContenderCount[ItemIndex]);
			return AtomicCount.load(std::memory_order_acquire) > 0;
#else
			return FPlatformAtomics::AtomicRead(reinterpret_cast<volatile int32*>(&ActiveContenderCount[ItemIndex])) > 0;
#endif
		}

		FORCEINLINE int32 GetActiveContentionCount(int32 ItemIndex)
		{
			check(IsValidIndex(ItemIndex));
			// not all platforms support atomic std::atomic_ref, but it's typically faster so do it when we can
#if defined(__cpp_lib_atomic_ref)
			std::atomic_ref<int32> AtomicCount(ActiveContenderCount[ItemIndex]);
			return AtomicCount.load(std::memory_order_acquire);
#else
			return FPlatformAtomics::AtomicRead(reinterpret_cast<volatile int32*>(&ActiveContenderCount[ItemIndex]));
#endif
		}

	private:
		const FContentionMatrix& Matrix;
		TArray<int32> ActiveContenderCount;

		FCriticalSection ActiveContenderCriticalSection;

		FORCEINLINE bool IsValidIndex(int32 Index) const
		{
			return Index >= 0 && Index < Matrix.GetSize();
		}

		void UpdateContentionCounts(int32 ItemIndex, int32 Delta)
		{
			const WordType* RESTRICT RowWords = Matrix.GetRowWordPointer(ItemIndex);
			const int32 MatrixSize = Matrix.GetSize();
			const int32 WordsPerRow = Matrix.GetWordsPerRow();
			const int32 DiagWordIndex = (ItemIndex >> BitsPerWordLog2);
			const int32 DiagBitOffset = (ItemIndex & (BitsPerWord-1));

			int32* RawCounts = ActiveContenderCount.GetData();

			for (int32 RowWordIndex = 0, BaseBitIndex = 0; RowWordIndex < WordsPerRow; ++RowWordIndex, BaseBitIndex += BitsPerWord)
			{
				WordType ContenderWord = (RowWordIndex != DiagWordIndex)
					? RowWords[RowWordIndex]
					: (RowWords[RowWordIndex] & ~(WordType(1) << DiagBitOffset));

				while (ContenderWord != 0)
				{
					const int32 TrailingZeroCount = static_cast<int32>(FMath::CountTrailingZeros64(ContenderWord));
					const int32 NeighborIndex = BaseBitIndex + TrailingZeroCount;

					if (NeighborIndex < MatrixSize)
					{
						// not all platforms support atomic std::atomic_ref, but it's typically faster so do it when we can
#if defined(__cpp_lib_atomic_ref)
						std::atomic_ref<int32> NeighborCount(RawCounts[NeighborIndex]);
						NeighborCount.fetch_add(Delta, std::memory_order_release);
#else
						FPlatformAtomics::InterlockedAdd(reinterpret_cast<volatile int32*>(&RawCounts[NeighborIndex]), Delta);
#endif
					}

					ContenderWord &= (ContenderWord - 1);
				}
			}
		}
	};

	/** Tracks which chunks are actively being processed with a refcount */
	struct FActiveChunkSet
	{
	private:
		struct FChunkSlot
		{
			uint64 Index;

		};
	public:
		static constexpr int32 Capacity = 32;
		static constexpr uint64 EmptyValue = 0;
		const int32 MaxSlots;

		FActiveChunkSet(int32 InMaxSlots)
			: MaxSlots(InMaxSlots)
		{
			checkf(MaxSlots <= Capacity, TEXT("FActiveChunkSet::MaxSlots (%d) exceeded Capacity (%d)."), MaxSlots, Capacity);

			for (std::atomic<uint64>& Slot : Slots)
			{
				Slot.store(EmptyValue, std::memory_order_relaxed);
			}
			for (std::atomic<int32>& RefCount : RefCounts)
			{
				RefCount.store(0, std::memory_order_relaxed);
			}
		}

		/**
		 * Adds an ID to the set.
		 * Returns true if added, false if it was already present.
		 */
		bool TryAddExclusive(uint64 ID)
		{
			check(ID != EmptyValue);

			int32 CandidateIndex = INDEX_NONE;

			for (;;)
			{
				for (int32 Index = 0; Index < MaxSlots; ++Index)
				{
					const uint64 Val = Slots[Index].load(std::memory_order_seq_cst);

					if (Val == ID)
					{
						return false; // Already exists
					}

					if (CandidateIndex == INDEX_NONE && Val == EmptyValue)
					{
						CandidateIndex = Index;
						break;
					}
				}

				if (CandidateIndex != INDEX_NONE)
				{
					uint64 Expected = EmptyValue;
					if (Slots[CandidateIndex].compare_exchange_strong(Expected, ID, std::memory_order_seq_cst))
					{
						// this is an exclusive add so we can just force refcount to 1 for the previously empty slot
						RefCounts[CandidateIndex].store(1, std::memory_order_seq_cst);
						return true;
					}

					// someone else took this slot or added an item, loop and try again
				}
				else
				{
					checkf(false, TEXT("FActiveChunkSet overflow! Active chunks exceeded Capacity (%d)."), Capacity);
					return false;
				}
			}
		}

		/**
		 * Adds the ID to the set if not present
		 * Increments the refcount for the ID
		 * Returns the new refcount
		 */
		int32 AddGetRefCount(uint64 ID)
		{
			check(ID != EmptyValue);

			int32 CandidateIndex = INDEX_NONE;

			for (;;)
			{
				for (int32 Index = 0; Index < MaxSlots; ++Index)
				{
					const uint64 Val = Slots[Index].load(std::memory_order_seq_cst);

					if (Val == ID)
					{
						int32 NewCount = RefCounts[Index].fetch_add(1, std::memory_order_seq_cst) + 1;
						return NewCount;
					}

					if (CandidateIndex == INDEX_NONE && Val == EmptyValue)
					{
						CandidateIndex = Index;
						break;
					}
				}

				if (CandidateIndex != INDEX_NONE)
				{
					uint64 Expected = EmptyValue;
					if (Slots[CandidateIndex].compare_exchange_strong(Expected, ID, std::memory_order_seq_cst))
					{
						// this is an exclusive add so we can just force refcount to 1 for the previously empty slot
						RefCounts[CandidateIndex].store(1, std::memory_order_seq_cst);
						return 1;
					}

					// someone else took this slot or added an item, loop and try again
				}
				else
				{
					checkf(false, TEXT("FActiveChunkSet overflow! Active chunks exceeded MaxSlots (%d)."), MaxSlots);
					return 0;
				}
			}
		}

		/**
		 * Decrement the refcount for ID and remove it if the count is 0
		 */
		void Remove(uint64 ID)
		{
			checkSlow(ID != EmptyValue);

			for (int32 Index = 0; Index < MaxSlots; ++Index)
			{
				std::atomic<uint64>& Slot = Slots[Index];

				if (Slot.load(std::memory_order_seq_cst) == ID)
				{
					int32 NewCount = RefCounts[Index].fetch_add(-1, std::memory_order_seq_cst) - 1;

					if (NewCount == 0)
					{
						Slot.store(EmptyValue, std::memory_order_seq_cst);
						// double check that the refcount is still 0:
						NewCount = RefCounts[Index].load(std::memory_order_seq_cst);
						if (NewCount > 0)
						{
							// something else added a refcount between our operations
							// restore the ID and move on
							Slot.store(ID, std::memory_order_seq_cst);
						}
						return;
					}
				}
			}
		}

		/**
		 * Checks if the ID exists in the set.
		 */
		bool Contains(uint64 ID) const
		{
			checkSlow(ID != EmptyValue);
			for (int32 i = 0; i < MaxSlots; ++i)
			{
				if (Slots[i].load(std::memory_order_acquire) == ID)
				{
					return true;
				}
			}
			return false;
		}

	private:
		// alignas(64) to avoid false sharing
		alignas(64) std::atomic<uint64> Slots[Capacity];
		alignas(64) std::atomic<int32> RefCounts[Capacity];
	};

	/**
	 * Increments a change-tracking atomic variable when leaving a scope.
	 * Used with FChangeListener for efficient synchronization of workers waiting for resources.
	 */
	template <typename T>
	struct FChangeScope
	{
		explicit FChangeScope(std::atomic<T>& InTrackingValue)
			: Tracker(InTrackingValue)
		{
		}
		FChangeScope(const FChangeScope&) = delete;
		FChangeScope& operator=(const FChangeScope&) = delete;
		FChangeScope(FChangeScope&& Other) = delete;

		~FChangeScope()
		{
			++Tracker;
		}
	private:
		std::atomic<T>& Tracker;
	};

	/**
	 * Busy-waits for an atomic variable (incremented by FChangeScope) to change.
	 * The spin marks itself oversubscribed so the scheduler can bring up a standby worker to make
	 * progress on the queue - it mitigates, but does not eliminate, the busy-wait (and can't help once
	 * the worker pool is already saturated).
	 */
	template <typename T>
	struct FChangeListener
	{
		explicit FChangeListener(std::atomic<T>& InValueToTrack)
			: ValueToTrack(InValueToTrack)
			// Set the initial value at construction time to capture changes that happen between declaration and subsequent Wait calls.
			, InitialValue(ValueToTrack.load(std::memory_order_acquire))
		{
		}

		void WaitForChange()
		{
			// This spin can run inside a UE::Tasks worker (the queue runner, a processor worker, or
			// ProcessChunks). Mark the worker as stalled so the scheduler can wake or spawn a standby worker
			// to run the other queued Mass tasks that will advance the queue and release us. No-op when not
			// on a task worker (e.g. the game thread).
			LowLevelTasks::FOversubscriptionScope OversubscriptionScope;
			for (;;)
			{
				const T CurrentValue = ValueToTrack.load(std::memory_order_acquire);
				if (CurrentValue != InitialValue)
				{
					// caller might wait for changes again
					InitialValue = CurrentValue;
					return;
				}
				FPlatformProcess::YieldThread();
			}
		}
	private:
		std::atomic<T>& ValueToTrack;
		T InitialValue;
	};

	/** Single struct to specify data access for a query */
	struct FDataAccess
	{
		FElementBitSet DirectReadsElements;
		FElementBitSet DirectWritesElements;
		FElementBitSet IndirectReadsElements;
		FElementBitSet IndirectWritesElements;
		FMassExternalSubsystemBitSet ReadsSubsystems;
		FMassExternalSubsystemBitSet WritesSubsystems;

		void Add(const FDataAccess& Other);
		bool SubsystemsCollide(const FDataAccess& Other) const;
		bool CollidesWithIndirect(const FDataAccess& Other) const;
		bool CollidesWithDirect(const FDataAccess& Other) const;
		void SetFragmentWriteRequirements(const FMassFragmentRequirements& FragmentRequirements);
	};

	/** Stores processing queue-specific data about a FMassEntityQuery */
	struct FQueryInfo
	{
		bool bChunksRequireGameThread = false;

		/** Contention index to run the query (gather chunks but not execute) */
		int32 QueryContentionIndex = INDEX_NONE;

		/** Contention index for cross-chunk contention on each chunk processed */
		int32 ChunkGlobalContentionIndex = INDEX_NONE;

		/** Contention index for direct chunk contention on each chunk processed */
		int32 ChunkContentionIndex = INDEX_NONE;

		FDataAccess DataAccess;
		const FMassEntityQuery* SourceQuery = nullptr;
	};

	/** A single processor managed by the processing queue. */
	struct FQueuedProcessor
	{
		enum class EState : uint8
		{
			/** Initialized, and waiting for explicit ordering dependencies */
			WaitingForDependencies,

			/** Explicit dependencies met, ready to activate (will activate when at the front of the queue and no contending items are active) */
			Ready,

			/** Currently running */
			Running,

			Completed,
			MAX
		};
		/** Current state in the execution pipeline */
		EState State = EState::MAX;

		TNotNull<UMassProcessor*> Processor;

		/**
		 * If the processor or any query requires game thread execution,
		 * then aggregate that requirement to all queries and the processor itself.
		 * Useful for emulating the legacy processor behavior.
		 */
		bool bAggregateThreadRequirements = true;

		/**
		 * Aggregate all data access requirements of the processor and queries to all queries and the processor itself
		 * Useful for emulating the legacy processor behavior.
		 * Set from bAggregateRequirements on the input processor.
		 */
		bool bAggregateRequirements;

		/** Contention table index for this processor (does not include query requirements) */
		int32 ContentionIndex = INDEX_NONE;

		/** Queries will follow te owning processor in the contention table, count is used to get the correct index */
		int32 QueryCount = 1;

		/** Explicit dependencies remaining (ExecuteBefore/ExecuteAfter) */
		std::atomic<int32> RemainingDependencyCount{ 0 };

		/** Total explicit dependency count (ExecuteBefore/ExecuteAfter) */
		int32 DependencyCount = 0;

		/** Indices of items that depend explicitly on this processor (within the FProcessingQueue's processor array) */
		TArray<int32, TInlineAllocator<8>> DependentProcessorIndices;

		/**
		 * The length of the longest dependency chain starting from this processor.
		 * 0 = Leaf node (nothing depends on this).
		*/
		int32 GraphDepth = 0;

		FDataAccess DataAccess;

		TArray<FQueryInfo> QueryInfos;
		int32 CurrentQueryIndex = INDEX_NONE;

		bool bRequiresGameThread = true;

		void InitThreadRequirement();
		void InitWriteContention(FContentionMatrix& ContentionMatrix, FQueuedProcessor& Other);
		void InitWriteContention(FContentionMatrix& ContentionMatrix, int32 MyQueryIndex, FQueuedProcessor& Other);
		void InitCrossChunkWriteContention(FContentionMatrix& ContentionMatrix, int32 MyQueryIndex, FQueuedProcessor& Other, int32 OtherQueryIndex);
		void InitChunkWriteContention(FContentionMatrix& ContentionMatrix, int32 MyQueryIndex, FQueuedProcessor& Other, int32 OtherQueryIndex);

		explicit FQueuedProcessor(TNotNull<UMassProcessor*> InProcessor);

		void PrepareForExecute();

		const FQueryInfo& GetCurrentQueryInfo() const;
		void SetCurrentQuery(const FMassEntityQuery* SourceQuery);

		/** Multithreading access detector to ensure ChunkQueue is not altered concurrently */
		UE_MT_DECLARE_RW_ACCESS_DETECTOR(ChunkQueueAccessDetector);

		/** Chunks that could not be executed immediately during the query and were queued. */
		TArray<FQueuedChunk> ChunkQueue;

		/** List of indices into ChunkQueue that have not been processed yet. */
		TArray<int32> PendingChunkIndices;

		/** Async chunk tasks dispatched during the current query */
		TArray<UE::Tasks::FTask> DispatchedChunkTasks;
#if WITH_MASSENTITY_DEBUG
		FProcessorWorker* DebugCurrentProcessorWorker = nullptr;
#endif

	};

	/** A worker to execute queued processors */
	class FProcessorWorker
	{
	public:
		FProcessorWorker(FProcessingQueue& InQueue, const FMassExecutionContext& InExecutionContext, bool bInInlineExecution = false);
		~FProcessorWorker();

		void AssignProcessor(FQueuedProcessor* QueuedProcessor);

		void QueueChunk(FMassChunkProcessingQueueParams&& Params);
		const FQueryInfo& GetCurrentQueryInfo() const
		{
			return RunningProcessor->GetCurrentQueryInfo();
		}

		void ExecuteProcessor();
		void WaitForQueryResources();
		void ProcessChunks();
		void Cleanup();
		void ChunkDone(FQueuedChunk* InChunk);

		/**
		 * Force-inline mode: execute a single chunk immediately on the calling thread without
		 * touching the chunk queue, contention tracker, or ChunksRemaining counter. Replaces
		 * the QueueChunk callback path entirely when mass.ForceInlineProcessorExecution is set.
		 */
		void ExecuteChunkInline(FMassChunkProcessingQueueParams&& Params);

		FProcessingQueue& OwningQueue;
		const FMassExecutionContext& GlobalExecutionContext;
		bool bInlineExecution = false;
		FQueuedProcessor* RunningProcessor = nullptr;
		std::atomic<int32> ChunksRemaining{ 0 };
		TSharedPtr<FMassCommandBuffer> CommandBuffer;
		int32 GlobalChunkContentionIndex = INDEX_NONE;

#if WITH_MASSENTITY_DEBUG
		FMassExecutionContext* DebugCurrentExecutionContext = nullptr;
#endif
	};

	/** A worker to process queued chunks */
	class FChunkWorker
	{
	public:

		FChunkWorker(FProcessingQueue& InQueue, FQueuedChunk* InChunk, bool UseChunkCommandBuffer = false);
		~FChunkWorker();
		void ExecuteChunk(bool bInlineExecution);
		void Cleanup();
		void AssignChunk(FQueuedChunk* InChunk)
		{
			Chunk = InChunk;
		}
		TSharedPtr<FMassCommandBuffer>& GetOrCreateCommandBuffer();

		FProcessingQueue& OwningQueue;
		FQueuedChunk* Chunk = nullptr;
		bool bUseContextCommandBuffer = false;
		/** Each worker owns a command buffer that is reused for all the chunks it processes */
		/** The buffer is created in the thread this worker uses, and registered with the queue to be merged/flushed */
		TSharedPtr<FMassCommandBuffer> CommandBuffer;
	};
} // namespace UE::Mass



#undef UE_API
