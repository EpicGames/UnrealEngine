// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructUtilsTypes.h"
#include "StructUtils/InstancedStruct.h"
#include "Templates/SubclassOf.h"
#include "Mass/EntityMacros.h"
#include "MassProcessingTypes.generated.h"

#define UE_API MASSENTITY_API

#ifndef MASS_DO_PARALLEL
#define MASS_DO_PARALLEL !UE_SERVER
#endif // MASS_DO_PARALLEL

struct FMassEntityManager;
class UMassProcessor;
class UMassCompositeProcessor;
struct FMassCommandBuffer;

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EProcessorExecutionFlags : uint8
{
	None = 0 UMETA(Hidden),
	Standalone = 1 << 0,
	Server = 1 << 1,
	Client = 1 << 2,
	Editor = 1 << 3,
	EditorWorld = 1 << 4,
	AllNetModes = Standalone | Server | Client UMETA(Hidden),
	AllWorldModes = Standalone | Server | Client | EditorWorld UMETA(Hidden),
	All = Standalone | Server | Client | Editor | EditorWorld UMETA(Hidden)
};
ENUM_CLASS_FLAGS(EProcessorExecutionFlags);

USTRUCT()
struct FProcessorAuxDataBase
{
	GENERATED_BODY()
};

USTRUCT(BlueprintType, meta = (Deprecated = "5.6"))
struct FMassProcessingContext_DEPRECATED
{
	GENERATED_BODY()

	UPROPERTY()
	float DeltaSeconds = 0.f;

	UPROPERTY()
	FInstancedStruct AuxData;

	UPROPERTY()
	bool bFlushCommandBuffer = true; 
};

namespace UE::Mass
{
	struct FProcessingContext;
} // namespace UE::Mass
using FMassProcessingContext = UE::Mass::FProcessingContext;

/** 
 *  Runtime-usable array of MassProcessor copies
 */
USTRUCT()
struct FMassRuntimePipeline
{
	GENERATED_BODY()

private:
	UPROPERTY()
	TArray<TObjectPtr<UMassProcessor>> Processors;

	EProcessorExecutionFlags ExecutionFlags = EProcessorExecutionFlags::None;

public:
	explicit FMassRuntimePipeline(EProcessorExecutionFlags WorldExecutionFlags = EProcessorExecutionFlags::None);
	UE_API FMassRuntimePipeline(TConstArrayView<TObjectPtr<UMassProcessor>> SeedProcessors, EProcessorExecutionFlags WorldExecutionFlags);
	UE_API FMassRuntimePipeline(TConstArrayView<UMassProcessor*> SeedProcessors, EProcessorExecutionFlags WorldExecutionFlags);

	UE_API void Reset();
	UE_API void Initialize(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager);
	
	/** Copies the given array over to this FMassRuntimePipeline instance. */
	UE_API void SetProcessors(TArrayView<UMassProcessor*> InProcessors);

	/** Directly moves the contents of given array to the Processors member array. */
	UE_API void SetProcessors(TArray<TObjectPtr<UMassProcessor>>&& InProcessors);

	/** Creates runtime copies of UMassProcessors given in InProcessors input parameter, using InOwner as new UMassProcessors' outer. */
	UE_API void CreateFromArray(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner);

	/** Calls CreateFromArray and calls Initialize on all processors afterwards. */
	UE_API void InitializeFromArray(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner, const TSharedRef<FMassEntityManager>& EntityManager);
	
	/** Creates runtime instances of UMassProcessors for each processor class given via InProcessorClasses. 
	 *  The instances will be created with InOwner as outer. */
	UE_API void InitializeFromClassArray(TConstArrayView<TSubclassOf<UMassProcessor>> InProcessorClasses, UObject& InOwner, const TSharedRef<FMassEntityManager>& EntityManager);

	/** Creates a runtime instance of every processors in the given InProcessors array. If a processor of that class
	 *  already exists in Processors array it gets overridden. Otherwise it gets added to the end of the collection.*/
	UE_API void AppendOrOverrideRuntimeProcessorCopies(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner);

	/** Creates a runtime instance of every processors in the given array if there's no processor of that class in Processors already.
	 *  Call this function when adding processors to an already configured FMassRuntimePipeline instance. If you're creating 
	 *  one from scratch calling any of the InitializeFrom* methods will be more efficient (and will produce same results)
	 *  or call AppendOrOverrideRuntimeProcessorCopies.
	 *	NOTE: there's a change in functionality since 5.6 - the function will no longer create duplicates for processors returning true from ShouldAllowMultipleInstances
	 */
	UE_API void AppendUniqueRuntimeProcessorCopies(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner, const TSharedRef<FMassEntityManager>& EntityManager);

	/** Adds InProcessor(s) to Processors without any additional checks */
	UE_API void AppendProcessor(UMassProcessor& InProcessor);
	UE_API void AppendProcessors(TArrayView<TObjectPtr<UMassProcessor>> InProcessors);
	UE_API void AppendProcessors(TArray<TObjectPtr<UMassProcessor>>&& InProcessors);

	/** @return true if the given processor has indeed been added, i.e. will return false if Processor was already part of the pipeline. */
	UE_API bool AppendUniqueProcessor(UMassProcessor& Processor);

	/** Creates an instance of ProcessorClass and adds it to Processors without any additional checks */
	UE_API void AppendProcessor(TSubclassOf<UMassProcessor> ProcessorClass, UObject& InOwner);

	/** @return whether the given processor has been removed from hosted processors collection */
	UE_API bool RemoveProcessor(const UMassProcessor& InProcessor);

	/** goes through Processor looking for a UMassCompositeProcessor instance which GroupName matches the one given as the parameter */
	UE_API UMassCompositeProcessor* FindTopLevelGroupByName(const FName GroupName);

	UE_API bool HasProcessorOfExactClass(TSubclassOf<UMassProcessor> InClass) const;
	bool IsEmpty() const;

	int32 Num() const;
	TConstArrayView<TObjectPtr<UMassProcessor>> GetProcessors() const;
	TConstArrayView<UMassProcessor*> GetProcessorsView() const
	{
		return ObjectPtrDecay(Processors);
	}
	TArrayView<TObjectPtr<UMassProcessor>> GetMutableProcessors();

	/** Returns Processors array using move semantics. This operation clears out this FMassRuntimePipeline instance. */
	TArray<TObjectPtr<UMassProcessor>>&& MoveProcessorsArray();

	UE_API friend uint32 GetTypeHash(const FMassRuntimePipeline& Instance);

	/**
	 * Sorts processors aggregates in Processors array so that the ones with higher ExecutionPriority are executed first
	 * The function will also remove nullptrs, if any, before sorting.
	 */
	UE_API void SortByExecutionPriority();

	//-----------------------------------------------------------------------------
	// DEPRECATED
	//-----------------------------------------------------------------------------
	UE_DEPRECATED(5.6, "This flavor of Initialize is deprecated. Please use the one requiring a FMassEntityManager parameter")
	UE_API void Initialize(UObject& Owner);

	UE_DEPRECATED(5.6, "This flavor of InitializeFromArray is deprecated. Please use the one requiring a FMassEntityManager parameter")
	UE_API void InitializeFromArray(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner);
	
	UE_DEPRECATED(5.6, "This flavor of InitializeFromClassArray is deprecated. Please use the one requiring a FMassEntityManager parameter")
	UE_API void InitializeFromClassArray(TConstArrayView<TSubclassOf<UMassProcessor>> InProcessorClasses, UObject& InOwner);

	UE_DEPRECATED(5.6, "This flavor of AppendUniqueRuntimeProcessorCopies is deprecated. Please use the one requiring a FMassEntityManager parameter")
	UE_API void AppendUniqueRuntimeProcessorCopies(TConstArrayView<const UMassProcessor*> InProcessors, UObject& InOwner);

	UE_DEPRECATED(5.6, "This flavor of SetProcessors is deprecated. Please use one of the others.")
	UE_API void SetProcessors(TArray<UMassProcessor*>&& InProcessors);
};

UENUM()
enum class EMassProcessingPhase : uint8
{
	PrePhysics,
	StartPhysics,
	DuringPhysics,
	EndPhysics,
	PostPhysics,
	FrameEnd,
	MAX,
};

struct FMassProcessorOrderInfo
{
	enum class EDependencyNodeType : uint8
	{
		Invalid,
		Processor,
		GroupStart,
		GroupEnd
	};

	FName Name;
	UMassProcessor* Processor = nullptr;
	EDependencyNodeType NodeType = EDependencyNodeType::Invalid;
	TArray<FName> Dependencies;
	int32 SequenceIndex = INDEX_NONE;
};

//-----------------------------------------------------------------------------
// Inlines
//-----------------------------------------------------------------------------
inline FMassRuntimePipeline::FMassRuntimePipeline(EProcessorExecutionFlags WorldExecutionFlags)
	: ExecutionFlags(WorldExecutionFlags)
{
	
}

inline bool FMassRuntimePipeline::IsEmpty() const
{
	return Processors.IsEmpty();
}

inline int32 FMassRuntimePipeline::Num() const
{
	return Processors.Num();
}

inline TConstArrayView<TObjectPtr<UMassProcessor>> FMassRuntimePipeline::GetProcessors() const
{
	return Processors;
}

inline TArrayView<TObjectPtr<UMassProcessor>> FMassRuntimePipeline::GetMutableProcessors()
{
	return Processors;
}

inline TArray<TObjectPtr<UMassProcessor>>&& FMassRuntimePipeline::MoveProcessorsArray()
{
	return MoveTemp(Processors);
}

#undef UE_API
