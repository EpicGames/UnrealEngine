// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#include "MassEntityManager.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassProcessingTypes.h"
#include "Async/TaskGraphInterfaces.h"
#include "MassCommandBuffer.h"
#include "MassRequirements.h"
#include "MassProcessor.generated.h"


#define UE_API MASSENTITY_API

struct FMassProcessingPhaseConfig;
class UMassCompositeProcessor;
struct FMassDebugger;
struct FMassEntityQuery;
struct FMassEntityManager;
struct FMassExecutionContext;
struct FMassExecutionRequirements;
struct FMassSubsystemRequirements;
namespace UE::Mass
{
	struct FQueryExecutor;
	struct FProcessingQueue;
	struct FQueuedProcessor;
} // namespace UE::Mass

enum class EProcessorCompletionStatus : uint8
{
	Invalid,
	Threaded,
	Postponed,
	Done
};

USTRUCT()
struct FMassProcessorExecutionOrder
{
	GENERATED_BODY()

	/** Determines which processing group this processor will be placed in. Leaving it empty ("None") means "top-most group for my ProcessingPhase" */
	UPROPERTY(EditAnywhere, Category = Processor, config)
	FName ExecuteInGroup = FName();

	UPROPERTY(EditAnywhere, Category = Processor, config)
	TArray<FName> ExecuteBefore;

	UPROPERTY(EditAnywhere, Category = Processor, config)
	TArray<FName> ExecuteAfter;
};

UENUM()
enum class EActivationState : uint8
{
	Inactive,
	Active,
	OneShot,	// one-shot processor will auto-disable itself after the next CallExecute call
};

/**
 * Values determining whether a processor wants to be pruned at runtime. The value is not used when
 * processing graph is generated for project configuration purposes or debug-time graph visualization purposes
 * This behavior can be overridden by UMassProcessor::ShouldAllowQueryBasedPruning child class overrides
 */
UENUM()
enum class EMassQueryBasedPruning : uint8
{
	Prune,		// pruning will always be applied at runtime
	Never,		// pruning will never be applied at runtime
	Default = Prune
};

UCLASS(abstract, EditInlineNew, CollapseCategories, config = Mass, defaultconfig, ConfigDoNotCheckDefaults, MinimalAPI)
class UMassProcessor : public UObject
{
	GENERATED_BODY()

public:

	UE_API UMassProcessor();
	UE_API explicit UMassProcessor(const FObjectInitializer& ObjectInitializer);

	bool IsInitialized() const;

	/** Calls InitializeInternal and handles initialization bookkeeping. */
	UE_API void CallInitialize(const TNotNull<UObject*> Owner, const TSharedRef<FMassEntityManager>& EntityManager);
	
	UE_API virtual FGraphEventRef DispatchProcessorTasks(const TSharedPtr<FMassEntityManager>& EntityManager, FMassExecutionContext& ExecutionContext, const FGraphEventArray& Prerequisites = FGraphEventArray());

	EProcessorExecutionFlags GetExecutionFlags() const;

	/** Whether this processor should execute according the CurrentExecutionFlags parameters */
	bool ShouldExecute(const EProcessorExecutionFlags CurrentExecutionFlags) const;
	UE_API void CallExecute(FMassEntityManager& EntityManager, FMassExecutionContext& Context);

	/** 
	 * Controls whether there can be multiple instances of a given class in a single FMassRuntimePipeline and during 
	 * dependency solving. 
	 */
	bool ShouldAllowMultipleInstances() const;

	void DebugOutputDescription(FOutputDevice& Ar) const;
	UE_API virtual void DebugOutputDescription(FOutputDevice& Ar, int32 Indent) const;
	UE_API virtual FString GetProcessorName() const;
	
	//----------------------------------------------------------------------//
	// Ordering functions 
	//----------------------------------------------------------------------//
	/**
	 * Indicates whether this processor can ever be pruned while considered for a phase processing graph. A processor
	 * can get pruned if none of its registered queries interact with archetypes instantiated at the moment of graph
	 * building. This can also happen for special processors that don't register any queries - if that's the case override 
	 * this function to return an appropriate value or use QueryBasedPruning to configure the expected behavior.
	 * By default, the processor will be the subject of pruning when bRuntimeMode == true.
	 * @param bRuntimeMode indicates whether the pruning is being done for game runtime (true) or editor-time presentation (false)
	 */
	UE_API virtual bool ShouldAllowQueryBasedPruning(const bool bRuntimeMode = true) const;

	UE_API virtual EMassProcessingPhase GetProcessingPhase() const;
	UE_API virtual void SetProcessingPhase(EMassProcessingPhase Phase);
	bool DoesRequireGameThreadExecution() const;
	
	const FMassProcessorExecutionOrder& GetExecutionOrder() const;

	/** By default,  fetches requirements declared entity queries registered via RegisterQuery. Processors can override 
	 *	this function to supply additional requirements */
	UE_API virtual void ExportRequirements(FMassExecutionRequirements& OutRequirements) const;

	const FMassSubsystemRequirements& GetProcessorRequirements() const;
	const FMassEntityCreationRequirements& GetProcessorEntityCreationRequirements() const;

	/**
	 * @return the current value of ExecutionPriority
	 * @see SetExecutionPriority
	 * @see ExecutionPriority
	 */
	int16 GetExecutionPriority() const;

	/**
	 * Sets new ExecutionPriority for this processor. The change will take effect the next time the processing graph is built
	 * Note that at this point this operation does not cause processing graph rebuilding, so this function should be used
	 * before processor's initialization or as part of code that will cause processing graph rebuilding anyway.
	 * @see ExecutionPriority
	 */
	void SetExecutionPriority(const int16 NewExecutionPriority);

	/** Adds Query to RegisteredQueries list. Query is required to be a member variable of this processor. Not meeting
	 *  this requirement will cause check failure and the query won't be registered. */
	UE_API void RegisterQuery(FMassEntityQuery& Query);

	void MarkAsDynamic();
	bool IsDynamic() const;

	/**
	 * Marks processor as "Active" (@see ActivationState for details). If called during Mass processing the
	 * call will take effect next phase.
	 */
	void MakeActive();
	/**
	 * Marks processor as "One Shot" (@see ActivationState for details). If called during Mass processing
	 * the call will take effect next phase. The processor will auto-disable after execution.
	 */
	void MakeOneShot();
	/**
	 * Deactivate the processor, it will no longer execute its `Execute` function.
	 */
	void MakeInactive();

	bool IsActive() const;

	bool ShouldAutoAddToGlobalList() const;
#if WITH_EDITOR
	bool ShouldShowUpInSettings() const;
#endif // WITH_EDITOR

	/** Sets bAutoRegisterWithProcessingPhases. Setting it to true will result in this processor class being always 
	 * instantiated to be automatically evaluated every frame. @see FMassProcessingPhaseManager
	 * Note that calling this function is only valid on CDOs. Calling it on a regular instance will fail an ensure and 
	 * have no other effect, i.e. CDO's value won't change */
	UE_API void SetShouldAutoRegisterWithGlobalList(const bool bAutoRegister);

	UE_API void GetArchetypesMatchingOwnedQueries(const FMassEntityManager& EntityManager, TArray<FMassArchetypeHandle>& OutArchetype);
	UE_API bool DoesAnyArchetypeMatchOwnedQueries(const FMassEntityManager& EntityManager);
	int32 GetOwnedQueriesNum() const;

	/**
	 * Indicates whether or not this processor has declared in its requirements that it creates entities.
	 */
	UE_API bool HasEntityCreationRequirements();

	/**
	 * Export the cached archetypes (by handle) that this processor can create
	 */
	UE_API void ExportArchetypesCreated(TArray<FMassArchetypeHandle>& OutArchetypes);

	/**
	 * Resolves all the created archetype handles and creates the archetypes if they don't exist
	 */
	UE_API void ResolveRequestedArchetypes(const FMassEntityManager& EntityManager);

	/**
	 * Clears the archetypes created cache to force it to rebuild before next use
	 */
	UE_API void ClearArchetypesCreatedHandles();

	
#if CPUPROFILERTRACE_ENABLED
	FString StatId;
#endif
	
protected:
	/** Called to initialize the processor's internal state. Override to perform custom steps. */
	UE_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager);

	/**
	 * Called internally during processor's initialization so that child classes configure their owned queries
	 * with requirements. This function is called before the processor gets considered by Mass dependency
	 * solver, and the requirement information stored in queries is crucial for that process. 
	 */
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager);

	UE_API virtual void PostInitProperties() override;

	/**
	 * Called during the processing phase to which this processor is registered.
	 * Default implementation requires that AutoExecuteQuery is populated with a QueryExecutor.
	 */
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context);

protected:
	/** Configures when this given processor can be executed in relation to other processors and processing groups, within its processing phase. */
	UPROPERTY(EditDefaultsOnly, Category = Processor, config)
	FMassProcessorExecutionOrder ExecutionOrder;

	/** Processing phase this processor will be automatically run as part of. Needs to be set before the processor gets
	 *  registered with MassProcessingPhaseManager, otherwise it will have no effect. This property is usually read via
	 *  a given class's CDO, so it's recommended to set it in the constructor. */
	UPROPERTY(EditDefaultsOnly, Category = Processor, config)
	EMassProcessingPhase ProcessingPhase = EMassProcessingPhase::PrePhysics;

	/** Whether this processor should be executed on StandAlone or Server or Client */
	UPROPERTY(EditAnywhere, Category = "Pipeline", meta = (Bitmask, BitmaskEnum = "/Script/MassEntity.EProcessorExecutionFlags"), config)
	uint8 ExecutionFlags;

	/** Configures whether this processor should be automatically included in the global list of processors executed every tick (see ProcessingPhase and ExecutionOrder). */
	UPROPERTY(EditDefaultsOnly, Category = Processor, config)
	uint8 bAutoRegisterWithProcessingPhases : 1 = true;

	/** Meant as a class property, make sure to set it in subclass' constructor. Controls whether there can be multiple
	 *  instances of a given class in a single FMassRuntimePipeline and during dependency solving. */
	uint8 bAllowMultipleInstances : 1 = false;

	UPROPERTY(EditDefaultsOnly, Category = Processor, config)
	uint8 bRequiresGameThreadExecution : 1 = false;

	/** Combine all query and processor requirements when resolving contention in FProcessingQueue
	 *  A value of true is legacy mode */
	UPROPERTY(EditDefaultsOnly, Category = Processor, config)
	uint8 bAggregateRequirements : 1 = false;

#if WITH_EDITORONLY_DATA
	/** Used to permanently remove a given processor class from PipeSetting's listing. Used primarily for test-time 
	 *  processor classes, but can also be used by project-specific code to prune the processor list. */
	UPROPERTY(config)
	uint8 bCanShowUpInSettings : 1 = true;
#endif // WITH_EDITORONLY_DATA

private:
	/**
	 * Gets set to true when an instance of the processor gets added to the phase processing as a "dynamic processor".
	 * Once set it's never expected to be cleared out to `false` thus the private visibility of the member variable.
	 * A "dynamic" processor is a one that has bAutoRegisterWithProcessingPhases == false, meaning it's not automatically
	 * added to the processing graph. Additionally, making processors dynamic allows one to have multiple instances
	 * of processors of the same class. 
	 * @see MarkAsDynamic()
	 * @see IsDynamic()
	 */
	uint8 bIsDynamic : 1 = false;

	/** Used to track whether Initialized has been called. */
	uint8 bInitialized : 1 = false;

	/** Cache presence of entity creation requirements in owned queries. */
	uint8 bQueryHasEntityCreationRequirements : 1 = false;

	/** Tracks if created archetypes handles are resolved*/
	uint8 bCreatedArchetypeHandlesResolved : 1 = false;

	/**
	 * Processors can be activated/deactivated at runtime. Deactivating a running processor will not disrupt the processing
	 * graph since the disabled processor's dependencies will get passed down to the subsequent processors depending on this one.
	 * Deactivating processor's CDO will result in every instance starting off as disabled. Those will still be considered
	 * while building the processor dependency graph and one included in the processing graph will function just as the
	 * processor instances disabled at runtime (i.e. won't run, but pass down their dependencies).
	 * A special type of activation is "One Shot" mode - it works just like "Active" state, but it will auto-disable itself
	 * upon completion of the next CallExecute call.
	 */
	UPROPERTY(EditDefaultsOnly, Category = Processor, config)
	EActivationState ActivationState = EActivationState::Active;

protected:
	/**
	 * Denoted how important it is for this processor to be executed as soon as possible within a processing graph.
	 * The larger the number the higher the priority. It's used in two ways:
	 * - used when sorting nodes that otherwise seem similar in terms of "which processor to pick for execution next"
	 * - affects the priority of the dependencies - if this super-important processor is waiting for processor A and B,
	 *		then A and B become super important as well.
	 */
	UPROPERTY(EditDefaultsOnly, Category = Processor, config)
	int16 ExecutionPriority = 0;

	/**
	 * Determines whether given processor wants to be pruned from the execution graph when there are
	 * no archetypes matching its requirements.
	 * Defaults to EMassQueryBasedPruning::Prune, @see EMassQueryBasedPruning
	 */
	EMassQueryBasedPruning QueryBasedPruning = EMassQueryBasedPruning::Default;

	friend UMassCompositeProcessor;
	friend FMassDebugger;
	friend UE::Mass::FQueuedProcessor; // needs access to OwnedQueries

	/** A query representing elements this processor is accessing in Execute function outside of query execution */
	FMassSubsystemRequirements ProcessorRequirements;

	/** Archetypes this processor can create when no query is on the stack */
	FMassEntityCreationRequirements ProcessorEntityCreationRequirements;

	/** A QueryExecutor that can optionally be run in lieu of overriding the Execute function. */
	TSharedPtr<UE::Mass::FQueryExecutor> AutoExecuteQuery;

private:
	/** Stores processor's queries registered via RegisterQuery. 
	 *  @note that it's safe to store pointers here since RegisterQuery does verify that a given registered query is 
	 *  a member variable of a given processor */
	TArray<FMassEntityQuery*> OwnedQueries;

#if WITH_MASSENTITY_DEBUG
	FString DebugDescription;
#endif

	//-----------------------------------------------------------------------------
	// DEPRECATED
	//-----------------------------------------------------------------------------
public:
	UE_DEPRECATED(5.6, "Initialize is deprecated. Override InitializeInternal(UObject&, const TSharedRef<FMassEntityManager>&) instead. If you want to call the function, use CallInitialize.")
	UE_API virtual void Initialize(UObject& Owner) final;
	UE_DEPRECATED(5.6, "This flavor of ConfigureQueries is deprecated. Override ConfigureQueries(const TSharedRef<FMassEntityManager>&) instead.")
	virtual void ConfigureQueries() final
	{
	};
};


UCLASS(MinimalAPI)
class UMassCompositeProcessor : public UMassProcessor
{
	GENERATED_BODY()

	friend FMassDebugger;
public:
	struct FDependencyNode
	{
		FName Name;
		UMassProcessor* Processor = nullptr;
		TArray<int32> Dependencies;
#if WITH_MASSENTITY_DEBUG
		int32 SequenceIndex = INDEX_NONE;
#endif // WITH_MASSENTITY_DEBUG
	};

public:
	UE_API UMassCompositeProcessor();

	UE_API void SetChildProcessors(TArrayView<UMassProcessor*> InProcessors);
	UE_API void SetChildProcessors(TArray<TObjectPtr<UMassProcessor>>&& InProcessors);

	UE_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& InEntityManager) override;
	UE_API virtual void DebugOutputDescription(FOutputDevice& Ar, int32 Indent = 0) const override;
	UE_API virtual void SetProcessingPhase(EMassProcessingPhase Phase) override;

	UE_API void SetGroupName(FName NewName);
	FName GetGroupName() const;

	UE_API virtual void SetProcessors(TArrayView<UMassProcessor*> InProcessorInstances, const TSharedPtr<FMassEntityManager>& EntityManager = nullptr);

	/** 
	 * Builds flat processing graph that's being used for multithreading execution of hosted processors.
	 */
	UE_API virtual void BuildFlatProcessingGraph(TConstArrayView<FMassProcessorOrderInfo> SortedProcessors);

	/**
	 * Adds processors in InOutOrderedProcessors to ChildPipeline. 
	 * Note that this operation is non-destructive for the existing processors - the ones of classes found in InOutOrderedProcessors 
	 * will be retained and used instead of the instances provided via InOutOrderedProcessors. Respective entries in InOutOrderedProcessors
	 * will be updated to reflect the reuse.
	 * The described behavior however is available only for processors with bAllowMultipleInstances == false.
	 */
	UE_API void UpdateProcessorsCollection(TArrayView<FMassProcessorOrderInfo> InOutOrderedProcessors, EProcessorExecutionFlags InWorldExecutionFlags = EProcessorExecutionFlags::None);

	/** adds SubProcessor to an appropriately named group. If RequestedGroupName == None then SubProcessor
	 *  will be added directly to ChildPipeline. If not then the indicated group will be searched for in ChildPipeline 
	 *  and if it's missing it will be created and AddGroupedProcessor will be called recursively */
	UE_API void AddGroupedProcessor(FName RequestedGroupName, UMassProcessor& SubProcessor);

	UE_API virtual FGraphEventRef DispatchProcessorTasks(const TSharedPtr<FMassEntityManager>& EntityManager, FMassExecutionContext& ExecutionContext, const FGraphEventArray& Prerequisites = FGraphEventArray()) override;

	bool IsEmpty() const;

	UE_API virtual FString GetProcessorName() const override;

	TConstArrayView<UMassProcessor*> GetChildProcessorsView() const;

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	/** RequestedGroupName can indicate a multi-level group name, like so: A.B.C
	 *  We need to extract the highest-level group name ('A' in the example), and see if it already exists. 
	 *  If not, create it. 
	 *  @param RequestedGroupName name of the group for which we want to find or create the processor.
	 *  @param OutRemainingGroupName contains the group name after cutting the high-level group. In the used example it
	 *    will contain "B.C". This value is then used to recursively create subgroups */
	UE_API UMassCompositeProcessor* FindOrAddGroupProcessor(FName RequestedGroupName, FString* OutRemainingGroupName = nullptr);

protected:
	UPROPERTY(VisibleAnywhere, Category=Mass)
	FMassRuntimePipeline ChildPipeline;

	/** Group name that will be used when resolving processor dependencies and grouping */
	UPROPERTY()
	FName GroupName;

#if WITH_MASSENTITY_DEBUG
	bool bDebugLogNewProcessingGraph = false;
#endif // WITH_MASSENTITY_DEBUG

	TArray<FDependencyNode> FlatProcessingGraph;

	struct FProcessorCompletion
	{
		FGraphEventRef CompletionEvent;
		EProcessorCompletionStatus Status = EProcessorCompletionStatus::Invalid;

		bool IsDone() const 
		{
			return Status == EProcessorCompletionStatus::Done || (CompletionEvent.IsValid() && CompletionEvent->IsComplete());
		}

		void Wait()
		{
			if (CompletionEvent.IsValid())
			{
				CompletionEvent->Wait();
			}
		}
	};
	TArray<FProcessorCompletion> CompletionStatus;

	//-----------------------------------------------------------------------------
	// DEPRECATED
	//-----------------------------------------------------------------------------
public:
	UE_DEPRECATED(5.6, "This flavor of SetChildProcessors is deprecated. Please use one of the others.")
	UE_API void SetChildProcessors(TArray<UMassProcessor*>&& InProcessors);
};


//-----------------------------------------------------------------------------
// UMassProcessor inlines
//-----------------------------------------------------------------------------
inline bool UMassProcessor::IsInitialized() const
{
	return bInitialized;
}

inline EProcessorExecutionFlags UMassProcessor::GetExecutionFlags() const
{
	return static_cast<EProcessorExecutionFlags>(ExecutionFlags);
}

inline bool UMassProcessor::ShouldExecute(const EProcessorExecutionFlags CurrentExecutionFlags) const
{
	return (GetExecutionFlags() & CurrentExecutionFlags) != EProcessorExecutionFlags::None;
}

inline bool UMassProcessor::ShouldAllowMultipleInstances() const
{
	return bAllowMultipleInstances;
}

inline int32 UMassProcessor::GetOwnedQueriesNum() const
{
	return OwnedQueries.Num();
}

inline void UMassProcessor::DebugOutputDescription(FOutputDevice& Ar) const
{
	DebugOutputDescription(Ar, 0);
}

inline bool UMassProcessor::DoesRequireGameThreadExecution() const
{
	return bRequiresGameThreadExecution;
}
	
inline const FMassProcessorExecutionOrder& UMassProcessor::GetExecutionOrder() const
{
	return ExecutionOrder;
}

inline const FMassSubsystemRequirements& UMassProcessor::GetProcessorRequirements() const
{
	return ProcessorRequirements;
}

inline const FMassEntityCreationRequirements& UMassProcessor::GetProcessorEntityCreationRequirements() const
{
	return ProcessorEntityCreationRequirements;
}

inline int16 UMassProcessor::GetExecutionPriority() const
{
	return ExecutionPriority; 
}

inline void UMassProcessor::SetExecutionPriority(const int16 NewExecutionPriority)
{
	ExecutionPriority = NewExecutionPriority; 
}

inline void UMassProcessor::MarkAsDynamic()
{
	bIsDynamic = true;
}

inline bool UMassProcessor::IsDynamic() const
{
	return bIsDynamic != 0;
}

inline void UMassProcessor::MakeActive()
{
	ActivationState = EActivationState::Active;
}

inline void UMassProcessor::MakeOneShot()
{
	ActivationState = EActivationState::OneShot;
}

inline void UMassProcessor::MakeInactive()
{
	ActivationState = EActivationState::Inactive;
}

inline bool UMassProcessor::IsActive() const
{
	return ActivationState != EActivationState::Inactive;
}

inline bool UMassProcessor::ShouldAutoAddToGlobalList() const
{
	return bAutoRegisterWithProcessingPhases;
}

#if WITH_EDITOR
inline bool UMassProcessor::ShouldShowUpInSettings() const
{
	return ShouldAutoAddToGlobalList() || bCanShowUpInSettings;
}
#endif // WITH_EDITOR

//-----------------------------------------------------------------------------
// UMassCompositeProcessor inlines
//-----------------------------------------------------------------------------
inline FName UMassCompositeProcessor::GetGroupName() const
{
	return GroupName;
}

inline bool UMassCompositeProcessor::IsEmpty() const
{
	return ChildPipeline.IsEmpty();
}

inline TConstArrayView<UMassProcessor*> UMassCompositeProcessor::GetChildProcessorsView() const
{
	return ChildPipeline.GetProcessors();
}

#undef UE_API 
