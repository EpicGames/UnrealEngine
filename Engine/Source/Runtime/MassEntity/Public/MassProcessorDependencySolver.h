// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessingTypes.h"
#include "MassEntityTypes.h"
#include "MassArchetypeTypes.h"
#include "Containers/StaticArray.h"

#define UE_API MASSENTITY_API


class UMassProcessor;
namespace UE::Mass
{
	struct FTypeManager;
} // namespace UE::Mass

namespace EMassAccessOperation
{
	constexpr uint32 Read = 0;
	constexpr uint32 Write = 1;
	constexpr uint32 MAX = 2;
};

template<typename T>
struct TMassExecutionAccess
{
	T Read;
	T Write;

	T& operator[](const uint32 OpIndex)
	{
		check(OpIndex < EMassAccessOperation::MAX);
		return OpIndex == EMassAccessOperation::Read ? Read : Write;
	}

	const T& operator[](const uint32 OpIndex) const
	{
		check(OpIndex < EMassAccessOperation::MAX);
		return OpIndex == EMassAccessOperation::Read ? Read : Write;
	}

	TConstArrayView<T> AsArrayView() const
	{
		return MakeArrayView(&Read, 2);
	}

	bool IsEmpty() const
	{
		return Read.IsEmpty() && Write.IsEmpty();
	}
};

/** 
 * TMassExecutionAccess specialization for FMassConstSharedFragmentBitSet to enforce lack of access (not needed) and
 * no "Write" component (conceptually doesn't make sense).
 */
template<>
struct TMassExecutionAccess<FMassConstSharedFragmentBitSet>
{
	FMassConstSharedFragmentBitSet Read;
	TConstArrayView<FMassConstSharedFragmentBitSet> AsArrayView() const
	{
		return MakeArrayView(&Read, 1);
	}
	bool IsEmpty() const
	{
		return Read.IsEmpty();
	}
};

struct FMassExecutionRequirements
{
	UE_API void Append(const FMassExecutionRequirements& Other);
	UE_API void CountResourcesUsed();
	UE_API int32 GetTotalBitsUsedCount();
	UE_API bool IsEmpty() const;
	UE_API FMassArchetypeCompositionDescriptor AsCompositionDescriptor() const;

	TMassExecutionAccess<FMassFragmentBitSet> Fragments;
	TMassExecutionAccess<FMassFragmentBitSet> IndirectFragments;
	TMassExecutionAccess<FMassChunkFragmentBitSet> ChunkFragments;
	TMassExecutionAccess<FMassSharedFragmentBitSet> SharedFragments;
	TMassExecutionAccess<FMassConstSharedFragmentBitSet> ConstSharedFragments;
	TMassExecutionAccess<FMassExternalSubsystemBitSet> RequiredSubsystems;
	/** Sparse elements - treated like indirect fragments (archetype-agnostic access) */
	TMassExecutionAccess<FMassElementBitSet> SparseElements;
	FMassTagBitSet RequiredAllTags;
	FMassTagBitSet RequiredAnyTags;
	FMassTagBitSet RequiredNoneTags;
	int32 ResourcesUsedCount = INDEX_NONE;
};

struct FMassProcessorDependencySolver
{
	struct FNode
	{
		FNode(const FName InName, UMassProcessor* InProcessor, const int32 InNodeIndex = INDEX_NONE)
			: Name(InName), Processor(InProcessor), NodeIndex(InNodeIndex)
		{
		}

		bool IsGroup() const
		{
			return Processor == nullptr;
		}
		/**
		 * @return `true` when everything's fine, `false` when cycles have been encountered. If that
		 *	happens `OutCycleIndices` gets filled with the relevant node indices.
		 */
		bool IncreaseWaitingNodesCount(TArrayView<FNode> InAllNodes, const int32 IterationsLimit, TArray<int32>& OutCycleIndices);
		bool IncreaseWaitingNodesCountAndPriority(TArrayView<FNode> InAllNodes, const int32 IterationsLimit, TArray<int32>& OutCycleIndices
			, const int32 InChildPriority = TNumericLimits<int32>::Min());
		void UpdateExecutionPriority(const int32 ChildExecutionPriority)
		{
			// picking the max execution priority - note that we're increasing child
			// priority to ensure dependencies  always have a higher stored priority
			// than the nodes that depend on them
			MaxExecutionPriority = FMath::Max(int32(ChildExecutionPriority) + 1, MaxExecutionPriority);
		}


		FName Name;
		UMassProcessor* Processor = nullptr;
		TArray<int32> OriginalDependencies;
		TArray<int32> TransientDependencies;
		TArray<FName> ExecuteBefore;
		TArray<FName> ExecuteAfter;
		FMassExecutionRequirements Requirements;
		int32 NodeIndex = INDEX_NONE;
		/** indicates how often given node can be found in dependencies sequence for other nodes  */
		int32 TotalWaitingNodes = 0;
		/**
		 * The highest execution priority across this node and all nodes in its logical dependency chain.
		 * Nodes that are only dependencies due to resource contention (i.e. blocking shared resources)
		 * are excluded from this calculation.
		 * Note: we use int32 rather than UMassProcessor.ExecutionPriority's int16 to avoid overflow
		 * when propagating priorities in UpdateExecutionPriority.
		 */
		int32 MaxExecutionPriority = 0;
		/** 
		 * indicates how deep within dependencies graph this give node is, or in other words, what's the longest sequence 
		 * from this node to a dependency-less "parent" node 
		 */
		int32 SequencePositionIndex = 0;
		TArray<int32> SubNodeIndices;
		TArray<FMassArchetypeHandle> ValidArchetypes;
	};

private:
	struct FResourceUsage
	{
		FResourceUsage(const TArray<FNode>& InAllNodes);

		bool CanAccessRequirements(const FMassExecutionRequirements& TestedRequirements, const TArray<FMassArchetypeHandle>& InArchetypes) const;
		void SubmitNode(const int32 NodeIndex, FNode& InOutNode);

	private:
		struct FResourceUsers
		{
			TArray<int32> Users;
		};
		
		struct FResourceAccess
		{
			TArray<FResourceUsers> Access;
		};
		
		FMassExecutionRequirements Requirements;
		TMassExecutionAccess<FResourceAccess> FragmentsAccess;
		TMassExecutionAccess<FResourceAccess> IndirectFragmentsAccess;
		TMassExecutionAccess<FResourceAccess> ChunkFragmentsAccess;
		TMassExecutionAccess<FResourceAccess> SharedFragmentsAccess;
		TMassExecutionAccess<FResourceAccess> RequiredSubsystemsAccess;
		TMassExecutionAccess<FResourceAccess> SparseElementsAccess;
		TConstArrayView<FNode> AllNodesView;

		template<typename TBitSet>
		void HandleElementType(TMassExecutionAccess<FResourceAccess>& ElementAccess
			, const TMassExecutionAccess<TBitSet>& TestedRequirements, FMassProcessorDependencySolver::FNode& InOutNode, const int32 NodeIndex, bool bSkipArchetypeOverlapTest = false);

		template<typename TBitSet>
		static bool CanAccess(const TMassExecutionAccess<TBitSet>& StoredElements, const TMassExecutionAccess<TBitSet>& TestedElements);

		/** Determines whether any of the Elements' (i.e. Fragment, Tag,...) users operate on any of the archetypes given via InArchetypes */
		bool HasArchetypeConflict(TMassExecutionAccess<FResourceAccess> ElementAccess, const TArray<FMassArchetypeHandle>& InArchetypes) const;
	};

public:
	/** Optionally returned by ResolveDependencies and contains information about processors that have been pruned and 
	 *  other potentially useful bits. To be used in a transient fashion. */
	struct FResult
	{
		FString DependencyGraphFileName;
		TArray<TObjectPtr<UMassProcessor>> PrunedProcessors;
		int32 MaxSequenceLength = 0;
		uint32 ArchetypeDataVersion = 0;
		UE_DEPRECATED(5.6, "This property is deprecated, replaced by PrunedProcessors")
		TArray<TSubclassOf<UMassProcessor>> PrunedProcessorClasses;

		void Reset()
		{
			PrunedProcessors.Reset();
			MaxSequenceLength = 0;
			ArchetypeDataVersion = 0;
		}
	};

	MASSENTITY_API FMassProcessorDependencySolver(TArrayView<UMassProcessor* const> InProcessors, const bool bIsGameRuntime = true);
	MASSENTITY_API void ResolveDependencies(TArray<FMassProcessorOrderInfo>& OutResult, TSharedPtr<FMassEntityManager> EntityManager = nullptr, FResult* InOutOptionalResult = nullptr);

	MASSENTITY_API static void CreateSubGroupNames(FName InGroupName, TArray<FString>& SubGroupNames);

	/** Determines whether the dependency solving that produced InResult will produce different results if run with a given EntityManager */
	static bool IsResultUpToDate(const FResult& InResult, TSharedPtr<FMassEntityManager> EntityManager);

	bool IsSolvingForSingleThread() const
	{
		return bSingleThreadTarget;
	}

protected:
	// note that internals are protected rather than private to support unit testing

	/**
	 * Traverses InOutIndicesRemaining in search of the first RootNode's node that has no dependencies left. Once found 
	 * the node's index gets added to OutNodeIndices, removed from dependency lists from all other nodes and the function 
	 * quits.
	 * @return 'true' if a dependency-less node has been found and added to OutNodeIndices; 'false' otherwise.
	 */
	bool PerformSolverStep(FResourceUsage& ResourceUsage, TArray<int32>& InOutIndicesRemaining, TArray<int32>& OutNodeIndices);
	
	int32 CreateNodes(UMassProcessor& Processor);
	void BuildDependencies();
	void Solve(TArray<FMassProcessorOrderInfo>& OutResult);
	void LogNode(const FNode& Node, int Indent = 0);

	/** Finds out which subsystems handle multithreaded RW operations, and caches the result in MultiThreadedSystemsBitSet */
	void GatherSubsystemInformation(const UE::Mass::FTypeManager& TypeManager);
	
	TArrayView<UMassProcessor* const> Processors;

	/**
	 * indicates whether we're generating processor order to be run in single-threaded or multithreaded environment (usually
	 * this means Dedicated Server vs Any other configuration). In Single-Threaded mode we can skip a bunch of expensive,
	 * fine-tuning tests.
	 * @Note currently the value depends on MASS_DO_PARALLEL and there's no way to configure it otherwise, but there's 
	 * nothing inherently stopping us from letting users configure it.
	 */
	const bool bSingleThreadTarget = !MASS_DO_PARALLEL;
	const bool bGameRuntime = true;
	FString DependencyGraphFileName;
	TArray<FNode> AllNodes;
	TMap<FName, int32> NodeIndexMap;

	/** Stores the subsystems we know of that handle multithreaded access well - we filter those out, we don't need to consider them. */
	FMassExternalSubsystemBitSet MultiThreadedSystemsBitSet;
};

#undef UE_API
