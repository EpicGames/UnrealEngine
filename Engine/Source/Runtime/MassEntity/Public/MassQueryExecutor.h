// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotNull.h"
#include "MassEntityQuery.h"
#include "MassExecutionContext.h"

#define UE_API MASSENTITY_API

class UMassProcessor;

namespace UE::Mass
{

/** Interface for QueryDefinition templates. Not intended for other direct inheritance. */
struct FQueryDefinitionBase
{
	virtual void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) = 0;
	virtual void SetupForExecute(FMassExecutionContext& Context) = 0;
};

/**
 * A MassEntityQuery wrapper with type-safe data access.
 */
struct FQueryExecutor
{
	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, FQueryExecutor>::IsDerived>::Type>
	static TSharedPtr<T> CreateQuery(FMassEntityQuery& InQuery, UObject* InLogOwner = nullptr)
	{
		TSharedPtr<T> Query = MakeShared<T>();
		Query->BoundQuery = &InQuery;
		Query->LogOwner = InLogOwner;

#if WITH_MASSENTITY_DEBUG
		Query->DebugSize = sizeof(T);
		Query->ValidateAccessors();
#endif

		return Query;
	}

	template<typename... Ts>
	friend struct FQueryDefinition;

	friend ::UMassProcessor;

	UE_API explicit FQueryExecutor(FMassEntityQuery& InQuery, UObject* InLogOwner = nullptr);
	virtual ~FQueryExecutor() = default;

	/** Override with logic to perform against the entities returned by this query. */
	virtual void Execute(FMassExecutionContext& Context) = 0;

protected:

	inline UObject* GetLogOwner()
	{
		return LogOwner.Get();
	}

	template<typename TAccessors, typename TFunc>
	inline void ForEachEntityChunk(FMassExecutionContext& ExecutionContext, TAccessors& Accessors, const TFunc&& ExecuteFunction)
	{
		BoundQuery->ForEachEntityChunk(ExecutionContext, [&Accessors, &ExecuteFunction](FMassExecutionContext& Context)
		{
			Accessors.AccessorTuple.ApplyBefore([&Context](auto&... Accessor)
			{
				(Accessor.SetupForChunk(Context), ...);
			});
			ExecuteFunction(Context, Accessors);
		});
	}

	template<typename TAccessors, typename TFunc>
	inline void ParallelForEachEntityChunk(FMassExecutionContext& ExecutionContext, const TAccessors& Accessors, const TFunc&& ExecuteFunction)
	{
		BoundQuery->ParallelForEachEntityChunk(ExecutionContext, [&Accessors, &ExecuteFunction](FMassExecutionContext& Context)
		{
			TAccessors LocalAccessors = TAccessors(Accessors);

			LocalAccessors.AccessorTuple.ApplyBefore([&Context](auto&... Accessor)
			{
				(Accessor.SetupForChunk(Context), ...);
			});

			ExecuteFunction(Context, LocalAccessors);
		});
	}

	template<typename TAccessors, typename TFunc>
	inline void ForEachEntity(FMassExecutionContext& ExecutionContext, TAccessors& Accessors, const TFunc&& ExecuteFunction)
	{
		BoundQuery->ForEachEntityChunk(ExecutionContext, [&Accessors, &ExecuteFunction](FMassExecutionContext& Context)
		{
			Accessors.AccessorTuple.ApplyBefore([&Context](auto&... Accessor)
			{
				(Accessor.SetupForChunk(Context), ...);
			});

			for (uint32 EntityIndex : Context.CreateEntityIterator())
			{
				ExecuteFunction(Context, Accessors, EntityIndex);
			}
		});
	}

	template<typename TAccessors, typename TFunc>
	inline void ParallelForEachEntity(FMassExecutionContext& ExecutionContext, TAccessors& Accessors, const TFunc&& ExecuteFunction)
	{
		BoundQuery->ParallelForEachEntityChunk(ExecutionContext, [&Accessors, &ExecuteFunction](FMassExecutionContext& Context)
		{
			TAccessors LocalAccessors = TAccessors(Accessors);
			LocalAccessors.AccessorTuple.ApplyBefore([&Context](auto&... Accessor)
			{
				(Accessor.SetupForChunk(Context), ...);
			});

			for (uint32 EntityIndex : Context.CreateEntityIterator())
			{
				ExecuteFunction(Context, LocalAccessors, EntityIndex);
			}
		});
	}

	UE_API FQueryExecutor();
private:
	static UE_API FMassEntityQuery DummyQuery;

	UE_API void ConfigureQuery(FMassSubsystemRequirements& ProcessorRequirements);
	UE_API void CallExecute(FMassExecutionContext& Context);

	TNotNull<FMassEntityQuery*> BoundQuery;
	TWeakObjectPtr<UObject> LogOwner;

	/** AccessorsPtr is only allowed to point to a member variable of this struct and will assert in all other cases. */
	FQueryDefinitionBase* AccessorsPtr;

#if WITH_MASSENTITY_DEBUG
	UE_API void ValidateAccessors();
	uint32 DebugSize = 0;
#endif
};

template <typename...>
inline constexpr auto IsUnique = std::true_type{};

template <typename T, typename... Rest>
inline constexpr auto IsUnique<T, Rest...> = std::bool_constant<(!std::is_same_v<typename T::FragmentType, typename Rest::FragmentType> && ...) && IsUnique<Rest...>>{};


template <typename T, typename U, typename... Us>
constexpr auto GetIndexRecursive()
{
	if constexpr (std::is_same<T, typename U::FragmentType>::value)
	{
		return 0;
	}
	else
	{
		static_assert(sizeof...(Us) > 0, "Type not found in accessor collection.");
		return 1 + GetIndexRecursive<T, Us...>();
	}
}

template <typename T, typename U, typename... Us>
constexpr auto GetAccessorIndex()
{
	static_assert(IsUnique<U, Us...>, "An accessor collection must only contain a single instance of each fragment/subsystem/tag type.");
	return GetIndexRecursive<T, U, Us...>();
}

/**
 * Defines the entity compositions to return in the query and provides type-safe data access to
 * entity and subsystem data. Must be a member variable of a QueryExecutor
 */
template<typename... Accessors>
struct FQueryDefinition : public FQueryDefinitionBase
{
	FQueryDefinition(FQueryExecutor& Owner)
	{
		Owner.AccessorsPtr = this;
	}

	TTuple<Accessors...> AccessorTuple{};

	virtual void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) override
	{
		AccessorTuple.ApplyBefore([&EntityQuery, &ProcessorRequirements](auto&... Accessor)
		{
			(Accessor.ConfigureQuery(EntityQuery, ProcessorRequirements), ...);
		});

	}

	virtual void SetupForExecute(FMassExecutionContext& Context) override
	{
		AccessorTuple.ApplyBefore([&Context](auto&... Accessor)
		{
			(Accessor.SetupForExecute(Context), ...);
		});
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
		AccessorTuple.ApplyBefore([&Context](auto&... Accessor)
		{
			(Accessor.SetupForChunk(Context), ...);
		});
	}

	template <typename TFragment>
	constexpr auto& Get()
	{
		constexpr std::size_t Index = GetAccessorIndex<TFragment, Accessors...>();
		return AccessorTuple.template Get<Index>().Get();
	}
};

template<typename TFragment>
struct FMutableFragmentAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TFragment;

	FMutableFragmentAccess() = default;

	inline TArrayView<TFragment>& Get()
	{
		return View;
	}

	inline TFragment& operator[](int32 Index)
	{
		return View[Index];
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements)
	{
		EntityQuery.AddRequirement<TFragment>(EMassFragmentAccess::ReadWrite);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
		View = Context.GetMutableFragmentView<TFragment>();
	}

	TArrayView<TFragment> View;
};

template<typename TFragment>
struct FConstFragmentAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TFragment;

	FConstFragmentAccess() = default;

	inline const TConstArrayView<TFragment>& Get() const
	{
		return View;
	}

	inline const TFragment& operator[](int32 Index) const
	{
		return View[Index];
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		EntityQuery.AddRequirement<TFragment>(EMassFragmentAccess::ReadOnly);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
		View = Context.GetFragmentView<TFragment>();
	}

	TConstArrayView<TFragment> View;
};

template<typename TFragment>
struct FMutableOptionalFragmentAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TFragment;

	FMutableOptionalFragmentAccess() = default;

	inline TArrayView<TFragment>& Get()
	{
		return View;
	}

	inline TFragment& operator[](int32 Index)
	{
		return View[Index];
	}

	inline int32 Num() const
	{
		return View.Num();
	}

	inline operator bool() const
	{
		return View.Num() > 0;
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		EntityQuery.AddRequirement<TFragment>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
		View = Context.GetMutableFragmentView<TFragment>();
	}

	TArrayView<TFragment> View;
};

template<typename TFragment>
struct FConstOptionalFragmentAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TFragment;

	FConstOptionalFragmentAccess() = default;

	inline const TConstArrayView<TFragment>& Get() const
	{
		return View;
	}

	inline const TFragment& operator[](int32 Index) const
	{
		return View[Index];
	}

	inline int32 Num() const
	{
		return View.Num();
	}

	inline operator bool() const
	{
		return View.Num() > 0;
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		EntityQuery.AddRequirement<TFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
		View = Context.GetFragmentView<TFragment>();
	}

	TConstArrayView<TFragment> View;
};

template<typename TTag>
struct FMassTagRequired
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TTag;

	FMassTagRequired() = default;

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		EntityQuery.AddTagRequirement<TTag>(EMassFragmentPresence::All);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
	}
};

template<typename TFragment>
struct FMassTagBlocked
{
	using FragmentType = TFragment;

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		EntityQuery.AddTagRequirement<TFragment>(EMassFragmentPresence::None);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
	}
};

template<typename TFragment>
struct FMutableSharedFragmentAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TFragment;

	FMutableSharedFragmentAccess() = default;

	inline TFragment& Get() const
	{
		check(Fragment);
		return *Fragment;
	}

	inline TFragment& operator*() const
	{
		check(Fragment);
		return *Fragment;
	}

	inline TFragment* operator->() const
	{
		return Fragment;
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		EntityQuery.AddSharedRequirement<TFragment>(EMassFragmentAccess::ReadWrite);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
		Fragment = &(Context.GetMutableSharedFragment<TFragment>());
	}

	TFragment* Fragment = nullptr;
};

template<typename TFragment>
struct FConstSharedFragmentAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TFragment;

	FConstSharedFragmentAccess() = default;

	inline const TFragment& Get() const
	{
		check(Fragment);
		return *Fragment;
	}

	inline const TFragment& operator*() const
	{
		check(Fragment);
		return *Fragment;
	}

	inline const TFragment* operator->() const
	{
		return Fragment;
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		EntityQuery.AddConstSharedRequirement<TFragment>();
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
		Fragment = &Context.GetConstSharedFragment<TFragment>();
	}

	const TFragment* Fragment = nullptr;
};

template<typename TFragment>
struct FMutableChunkFragmentAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TFragment;

	FMutableChunkFragmentAccess() = default;

	inline TFragment& Get() const
	{
		check(Fragment);
		return *Fragment;
	}

	inline TFragment& operator*() const
	{
		check(Fragment);
		return *Fragment;
	}

	inline TFragment* operator->() const
	{
		return Fragment;
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		EntityQuery.AddChunkRequirement<TFragment>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
		Fragment = Context.GetMutableChunkFragmentPtr<TFragment>();
	}

	TFragment* Fragment = nullptr;
};

template<typename TFragment>
struct FConstChunkFragmentAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TFragment;

	FConstChunkFragmentAccess() = default;

	inline const TFragment& Get() const
	{
		check(Fragment);
		return *Fragment;
	}

	inline const TFragment& operator*() const
	{
		check(Fragment);
		return *Fragment;
	}

	inline const TFragment* operator->() const
	{
		return Fragment;
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		EntityQuery.AddChunkRequirement<TFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
		Fragment = Context.GetChunkFragmentPtr<TFragment>();
	}

	const TFragment* Fragment = nullptr;
};

template<typename TFragment>
struct FMutableOptionalChunkFragmentAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TFragment;

	FMutableOptionalChunkFragmentAccess() = default;

	inline TFragment* Get() const
	{
		check(Fragment);
		return Fragment;
	}

	inline TFragment& operator*() const
	{
		check(Fragment);
		return *Fragment;
	}

	inline TFragment* operator->() const
	{
		return Fragment;
	}

	inline operator bool()
	{
		return Fragment != nullptr;
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		EntityQuery.AddChunkRequirement<TFragment>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
		Fragment = Context.GetMutableChunkFragmentPtr<TFragment>();
	}

	TFragment* Fragment = nullptr;
};

template<typename TFragment>
struct FConstOptionalChunkFragmentAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TFragment;

	FConstOptionalChunkFragmentAccess() = default;

	inline const TFragment* Get()
	{
		check(Fragment);
		return Fragment;
	}

	inline const TFragment& operator*() const
	{
		check(Fragment);
		return *Fragment;
	}

	inline const TFragment* operator->() const
	{
		return Fragment;
	}

	inline operator bool() const
	{
		return Fragment != nullptr;
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		EntityQuery.AddChunkRequirement<TFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
		Fragment = Context.GetChunkFragmentPtr<TFragment>();
	}

	const TFragment* Fragment = nullptr;
};

template<typename TSubsystem, typename = typename TEnableIf<TIsDerivedFrom<TSubsystem, USubsystem>::IsDerived>::Type>
struct FMutableSubsystemAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TSubsystem;

	FMutableSubsystemAccess() = default;

	TSubsystem& Get() const
	{
		check(Subsystem);
		return *Subsystem;
	}

	inline TSubsystem& operator*() const
	{
		check(Subsystem);
		return *Subsystem;
	}

	inline TSubsystem* operator->() const
	{
		return Subsystem;
	}

	inline operator bool() const
	{
		return Subsystem != nullptr;
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		ProcessorRequirements.AddSubsystemRequirement<TSubsystem>(EMassFragmentAccess::ReadWrite);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
		Subsystem = Context.GetMutableSubsystem<TSubsystem>();
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
	}

	TSubsystem* Subsystem = nullptr;
};

template<typename TSubsystem, typename = typename TEnableIf<TIsDerivedFrom<TSubsystem, USubsystem>::IsDerived>::Type>
struct FConstSubsystemAccess
{
	template<typename... Ts>
	friend struct FQueryDefinition;

	using FragmentType = TSubsystem;

	FConstSubsystemAccess() = default;

	const TSubsystem& Get() const
	{
		check(Subsystem);
		return *Subsystem;
	}

	inline const TSubsystem& operator*() const
	{
		check(Subsystem);
		return *Subsystem;
	}

	inline const TSubsystem* operator->() const
	{
		return Subsystem;
	}

	inline operator bool() const
	{
		return Subsystem != nullptr;
	}

	inline void ConfigureQuery(FMassEntityQuery& EntityQuery, FMassSubsystemRequirements& ProcessorRequirements) const
	{
		ProcessorRequirements.AddSubsystemRequirement<TSubsystem>(EMassFragmentAccess::ReadOnly);
	}

	inline void SetupForExecute(FMassExecutionContext& Context)
	{
		Subsystem = Context.GetSubsystem<TSubsystem>();
	}

	inline void SetupForChunk(FMassExecutionContext& Context)
	{
	}

	const TSubsystem* Subsystem = nullptr;
};


} // namespace UE::Mass

#undef UE_API
