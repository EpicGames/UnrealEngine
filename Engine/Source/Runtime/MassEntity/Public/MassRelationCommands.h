// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommands.h"
#include "MassEntityManager.h"

#if CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG
#	define DEBUG_NAME(Name) , FName(TEXT(Name))
#	define DEBUG_NAME_PARAM(Name) , const FName InDebugName = TEXT(Name)
#	define FORWARD_DEBUG_NAME_PARAM , InDebugName
#else
#	define DEBUG_NAME(Name)
#	define DEBUG_NAME_PARAM(Name)
#	define FORWARD_DEBUG_NAME_PARAM
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

template<typename TRelation>
struct FMassCommandMakeRelation : public FMassBatchedEntityCommand
{
	using Super = FMassBatchedEntityCommand;

	FMassCommandMakeRelation(EMassCommandOperationType OperationType = EMassCommandOperationType::Add DEBUG_NAME_PARAM("MakeRelation"))
		: Super(OperationType FORWARD_DEBUG_NAME_PARAM)
	{}

	void Add(const FMassEntityHandle ChildEntity, const FMassEntityHandle ParentEntity)
	{
		Super::Add(ChildEntity);
		Parents.Add(ParentEntity);
	}

	void Add(TConstArrayView<FMassEntityHandle> ChildEntities, TConstArrayView<FMassEntityHandle> ParentEntities)
	{
		ensure(ChildEntities.Num() != 0 && ParentEntities.Num() != 0);

		Super::Add(ChildEntities);

		if (ChildEntities.Num() > ParentEntities.Num())
		{
			// @todo to be improved - we should have a dedicated path for Multi-children -> Single parent  operations
			do
			{
				Parents.Append(ParentEntities.GetData(), ParentEntities.Num());
			} while (Parents.Num() < TargetEntities.Num());
			Parents.SetNum(TargetEntities.Num(), EAllowShrinking::No);
		}
		else
		{
			Parents.Append(ParentEntities.GetData(), FMath::Min(ParentEntities.Num(), ChildEntities.Num()));
		}
	}

protected:
	virtual void Reset() override
	{
		Parents.Reset();
		Super::Reset();
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return Super::GetAllocatedSize() + Parents.GetAllocatedSize();
	}

	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMassCommandMakeRelation_Execute);
		EntityManager.BatchCreateRelations<TRelation>(TargetEntities, Parents);
	}

	TArray<FMassEntityHandle> Parents;
};

#undef DEBUG_NAME
#undef DEBUG_NAME_PARAM
#undef FORWARD_DEBUG_NAME_PARAM
