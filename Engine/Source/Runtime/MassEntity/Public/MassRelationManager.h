// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"
#include "MassTypeManager.h"
#include "Mass/EntityHandle.h"
#include "MassEntityRelations.h"

#define UE_API MASSENTITY_API

struct FMassEntityHandle;
struct FMassEntityManager;
struct FMassExecutionContext;

namespace UE::Mass
{
	struct FTypeRegistry;
	struct FRelationManager;
	
	struct FRelationData
	{
		FRelationData(const FRelationTypeTraits& InTraits);

		const FRelationTypeTraits Traits;

		struct FRelationInstanceData
		{
			TStaticArray<TArray<FMassRelationRoleInstanceHandle>, static_cast<uint32>(ERelationRole::MAX)> RelationEntityContainers;

			bool IsEmpty() const
			{
				bool bIsEmpty = true;
				for (const TArray<FMassRelationRoleInstanceHandle>& Container : RelationEntityContainers)
				{
					bIsEmpty = bIsEmpty && Container.IsEmpty();
				}
				return bIsEmpty;
			}

			TArray<FMassRelationRoleInstanceHandle>& operator[](const int32 Index)
			{
				return RelationEntityContainers[Index];
			}

			const TArray<FMassRelationRoleInstanceHandle>& operator[](const int32 Index) const
			{
				return RelationEntityContainers[Index];
			}

			TArray<FMassRelationRoleInstanceHandle>& operator[](const ERelationRole Index)
			{
				return (*this)[static_cast<int32>(Index)];
			}

			const TArray<FMassRelationRoleInstanceHandle>& operator[](const ERelationRole Index) const
			{
				return (*this)[static_cast<int32>(Index)];
			}
		};
		TMap<FMassEntityHandle, FRelationInstanceData> RoleMap;

		TArray<FMassEntityHandle> GetParticipants(const FMassEntityManager& EntityManager, const FMassEntityHandle RoleEntity, ERelationRole QueriedRole) const;

		/** Returns true if RoleEntity has at least one valid participant in the queried role. No allocation. */
		bool HasParticipants(const FMassEntityManager& EntityManager, FMassEntityHandle RoleEntity, ERelationRole QueriedRole) const;

		/** Callback-based enumeration of participants. Same iteration as GetParticipants but invokes a callback per valid entity instead of building a TArray. */
		void ForEachParticipant(const FMassEntityManager& EntityManager, FMassEntityHandle RoleEntity,
			ERelationRole QueriedRole, TFunctionRef<void(FMassEntityHandle)> Callback) const;
	};


	struct FRelationManager
	{
		UE_API explicit FRelationManager(FMassEntityManager& EntityManager);

		template<UE::Mass::CRelation T>
		FMassEntityHandle CreateRelationInstance(FMassEntityHandle Subject, FMassEntityHandle Object);
		FMassEntityHandle CreateRelationInstance(const FTypeHandle RelationTypeHandle, FMassEntityHandle Subject, FMassEntityHandle Object);

		/** Creates a relation type handle with RelationType, and calls the other CreateRelationInstances implementation */
		TArray<FMassEntityHandle> CreateRelationInstances(TNotNull<const UScriptStruct*> RelationType, TArrayView<FMassEntityHandle> Subjects, TArrayView<FMassEntityHandle> Objects);

		/**
		 * Creates valid relation instances of type RelationTypeHandle, binding Subjects and Objects
		 * Note that the input arrays can have their order modified by the function, all the relation pairs that are not valid, are moved to the back of the arrays
		 * The number of elements in Subjects and Objects must match.
		 */
		UE_API TArray<FMassEntityHandle> CreateRelationInstances(const FTypeHandle RelationTypeHandle, TArrayView<FMassEntityHandle> Subjects, TArrayView<FMassEntityHandle> Objects);

		UE_API bool DestroyRelationInstance(const FTypeHandle RelationTypeHandle, const FMassEntityHandle Subject, const FMassEntityHandle Object);
		UE_API bool DestroyRelationInstance(FMassRelationRoleInstanceHandle RelationHandle) const;

		/** Fetch all the entities that are "subjects" in instances of the given relation type, where ObjectEntity is the "object" of the relation */
		[[nodiscard]] TArray<FMassEntityHandle> GetRelationSubjects(TNotNull<const UScriptStruct*> RelationType, const FMassEntityHandle ObjectEntity) const;
		[[nodiscard]] TArray<FMassEntityHandle> GetRelationSubjects(const FTypeHandle RelationTypeHandle, const FMassEntityHandle ObjectEntity) const;

		/** Fetch all the entities that are "objects" in instances of the given relation type, where SubjectEntity is the "subject" of the relation */
		[[nodiscard]] TArray<FMassEntityHandle> GetRelationObjects(TNotNull<const UScriptStruct*> RelationType, const FMassEntityHandle SubjectEntity) const;
		[[nodiscard]] TArray<FMassEntityHandle> GetRelationObjects(const FTypeHandle RelationTypeHandle, const FMassEntityHandle SubjectEntity) const;

		[[nodiscard]] TArray<FMassEntityHandle> GetRelationEntities(TConstArrayView<FMassRelationRoleInstanceHandle> RelationEntitiesContainer) const;
		void GetRelationEntities(TConstArrayView<FMassRelationRoleInstanceHandle> RelationEntitiesContainer, TArray<FMassEntityHandle>& InOutEntityHandles) const;
		[[nodiscard]] TArray<FMassEntityHandle> GetRoleEntities(TConstArrayView<FMassRelationRoleInstanceHandle> RelationEntitiesContainer) const;
		void GetRoleEntities(TConstArrayView<FMassRelationRoleInstanceHandle> RelationEntitiesContainer, TArray<FMassEntityHandle>& InOutEntityHandles) const;

		FTypeHandle GetRelationTypeHandle(TNotNull<const UScriptStruct*> RelationType) const;

		/** Callback-based enumeration of subjects for a given object in a relation. Zero allocation. */
		UE_API void ForEachRelationSubject(const FTypeHandle RelationTypeHandle, FMassEntityHandle ObjectEntity,
			TFunctionRef<void(FMassEntityHandle)> Callback) const;

		/** Callback-based enumeration of objects for a given subject in a relation. Zero allocation. */
		UE_API void ForEachRelationObject(const FTypeHandle RelationTypeHandle, FMassEntityHandle SubjectEntity,
			TFunctionRef<void(FMassEntityHandle)> Callback) const;

		/** Returns true if ObjectEntity has at least one subject in this relation type. No allocation. */
		UE_API bool HasRelationSubjects(const FTypeHandle RelationTypeHandle, FMassEntityHandle ObjectEntity) const;

		/** Returns true if SubjectEntity has at least one object in this relation type. No allocation. */
		UE_API bool HasRelationObjects(const FTypeHandle RelationTypeHandle, FMassEntityHandle SubjectEntity) const;

		UE_API bool IsSubjectOfRelation(const FTypeHandle RelationTypeHandle, const FMassEntityHandle Subject, const FMassEntityHandle Object) const;
		UE_API bool IsSubjectOfRelation(const FRelationData& RelationDataInstance, const FMassEntityHandle Subject, const FMassEntityHandle Object) const;
		UE_API bool IsSubjectOfRelationRecursive(const FTypeHandle RelationTypeHandle, const FMassEntityHandle Subject, const FMassEntityHandle Object) const;
		UE_API bool IsSubjectOfRelationRecursive(const FRelationData& RelationDataInstance, const FMassEntityHandle Subject, const FMassEntityHandle Object) const;


		void OnRelationTypeRegistered(const FTypeHandle RegisteredTypeHandle, const FRelationTypeTraits& RelationTypeTraits);

		const FRelationData& GetRelationDataChecked(const FTypeHandle RelationTypeHandle) const;
		FRelationData& GetRelationDataChecked(const FTypeHandle RelationTypeHandle);

	protected:

		FRelationData& CreateRelationData(const FTypeHandle RelationTypeHandle);

		UE_API TArray<FMassEntityHandle> GetRelationObjects(const FRelationData& RelationData, const FMassEntityHandle SubjectEntity) const;
		UE_API TArray<FMassEntityHandle> GetRelationSubjects(const FRelationData& RelationData, const FMassEntityHandle ObjectEntity) const;

		const FRelationData* GetRelationData(const FTypeHandle RelationTypeHandle) const;

		struct FHierarchyEntitiesContainer
		{
			void StoreUnique(const uint32 Depth, TArray<FMassEntityHandle>& InOutArray);
			void StoreUnique(const uint32 Depth, FMassEntityHandle Handle);

			TArray<FMassEntityHandle>& operator[](const uint32 Depth)
			{
				return ContainerPerLevel[Depth];
			}
			const TArray<FMassEntityHandle>& operator[](const uint32 Depth) const
			{
				return ContainerPerLevel[Depth];
			}

			uint32 Num() const
			{
				return static_cast<uint32>(ContainerPerLevel.Num());
			}
		protected:
			TArray<TArray<FMassEntityHandle>> ContainerPerLevel;
			TSet<FMassEntityHandle> ExistingElements;
		};

		void GatherHierarchy(const FRelationData& RelationData, const FMassEntityHandle SubjectHandle, FHierarchyEntitiesContainer& SubSubjects, uint32 Depth = 0) const;
		
		FMassEntityManager& EntityManager;
		const FTypeManager& TypeManager;

		TMap<FTypeHandle, FRelationData> RelationsDataMap;
	};

	//-----------------------------------------------------------------------------
	// INLINES
	//-----------------------------------------------------------------------------
	template<UE::Mass::CRelation T>
	FMassEntityHandle FRelationManager::CreateRelationInstance(FMassEntityHandle Subject, FMassEntityHandle Object)
	{
		TArray<FMassEntityHandle> CreatedRelationshipEntities = CreateRelationInstances(T::StaticStruct(), TArrayView<FMassEntityHandle>(&Subject, 1), TArrayView<FMassEntityHandle>(&Object, 1));
		return CreatedRelationshipEntities.IsEmpty() ? FMassEntityHandle() : CreatedRelationshipEntities[0];
	}

	inline FMassEntityHandle FRelationManager::CreateRelationInstance(const FTypeHandle RelationTypeHandle, FMassEntityHandle Subject, FMassEntityHandle Object)
	{
		TArray<FMassEntityHandle> CreatedRelationshipEntities = CreateRelationInstances(RelationTypeHandle, TArrayView<FMassEntityHandle>(&Subject, 1), TArrayView<FMassEntityHandle>(&Object, 1));
		return CreatedRelationshipEntities.IsEmpty() ? FMassEntityHandle() : CreatedRelationshipEntities[0];
	}

	inline TArray<FMassEntityHandle> FRelationManager::CreateRelationInstances(TNotNull<const UScriptStruct*> RelationType, TArrayView<FMassEntityHandle> Subjects, TArrayView<FMassEntityHandle> Objects)
	{
		const FTypeHandle RelationTypeHandle = TypeManager.GetRelationTypeHandle(RelationType);
		if (!ensureMsgf(RelationTypeHandle.IsValid(), TEXT("%hs: Unknown relation type %s, make sure to register it first"), __FUNCTION__, *RelationType->GetName()))
		{
			return {};
		}

		return CreateRelationInstances(RelationTypeHandle, Subjects, Objects);
	}

	inline FTypeHandle FRelationManager::GetRelationTypeHandle(TNotNull<const UScriptStruct*> RelationType) const
	{
		checkf(UE::Mass::IsA<FMassRelation>(RelationType)
			, TEXT("Provided RelationType, %s, is not a relation type")
			, *RelationType->GetName());

		return TypeManager.GetRelationTypeHandle(RelationType);
	}
	
	inline const FRelationData* FRelationManager::GetRelationData(const FTypeHandle RelationTypeHandle) const
	{
		return RelationsDataMap.Find(RelationTypeHandle);
	}

	inline const FRelationData& FRelationManager::GetRelationDataChecked(const FTypeHandle RelationTypeHandle) const
	{
		return RelationsDataMap.FindChecked(RelationTypeHandle);
	}

	inline FRelationData& FRelationManager::GetRelationDataChecked(const FTypeHandle RelationTypeHandle)
	{
		return RelationsDataMap.FindChecked(RelationTypeHandle);
	}

	inline TArray<FMassEntityHandle> FRelationManager::GetRelationSubjects(TNotNull<const UScriptStruct*> RelationType, const FMassEntityHandle ObjectEntity) const
	{
		const FTypeHandle RelationTypeHandle = TypeManager.GetRelationTypeHandle(RelationType);
		return GetRelationSubjects(RelationTypeHandle, ObjectEntity);
	}

	inline TArray<FMassEntityHandle> FRelationManager::GetRelationSubjects(const FTypeHandle RelationTypeHandle, const FMassEntityHandle ObjectEntity) const
	{
		const FRelationData& RelationData = GetRelationDataChecked(RelationTypeHandle);
		return GetRelationSubjects(RelationData, ObjectEntity);
	}

	inline TArray<FMassEntityHandle> FRelationManager::GetRelationObjects(TNotNull<const UScriptStruct*> RelationType, const FMassEntityHandle SubjectEntity) const
	{
		const FTypeHandle RelationTypeHandle = TypeManager.GetRelationTypeHandle(RelationType);
		return GetRelationObjects(RelationTypeHandle, SubjectEntity);
	}

	inline TArray<FMassEntityHandle> FRelationManager::GetRelationObjects(const FTypeHandle RelationTypeHandle, const FMassEntityHandle SubjectEntity) const
	{
		const FRelationData& RelationData = GetRelationDataChecked(RelationTypeHandle);
		return GetRelationObjects(RelationData, SubjectEntity);
	}

} // namespace UE::Mass

#undef UE_API
