// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassObserverProcessor.h"
#include "MassEntityQuery.h"
#include "MassRelationObservers.generated.h"

#define UE_API MASSENTITY_API

namespace UE::Mass::Relations
{
	struct FRelationTypeTraits;
	enum class ERelationRole : uint8;
} // namespace UE::Mass::Relations

UCLASS(MinimalAPI)
class UMassRelationObserver : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassRelationObserver();

	/**
	 * @return whether the configuration was successful. Failed observers won't get added to the observer manager and will get destroyed promptly.
	 */
	UE_API virtual bool ConfigureRelationObserver(UE::Mass::FTypeHandle InRegisteredTypeHandle, const UE::Mass::FRelationTypeTraits& Traits);

protected:
	UE_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	FMassEntityQuery EntityQuery;
	UE::Mass::FTypeHandle RelationTypeHandle;
	/** Relevant only when the primary observed type is a fragment */
	EMassFragmentAccess ObservedTypeAccess = EMassFragmentAccess::ReadWrite;
	EMassFragmentAccess RelationFragmentAccessType = EMassFragmentAccess::ReadWrite;
	bool bAutoAddRelationFragmentRequirement = true;
	bool bAutoAddRelationTagRequirement = true;

	FString DebugDescription;
};

UCLASS(MinimalAPI)
class UMassRelationEntityCreation : public UMassRelationObserver
{
	GENERATED_BODY()
public:
	static constexpr int32 RelationCreationObserverExecutionPriority = 1024;

	UE_API UMassRelationEntityCreation();

protected:
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};

/** Debug-time, detects relation elements being removed from the relation entities, which is not supposed to be done */
UCLASS(MinimalAPI)
class UMassRelationEntityGuardDog : public UMassRelationObserver
{
	GENERATED_BODY()
public:
	UE_API UMassRelationEntityGuardDog();

protected:
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};

/** cleans up data, removes enties in RoleMap if Entry.IsEmpty */
UCLASS(MinimalAPI)
class UMassRelationEntityDestruction : public UMassRelationObserver
{
	GENERATED_BODY()
public:
	UE_API UMassRelationEntityDestruction();

protected:
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};

UCLASS(MinimalAPI)
class UMassRelationRoleDestruction : public UMassRelationObserver
{
	GENERATED_BODY()

public:
	UE_API UMassRelationRoleDestruction();

	UE_API virtual bool ConfigureRelationObserver(UE::Mass::FTypeHandle InRegisteredTypeHandle, const UE::Mass::FRelationTypeTraits& Traits) override;
	static UE_API void AddObserverInstances(FMassObserverManager& GetObserverManager, UE::Mass::FTypeHandle InRegisteredTypeHandle, const UE::Mass::FRelationTypeTraits& Traits);

protected:
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	FMassExecuteFunction ExecuteFunction;

	/**
	 * indicates which side of the relation the given destructor handles. MAX indicates both sides.
	 */
	UE::Mass::Relations::ERelationRole RelationRole = UE::Mass::Relations::ERelationRole::MAX;

	/**
	 * indicates the relation-specific fragment, that characterizes relation-entities. We use this
	 * information to filer these entities out - this observer is meant to only handle "role" entities
	 */
	const UScriptStruct* ExcludedRelationFragmentType = nullptr;
};

#undef UE_API
