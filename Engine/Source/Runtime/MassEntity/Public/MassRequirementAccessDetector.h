// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessingTypes.h"
#if WITH_MASSENTITY_DEBUG
#include "Containers/ArrayView.h"
#include "MassEntityQuery.h"
#include "Misc/MTAccessDetector.h"


struct FMassEntityManager;

struct FMassRequirementAccessDetector
{
	MASSENTITY_API void Initialize();
	MASSENTITY_API void RequireAccess(const FMassEntityQuery& Query);
	MASSENTITY_API void ReleaseAccess(const FMassEntityQuery& Query);

private:
	using FDetectorMethod = bool (FRWAccessDetector::*)() const;
	template<typename TBitSet>
	void Operation(const TBitSet& BitSet, FDetectorMethod Op)
	{
		TArray<const UStruct*> Types;
		BitSet.ExportTypes(Types);
		for (const UStruct* Type : Types)
		{
			if (TSharedRef<FRWAccessDetector>* Detector = Detectors.Find(Type))
			{
				FRWAccessDetector& DetectorRef = Detector->Get();
				(DetectorRef.*Op)();
			}
		}
	}

	void Acquire(TConstArrayView<FMassFragmentRequirementDescription> Requirements)
	{
		for (const FMassFragmentRequirementDescription& Req : Requirements)
		{
			if (Req.Presence != EMassFragmentPresence::None)
			{
				if (Req.AccessMode == EMassFragmentAccess::ReadWrite)
				{
					Detectors.Find(Req.StructType)->Get().AcquireWriteAccess();
				}
				else if (Req.AccessMode == EMassFragmentAccess::ReadOnly)
				{
					Detectors.Find(Req.StructType)->Get().AcquireReadAccess();
				}
			}
		}
	}

	void Release(TConstArrayView<FMassFragmentRequirementDescription> Requirements)
	{
		for (const FMassFragmentRequirementDescription& Req : Requirements)
		{
			if (Req.Presence != EMassFragmentPresence::None)
			{
				if (Req.AccessMode == EMassFragmentAccess::ReadWrite)
				{
					Detectors.Find(Req.StructType)->Get().ReleaseWriteAccess();
				}
				else if (Req.AccessMode == EMassFragmentAccess::ReadOnly)
				{
					Detectors.Find(Req.StructType)->Get().ReleaseReadAccess();
				}
			}
		}
	}

	/** @Note the function is not thread-safe and meant to be only called internally on game thread (see FMassRequirementAccessDetector::Initialize) */
	void AddDetectors(const FStructTracker& StructTracker);

	TMap<const UStruct*, TSharedRef<FRWAccessDetector>> Detectors;
};

#else
struct FMassEntityQuery;
#endif // WITH_MASSENTITY_DEBUG

namespace UE::Mass::Debug
{
	struct FScopedRequirementAccessDetector
	{
#if WITH_MASSENTITY_DEBUG
		MASSENTITY_API FScopedRequirementAccessDetector(const FMassEntityQuery& InQuery);
		MASSENTITY_API ~FScopedRequirementAccessDetector();

		TSharedPtr<FMassEntityManager> EntityManager;
		const FMassEntityQuery& Query;
#else
		FScopedRequirementAccessDetector(const FMassEntityQuery&)
		{
		}
#endif // WITH_MASSENTITY_DEBUG
	};
} // namespace UE::Mass::Debug
