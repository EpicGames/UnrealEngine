// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "MassEntityLinkFragments.h"
#include "MassArchetypeTypes.h"
#include "Templates/SubclassOf.h"
#include "MassRequirements.generated.h"

struct FMassDebugger;
struct FMassArchetypeHandle;
struct FMassExecutionRequirements;
struct FMassRequirementAccessDetector;
class USubsystem;

namespace UE::Mass::ErrorReporting
{
extern FOutputDevice& GetOutputDevice();

/**
 * Helper struct that can be used to set up a threadsafe global output device
 * so nested calls can use UE::Mass::ErrorReporting::GetOutputDevice() to report
 * error messages.
 */
struct FScopedOutputDevice
{
	explicit FScopedOutputDevice(FOutputDevice& OutputDevice);
	~FScopedOutputDevice();

private:
	FOutputDevice* ActiveOutputDevice = nullptr;
};

} // namespace UE::Mass::ErrorReporting

UENUM()
enum class EMassFragmentAccess : uint8
{
	/** no binding required */
	None, 

	/** We want to read the data from the fragment */
	ReadOnly,

	/** We want to read and write the data for the fragment */
	ReadWrite,

	MAX
};

UENUM()
enum class EMassFragmentPresence : uint8
{
	/** All the required fragments must be present */
	All,

	/** One of the required fragments must be present */
	Any,

	/** None of the required fragments can be present */
	None,

	/** If fragment is present we'll use it */
	Optional,

	MAX
};


struct FMassFragmentRequirementDescription
{
	FMassFragmentRequirementDescription() = default;
	FMassFragmentRequirementDescription(const UScriptStruct* InStruct, const EMassFragmentAccess InAccessMode, const EMassFragmentPresence InPresence);

	bool RequiresBinding() const;
	bool IsOptional() const;

	/** these functions are used for sorting. See FScriptStructSortOperator */
	int32 GetStructureSize() const;

	FName GetFName() const;

	const UScriptStruct* StructType = nullptr;
	EMassFragmentAccess AccessMode = EMassFragmentAccess::None;
	EMassFragmentPresence Presence = EMassFragmentPresence::Optional;
};

/**
 *  FMassSubsystemRequirements is a structure that declares runtime subsystem access type given calculations require.
 */
struct FMassSubsystemRequirements
{

	friend FMassDebugger;
	friend FMassRequirementAccessDetector;

	template<typename T>
	FMassSubsystemRequirements& AddSubsystemRequirement(const EMassFragmentAccess AccessMode)
	{
		check(AccessMode != EMassFragmentAccess::None && AccessMode != EMassFragmentAccess::MAX);

		// Compilation errors here like: 'GameThreadOnly': is not a member of 'TMassExternalSubsystemTraits<USmartObjectSubsystem>
		// indicate that there is a missing header that defines the subsystem's trait or that you need to define one for that subsystem type.
		// @see "Mass/ExternalSubsystemTraits.h" for details

		switch (AccessMode)
		{
		case EMassFragmentAccess::ReadOnly:
			RequiredConstSubsystems.Add<T>();
			bRequiresGameThreadExecution |= TMassExternalSubsystemTraits<T>::GameThreadOnly;
			break;
		case EMassFragmentAccess::ReadWrite:
			RequiredMutableSubsystems.Add<T>();
			bRequiresGameThreadExecution |= TMassExternalSubsystemTraits<T>::GameThreadOnly;
			break;
		default:
			check(false);
		}

		return *this;
	}

	FMassSubsystemRequirements& AddSubsystemRequirement(const TSubclassOf<USubsystem> SubsystemClass, const EMassFragmentAccess AccessMode, const bool bGameThreadOnly)
	{
		check(AccessMode != EMassFragmentAccess::None && AccessMode != EMassFragmentAccess::MAX);

		switch (AccessMode)
		{
		case EMassFragmentAccess::ReadOnly:
			RequiredConstSubsystems.Add(*SubsystemClass);
			bRequiresGameThreadExecution |= bGameThreadOnly;
			break;
		case EMassFragmentAccess::ReadWrite:
			RequiredMutableSubsystems.Add(*SubsystemClass);
			bRequiresGameThreadExecution |= bGameThreadOnly;
			break;
		default:
			check(false);
		}

		return *this;
	}

	FMassSubsystemRequirements& AddSubsystemRequirement(const TSubclassOf<USubsystem> SubsystemClass, const EMassFragmentAccess AccessMode, const TSharedRef<FMassEntityManager>& EntityManager)
	{
		return AddSubsystemRequirement(SubsystemClass, AccessMode, IsGameThreadOnlySubsystem(SubsystemClass, EntityManager));
	}

	UE_DEPRECATED(5.6, "This flavor of FMassSubsystemRequirements::AddSubsystemRequirement is deprecated. Use one of the other flavors, or call FMassEntityQuery::AddSubsystemRequirement if applicable.")
	FMassSubsystemRequirements& AddSubsystemRequirement(const TSubclassOf<USubsystem> SubsystemClass, const EMassFragmentAccess AccessMode)
	{
		return AddSubsystemRequirement(SubsystemClass, AccessMode, /*bGameThreadOnly=*/true);
	}

	MASSENTITY_API void Reset();

	const FMassExternalSubsystemBitSet& GetRequiredConstSubsystems() const;
	const FMassExternalSubsystemBitSet& GetRequiredMutableSubsystems() const;
	bool IsEmpty() const;

	bool DoesRequireGameThreadExecution() const;
	MASSENTITY_API void ExportRequirements(FMassExecutionRequirements& OutRequirements) const;

	friend uint32 GetTypeHash(const FMassSubsystemRequirements& Instance);

protected:
	MASSENTITY_API static bool IsGameThreadOnlySubsystem(const TSubclassOf<USubsystem> SubsystemClass, const TSharedRef<FMassEntityManager>& EntityManager);

	FMassExternalSubsystemBitSet RequiredConstSubsystems;
	FMassExternalSubsystemBitSet RequiredMutableSubsystems;

private:
	bool bRequiresGameThreadExecution = false;
};

/** 
 *  FMassFragmentRequirements is a structure that describes properties required of an archetype that's a subject of calculations.
 */
struct FMassFragmentRequirements
{
	friend FMassDebugger;
	friend FMassRequirementAccessDetector;

	FMassFragmentRequirements() = default;
	MASSENTITY_API explicit FMassFragmentRequirements(const TSharedPtr<FMassEntityManager>& EntityManager);
	MASSENTITY_API explicit FMassFragmentRequirements(const TSharedRef<FMassEntityManager>& EntityManager);

	MASSENTITY_API void Initialize(const TSharedRef<FMassEntityManager>& EntityManager);

	FMassFragmentRequirements& AddElementRequirement(TNotNull<const UScriptStruct*> ElementType, const EMassFragmentAccess AccessMode, const EMassFragmentPresence Presence = EMassFragmentPresence::All)
	{
		if (UE::Mass::IsA<FMassFragment>(ElementType))
		{
			return AddRequirement(ElementType, AccessMode, Presence);
		}
		return AddTagRequirement(ElementType, Presence);
	}

	FMassFragmentRequirements& AddRequirement(const UScriptStruct* FragmentType, const EMassFragmentAccess AccessMode, const EMassFragmentPresence Presence = EMassFragmentPresence::All)
	{
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		check(FragmentType);

		if (UE::Mass::IsSparse(FragmentType))
		{
			AddSparseRequirement(FragmentType, Presence);
		}
		else
		{
			checkf(FragmentRequirements.FindByPredicate([FragmentType](const FMassFragmentRequirementDescription& Item) { return Item.StructType == FragmentType; }) == nullptr
				, TEXT("Duplicated requirements are not supported. %s already present"), *GetNameSafe(FragmentType));

			if (Presence != EMassFragmentPresence::None)
			{
				FragmentRequirements.Emplace(FragmentType, AccessMode, Presence);
			}

			switch (Presence)
			{
			case EMassFragmentPresence::All:
				RequiredAllFragments.Add(FragmentType);
				break;
			case EMassFragmentPresence::Any:
				RequiredAnyFragments.Add(FragmentType);
				break;
			case EMassFragmentPresence::Optional:
				RequiredOptionalFragments.Add(FragmentType);
				break;
			case EMassFragmentPresence::None:
				RequiredNoneFragments.Add(FragmentType);
				break;
			}
			// force recaching the next time this query is used or the following CacheArchetypes call.
			IncrementChangeCounter();
		}
		return *this;
	}

	/** FMassFragmentRequirements ref returned for chaining */
	template<typename T>
	FMassFragmentRequirements& AddRequirement(const EMassFragmentAccess AccessMode, const EMassFragmentPresence Presence = EMassFragmentPresence::All)
	{
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkf(FragmentRequirements.FindByPredicate([](const FMassFragmentRequirementDescription& Item) { return Item.StructType == T::StaticStruct(); }) == nullptr
			, TEXT("Duplicated requirements are not supported. %s already present"), *T::StaticStruct()->GetName());

		MASS_STATIC_CHECK_FRAGMENT(T);

		if constexpr (UE::Mass::CSparse<T>)
		{
			AddSparseRequirement(T::StaticStruct(), Presence);
		}
		else
		{
			if (Presence != EMassFragmentPresence::None)
			{
				FragmentRequirements.Emplace(T::StaticStruct(), AccessMode, Presence);
			}

			switch (Presence)
			{
			case EMassFragmentPresence::All:
				RequiredAllFragments.Add<T>();
				break;
			case EMassFragmentPresence::Any:
				RequiredAnyFragments.Add<T>();
				break;
			case EMassFragmentPresence::Optional:
				RequiredOptionalFragments.Add<T>();
				break;
			case EMassFragmentPresence::None:
				RequiredNoneFragments.Add<T>();
				break;
			}
			// force recaching the next time this query is used or the following CacheArchetypes call.
			IncrementChangeCounter();
		}
		return *this;
	}

	FMassFragmentRequirements& AddTagRequirement(TNotNull<const UScriptStruct*> TagType, const EMassFragmentPresence Presence)
	{
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkf(static_cast<int32>(Presence) != static_cast<int32>(EMassFragmentPresence::MAX), TEXT("MAX presence is not a valid value for AddTagRequirement"));

		if (UE::Mass::IsSparse(TagType))
		{
			AddSparseRequirement(TagType, Presence);
		}
		else
		{
			switch (Presence)
			{
			case EMassFragmentPresence::All:
				RequiredAllTags.Add(TagType);
				break;
			case EMassFragmentPresence::Any:
				RequiredAnyTags.Add(TagType);
				break;
			case EMassFragmentPresence::None:
				RequiredNoneTags.Add(TagType);
				break;
			case EMassFragmentPresence::Optional:
				RequiredOptionalTags.Add(TagType);
				break;
			}
			IncrementChangeCounter();
		}
		return *this;
	}

	void AddTagRequirement(const UScriptStruct& TagType, const EMassFragmentPresence Presence)
	{
		AddTagRequirement(&TagType, Presence);
	}

	template<typename T>
	FMassFragmentRequirements& AddTagRequirement(const EMassFragmentPresence Presence)
	{
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkf(static_cast<int32>(Presence) != static_cast<int32>(EMassFragmentPresence::MAX), TEXT("MAX presence is not a valid value for AddTagRequirement"));

		static_assert(UE::Mass::CTag<T>, "Given struct doesn't represent a valid tag type. Make sure to inherit from FMassTag or one of its child-types.");

		if constexpr (UE::Mass::CSparse<T>)
		{
			AddSparseRequirement(T::StaticStruct(), Presence);
		}
		else 
		{
			switch (Presence)
			{
			case EMassFragmentPresence::All:
				RequiredAllTags.Add<T>();
				break;
			case EMassFragmentPresence::Any:
				RequiredAnyTags.Add<T>();
				break;
			case EMassFragmentPresence::None:
				RequiredNoneTags.Add<T>();
				break;
			case EMassFragmentPresence::Optional:
				RequiredOptionalTags.Add<T>();
				break;
			}
			IncrementChangeCounter();
		}
		return *this;
	}

	/** actual implementation in specializations */
	template<EMassFragmentPresence Presence>
	FMassFragmentRequirements& AddTagRequirements(const FMassTagBitSet& TagBitSet)
	{
		static_assert(Presence == EMassFragmentPresence::None || Presence == EMassFragmentPresence::All || Presence == EMassFragmentPresence::Any
			, "The only valid values for AddTagRequirements are All, Any and None");
		return *this;
	}

	/** Clears given tags out of all collected requirements, including negative ones */
	MASSENTITY_API FMassFragmentRequirements& ClearTagRequirements(const FMassTagBitSet& TagsToRemoveBitSet);

	/**
	 * Adds given sparse element type as a requirement.
	 */
	template<UE::Mass::CSparse T>
	FMassFragmentRequirements& AddSparseRequirement(const EMassFragmentPresence Presence = EMassFragmentPresence::All)
	{
		check(Presence != EMassFragmentPresence::MAX);
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));

		switch (Presence)
		{
		case EMassFragmentPresence::All:
			RequiredAllSparseElements.Add<T>();
			break;
		case EMassFragmentPresence::Any:
			RequiredAnySparseElements.Add<T>();
			break;
		case EMassFragmentPresence::None:
			RequiredNoneSparseElements.Add<T>();
			break;
		case EMassFragmentPresence::Optional:
			RequiredOptionalSparseElements.Add<T>();
			break;
		}

		IncrementChangeCounter();
		return *this;
	}

	/**
	 * Adds given sparse element type as a requirement.
	 */
	FMassFragmentRequirements& AddSparseRequirement(TNotNull<const UScriptStruct*> ElementType, const EMassFragmentPresence Presence = EMassFragmentPresence::All)
	{
		check(Presence != EMassFragmentPresence::MAX);
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkf(UE::Mass::IsSparse(ElementType), TEXT("%s doesn't represent a valid sparse element type. Make sure to inherit from FMassSparseFragment or FMassSparseTag."), *ElementType->GetName());

		switch (Presence)
		{
		case EMassFragmentPresence::All:
			RequiredAllSparseElements.Add(ElementType);
			break;
		case EMassFragmentPresence::Any:
			RequiredAnySparseElements.Add(ElementType);
			break;
		case EMassFragmentPresence::None:
			RequiredNoneSparseElements.Add(ElementType);
			break;
		case EMassFragmentPresence::Optional:
			RequiredOptionalSparseElements.Add(ElementType);
			break;
		}

		IncrementChangeCounter();
		return *this;
	}

	template<typename T>
	FMassFragmentRequirements& AddChunkRequirement(const EMassFragmentAccess AccessMode, const EMassFragmentPresence Presence = EMassFragmentPresence::All)
	{
		static_assert(UE::Mass::CChunkFragment<T>, "Given struct doesn't represent a valid chunk fragment type. Make sure to inherit from FMassChunkFragment or one of its child-types.");
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkf(ChunkFragmentRequirements.FindByPredicate([](const FMassFragmentRequirementDescription& Item) { return Item.StructType == T::StaticStruct(); }) == nullptr
			, TEXT("Duplicated requirements are not supported. %s already present"), *T::StaticStruct()->GetName());
		checkf(Presence != EMassFragmentPresence::Any, TEXT("\'Any\' is not a valid Presence value for AddChunkRequirement."));

		switch (Presence)
		{
		case EMassFragmentPresence::All:
			RequiredAllChunkFragments.Add<T>();
			ChunkFragmentRequirements.Emplace(T::StaticStruct(), AccessMode, Presence);
			break;
		case EMassFragmentPresence::Optional:
			RequiredOptionalChunkFragments.Add<T>();
			ChunkFragmentRequirements.Emplace(T::StaticStruct(), AccessMode, Presence);
			break;
		case EMassFragmentPresence::None:
			RequiredNoneChunkFragments.Add<T>();
			break;
		}
		IncrementChangeCounter();
		return *this;
	}

	FMassFragmentRequirements& AddChunkRequirement(TNotNull<const UScriptStruct*> ChunkFragmentType, const EMassFragmentAccess AccessMode, const EMassFragmentPresence Presence = EMassFragmentPresence::All)
	{
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkf(ChunkFragmentRequirements.FindByPredicate([&ChunkFragmentType](const FMassFragmentRequirementDescription& Item) { return Item.StructType == ChunkFragmentType; }) == nullptr
			, TEXT("Duplicated requirements are not supported. %s already present"), *ChunkFragmentType->GetName());
		checkf(Presence != EMassFragmentPresence::Any, TEXT("\'Any\' is not a valid Presence value for AddChunkRequirement."));

		switch (Presence)
		{
		case EMassFragmentPresence::All:
			RequiredAllChunkFragments.Add(ChunkFragmentType);
			ChunkFragmentRequirements.Emplace(ChunkFragmentType, AccessMode, Presence);
			break;
		case EMassFragmentPresence::Optional:
			RequiredOptionalChunkFragments.Add(ChunkFragmentType);
			ChunkFragmentRequirements.Emplace(ChunkFragmentType, AccessMode, Presence);
			break;
		case EMassFragmentPresence::None:
			RequiredNoneChunkFragments.Add(ChunkFragmentType);
			break;
		}
		IncrementChangeCounter();
		return *this;
	}

	template<typename T>
	FMassFragmentRequirements& AddConstSharedRequirement(const EMassFragmentPresence Presence = EMassFragmentPresence::All)
	{
		static_assert(UE::Mass::CConstSharedFragment<T>, "Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.");
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkf(ConstSharedFragmentRequirements.FindByPredicate([](const FMassFragmentRequirementDescription& Item) { return Item.StructType == T::StaticStruct(); }) == nullptr
			, TEXT("Duplicated requirements are not supported. %s already present"), *T::StaticStruct()->GetName());
		checkf(Presence != EMassFragmentPresence::Any, TEXT("\'Any\' is not a valid Presence value for AddConstSharedRequirement."));

		switch (Presence)
		{
		case EMassFragmentPresence::All:
			RequiredAllConstSharedFragments.Add<T>();
			ConstSharedFragmentRequirements.Emplace(T::StaticStruct(), EMassFragmentAccess::ReadOnly, Presence);
			break;
		case EMassFragmentPresence::Optional:
			RequiredOptionalConstSharedFragments.Add<T>();
			ConstSharedFragmentRequirements.Emplace(T::StaticStruct(), EMassFragmentAccess::ReadOnly, Presence);
			break;
		case EMassFragmentPresence::None:
			RequiredNoneConstSharedFragments.Add<T>();
			break;
		}
		IncrementChangeCounter();
		return *this;
	}

	FMassFragmentRequirements& AddConstSharedRequirement(const UScriptStruct* FragmentType, const EMassFragmentPresence Presence = EMassFragmentPresence::All)
	{
		if (!ensureMsgf(UE::Mass::IsA<FMassConstSharedFragment>(FragmentType)
			, TEXT("Given struct doesn't represent a valid const shared fragment type. Make sure to inherit from FMassConstSharedFragment or one of its child-types.")))
		{
			return *this;
		}

		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkf(ConstSharedFragmentRequirements.FindByPredicate([FragmentType](const FMassFragmentRequirementDescription& Item) { return Item.StructType == FragmentType; }) == nullptr
			, TEXT("Duplicated requirements are not supported. %s already present"), *FragmentType->GetName());
		checkf(Presence != EMassFragmentPresence::Any, TEXT("\'Any\' is not a valid Presence value for AddConstSharedRequirement."));

		switch (Presence)
		{
		case EMassFragmentPresence::All:
			RequiredAllConstSharedFragments.Add(FragmentType);
			ConstSharedFragmentRequirements.Emplace(FragmentType, EMassFragmentAccess::ReadOnly, Presence);
			break;
		case EMassFragmentPresence::Optional:
			RequiredOptionalConstSharedFragments.Add(FragmentType);
			ConstSharedFragmentRequirements.Emplace(FragmentType, EMassFragmentAccess::ReadOnly, Presence);
			break;
		case EMassFragmentPresence::None:
			RequiredNoneConstSharedFragments.Add(FragmentType);
			break;
		}
		IncrementChangeCounter();
		return *this;
	}

	template<typename T>
	FMassFragmentRequirements& AddSharedRequirement(const EMassFragmentAccess AccessMode, const EMassFragmentPresence Presence = EMassFragmentPresence::All)
	{
		static_assert(UE::Mass::CSharedFragment<T>, "Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types.");
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkf(SharedFragmentRequirements.FindByPredicate([](const FMassFragmentRequirementDescription& Item) { return Item.StructType == T::StaticStruct(); }) == nullptr
			, TEXT("Duplicated requirements are not supported. %s already present"), *T::StaticStruct()->GetName());
		checkf(Presence != EMassFragmentPresence::Any, TEXT("\'Any\' is not a valid Presence value for AddSharedRequirement."));

		switch (Presence)
		{
		case EMassFragmentPresence::All:
			RequiredAllSharedFragments.Add<T>();
			SharedFragmentRequirements.Emplace(T::StaticStruct(), AccessMode, Presence);
			if (AccessMode == EMassFragmentAccess::ReadWrite)
			{
				bRequiresGameThreadExecution |= TMassSharedFragmentTraits<T>::GameThreadOnly;
			}
			break;
		case EMassFragmentPresence::Optional:
			RequiredOptionalSharedFragments.Add<T>();
			SharedFragmentRequirements.Emplace(T::StaticStruct(), AccessMode, Presence);
			if (AccessMode == EMassFragmentAccess::ReadWrite)
			{
				bRequiresGameThreadExecution |= TMassSharedFragmentTraits<T>::GameThreadOnly;
			}
			break;
		case EMassFragmentPresence::None:
			RequiredNoneSharedFragments.Add<T>();
			break;
		}
		IncrementChangeCounter();
		return *this;
	}

	FMassFragmentRequirements& AddSharedRequirement(TNotNull<const UScriptStruct*> SharedFragmentType, const EMassFragmentAccess AccessMode, const EMassFragmentPresence Presence = EMassFragmentPresence::All)
	{
		checkf(UE::Mass::IsA<FMassSharedFragment>(SharedFragmentType), TEXT("Given struct doesn't represent a valid shared fragment type. Make sure to inherit from FMassSharedFragment or one of its child-types."));
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkf(SharedFragmentRequirements.FindByPredicate([&SharedFragmentType](const FMassFragmentRequirementDescription& Item) { return Item.StructType == SharedFragmentType; }) == nullptr
			, TEXT("Duplicated requirements are not supported. %s already present"), *SharedFragmentType->GetName());
		checkf(Presence != EMassFragmentPresence::Any, TEXT("\'Any\' is not a valid Presence value for AddSharedRequirement."));

		switch (Presence)
		{
		case EMassFragmentPresence::All:
			RequiredAllSharedFragments.Add(SharedFragmentType);
			SharedFragmentRequirements.Emplace(SharedFragmentType, AccessMode, Presence);
			if (AccessMode == EMassFragmentAccess::ReadWrite)
			{
				bRequiresGameThreadExecution |= IsGameThreadOnlySharedFragment(SharedFragmentType);
			}
			break;
		case EMassFragmentPresence::Optional:
			RequiredOptionalSharedFragments.Add(SharedFragmentType);
			SharedFragmentRequirements.Emplace(SharedFragmentType, AccessMode, Presence);
			if (AccessMode == EMassFragmentAccess::ReadWrite)
			{
				bRequiresGameThreadExecution |= IsGameThreadOnlySharedFragment(SharedFragmentType);
			}
			break;
		case EMassFragmentPresence::None:
			RequiredNoneSharedFragments.Add(SharedFragmentType);
			break;
		}
		IncrementChangeCounter();
		return *this;
	}

	/**
	 * Adds an access requirement for a fragment on a linked entity.
	 * This is a specialization of Indirect fragment access that requires the target entity be "Linked"
	 * To link an entity, add a FMassEntityLinkFragment to the entities you want to link from and set the LinkedEntityHandle to the target entity.
	 * FMassEntityLinkFragment can be inherited to support specific link types and multiple linked entities.
	 * Linking entities this way will group all entities that link to the target entity in memory for better cache behavior when iterating through query results.
	 */
	FMassFragmentRequirements& AddLinkedEntityRequirement(TNotNull<const UScriptStruct*> FragmentType, const EMassFragmentAccess AccessMode, TNotNull<const UScriptStruct*> LinkType = FMassEntityLinkFragment::StaticStruct())
	{
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkf(UE::Mass::IsA<FMassFragment>(FragmentType), TEXT("Fragment must inherit FMassFragment."));
		checkfSlow(LinkedEntityFragmentRequirements.FindByPredicate([FragmentType](const FMassFragmentRequirementDescription& Item) { return Item.StructType == FragmentType; }) == nullptr
			, TEXT("Duplicated linked entity requirements are not supported. %s already present"), *GetNameSafe(FragmentType));

		const bool bHasLinkType = 
			ConstSharedFragmentRequirements.ContainsByPredicate([LinkType](const FMassFragmentRequirementDescription& Item)
			{
				return Item.StructType == LinkType;
			});

		if (!bHasLinkType)
		{
			checkfSlow(ConstSharedFragmentRequirements.ContainsByPredicate([](const FMassFragmentRequirementDescription& Item)
				{
					return Item.StructType->IsChildOf(FMassEntityLinkFragment::StaticStruct());
				}) == false
				, TEXT("A query can not use more than one Entity Link type. Use Indirect Fragment access instead, or separate queries."));

			AddConstSharedRequirement(LinkType);
		}


		LinkedEntityFragmentRequirements.Emplace(FragmentType, AccessMode, EMassFragmentPresence::Any);

		switch (AccessMode)
		{
		case EMassFragmentAccess::ReadOnly:
			IndirectReadOnlyFragments.Add(FragmentType);
			break;
		case EMassFragmentAccess::ReadWrite:
			IndirectReadWriteFragments.Add(FragmentType);
			break;
		}

		// force recaching the next time this query is used or the following CacheArchetypes call.
		IncrementChangeCounter();
		return *this;
	}

	/**
	 * Adds an access requirement for a fragment on a linked entity.
	 * This is a specialization of Indirect fragment access that requires the target entity be "Linked"
	 * To link an entity, add a FMassEntityLinkFragment to the entities you want to link from and set the LinkedEntityHandle to the target entity.
	 * FMassEntityLinkFragment can be inherited to support specific link types and multiple linked entities.
	 * Linking entities this way will group all entities that link to the target entity in memory for better cache behavior when iterating through query results.
	 */
	template<UE::Mass::CFragment T, typename TLinkFragment = FMassEntityLinkFragment>
	FMassFragmentRequirements& AddLinkedEntityRequirement(const EMassFragmentAccess AccessMode)
	{
		MASS_STATIC_CHECK_FRAGMENT(T);
		return AddLinkedEntityRequirement(T::StaticStruct(), AccessMode, TLinkFragment::StaticStruct());
	}

	template<UE::Mass::CFragment T>
	bool HasConstIndirectEntityRequirement()
	{
		const int32 TypeIndex = FMassFragmentBitSet::GetTypeIndex<T>();
		return IndirectReadOnlyFragments.IsBitSet(TypeIndex) || IndirectReadWriteFragments.IsBitSet(TypeIndex);
	}

	template<UE::Mass::CFragment T>
	bool HasMutableIndirectEntityRequirement()
	{
		return IndirectReadWriteFragments.IsBitSet(FMassFragmentBitSet::GetTypeIndex<T>());
	}

	/**
	 * Adds an access requirement for a fragment on another entity.
	 * This is used to enforce access requirements in the dependency solver, not to discover entities/archetypes.
	 * Use FMassExecutionContext::GetIndirectFragmentPtr to access fragments indirectly in your processor.
	 */
	FMassFragmentRequirements& AddIndirectFragmentRequirement(TNotNull<const UScriptStruct*> FragmentType, const EMassFragmentAccess AccessMode)
	{
		checkf(bInitialized, TEXT("Modifying requirements before initialization is not supported."));
		checkfSlow(UE::Mass::IsA<FMassFragment>(FragmentType), TEXT("Fragment must inherit FMassFragment."));

		switch (AccessMode)
		{
		case EMassFragmentAccess::ReadOnly:
			IndirectReadOnlyFragments.Add(FragmentType);
			break;
		case EMassFragmentAccess::ReadWrite:
			IndirectReadWriteFragments.Add(FragmentType);
			break;
		}

		// force recaching the next time this query is used or the following CacheArchetypes call.
		IncrementChangeCounter();
		return *this;
	}

	/**
	 * Adds an access requirement for a fragment on another entity.
	 * This is used to enforce access requirements in the dependency solver, not to discover entities/archetypes.
	 * Use FMassExecutionContext::GetIndirectFragmentPtr to access fragments indirectly in your processor.
	 */
	template<UE::Mass::CFragment T>
	FMassFragmentRequirements& AddIndirectFragmentRequirement(const EMassFragmentAccess AccessMode)
	{
		MASS_STATIC_CHECK_FRAGMENT(T);
		return AddIndirectFragmentRequirement(T::StaticStruct(), AccessMode);
	}

	MASSENTITY_API void Reset();

	/** 
	 * The function validates requirements we make for queries. See the FMassFragmentRequirements struct description for details.
	 * Even though the code of the function is non-trivial the consecutive calls will be essentially free due to the result 
	 * being cached (note that the caching gets invalidated if the composition changes).
	 * Note: sparse-elements-only requirements are not valid. Use FMassSparseElementIterator instead
	 * @return whether this query's requirements follow the rules.
	 */
	MASSENTITY_API bool CheckValidity() const;

	TConstArrayView<FMassFragmentRequirementDescription> GetFragmentRequirements() const;
	TConstArrayView<FMassFragmentRequirementDescription> GetChunkFragmentRequirements() const;
	TConstArrayView<FMassFragmentRequirementDescription> GetConstSharedFragmentRequirements() const;
	TConstArrayView<FMassFragmentRequirementDescription> GetSharedFragmentRequirements() const;
	const FMassFragmentBitSet& GetRequiredAllFragments() const;
	const FMassFragmentBitSet& GetRequiredAnyFragments() const;
	const FMassFragmentBitSet& GetRequiredOptionalFragments() const;
	const FMassFragmentBitSet& GetRequiredNoneFragments() const;
	const FMassFragmentBitSet& GetIndirectReadOnlyFragments() const;
	const FMassFragmentBitSet& GetIndirectReadWriteFragments() const;
	const FMassTagBitSet& GetRequiredAllTags() const;
	const FMassTagBitSet& GetRequiredAnyTags() const;
	const FMassTagBitSet& GetRequiredNoneTags() const;
	const FMassTagBitSet& GetRequiredOptionalTags() const;
	const FMassChunkFragmentBitSet& GetRequiredAllChunkFragments() const;
	const FMassChunkFragmentBitSet& GetRequiredOptionalChunkFragments() const;
	const FMassChunkFragmentBitSet& GetRequiredNoneChunkFragments() const;
	const FMassSharedFragmentBitSet& GetRequiredAllSharedFragments() const;
	const FMassSharedFragmentBitSet& GetRequiredOptionalSharedFragments() const;
	const FMassSharedFragmentBitSet& GetRequiredNoneSharedFragments() const;
	const FMassConstSharedFragmentBitSet& GetRequiredAllConstSharedFragments() const;
	const FMassConstSharedFragmentBitSet& GetRequiredOptionalConstSharedFragments() const;
	const FMassConstSharedFragmentBitSet& GetRequiredNoneConstSharedFragments() const;
	const FMassElementBitSet& GetRequiredAllSparseElements() const;
	const FMassElementBitSet& GetRequiredAnySparseElements() const;
	const FMassElementBitSet& GetRequiredOptionalSparseElements() const;
	const FMassElementBitSet& GetRequiredNoneSparseElements() const;

	bool IsInitialized() const;
	MASSENTITY_API bool IsEmpty() const;
	bool HasPositiveRequirements() const;
	bool HasNegativeRequirements() const;
	bool HasOptionalRequirements() const;
	bool HasSparseRequirements() const;
	bool HasLinkedEntityRequirements() const;

	/**
	 * Note that DoesArchetypeMatchRequirements functions do not check sparse elements. Sparse elements are not
	 * a part of immutable composition of a given archetype and can change at runtime, and as such cannot be used
	 * for filtering archetypes when the results of that filtering get cached (like in FMassEntityQuery)
	 */
	MASSENTITY_API bool DoesArchetypeMatchRequirements(const FMassArchetypeHandle& ArchetypeHandle) const;
	MASSENTITY_API bool DoesArchetypeMatchRequirements(const FMassElementBitSet& ArchetypeCompositionBitSet) const;
	bool DoesArchetypeMatchRequirements(const FMassArchetypeCompositionDescriptor& ArchetypeComposition) const
	{
		return DoesArchetypeMatchRequirements(ArchetypeComposition.GetElementsBitSet());
	}
	MASSENTITY_API bool DoesMatchAnyOptionals(const FMassElementBitSet& ArchetypeCompositionBitSet) const;
	bool DoesMatchAnyOptionals(const FMassArchetypeCompositionDescriptor& ArchetypeComposition) const
	{
		return DoesMatchAnyOptionals(ArchetypeComposition.GetElementsBitSet());
	}

	bool DoesRequireGameThreadExecution() const;
	MASSENTITY_API void ExportRequirements(FMassExecutionRequirements& OutRequirements) const;

	MASSENTITY_API friend uint32 GetTypeHash(const FMassFragmentRequirements& Instance);

protected:
	MASSENTITY_API void SortRequirements();

	void IncrementChangeCounter();
	void ConsumeIncrementalChangesCount();
	bool HasIncrementalChanges() const;
	
	/**
	 * A helper function that passes the query over to CachedEntityManager.
	 * Main purpose is to have the implementation in cpp and not include the EntityManager header here
	 * @todo this function always returns True at the moment, proper implementation waiting for implementation of "type trait information" (WIP)
	 */
	MASSENTITY_API bool IsGameThreadOnlySharedFragment(TNotNull<const UScriptStruct*> SharedFragmentType) const;

	friend FMassRequirementAccessDetector;

	TArray<FMassFragmentRequirementDescription> FragmentRequirements;
	TArray<FMassFragmentRequirementDescription> ChunkFragmentRequirements;
	TArray<FMassFragmentRequirementDescription> ConstSharedFragmentRequirements;
	TArray<FMassFragmentRequirementDescription> SharedFragmentRequirements;
	FMassTagBitSet RequiredAllTags;
	FMassTagBitSet RequiredAnyTags;
	FMassTagBitSet RequiredNoneTags;
	/**
	 * note that optional tags have meaning only if there are no other strict requirements, i.e. everything is optional,
	 * so we're looking for anything matching any of the optionals (both tags as well as fragments).
	 */
	FMassTagBitSet RequiredOptionalTags;
	FMassFragmentBitSet RequiredAllFragments;
	FMassFragmentBitSet RequiredAnyFragments;
	FMassFragmentBitSet RequiredOptionalFragments;
	FMassFragmentBitSet RequiredNoneFragments;

	/**
	 * Sparse elements. These can only be tested at runtime - i.e. the results cannot be cached, since
	 * as the entities move between archetypes the individual archetypes may or may not contain entities with
	 * given sparse elements
	 */
	FMassElementBitSet RequiredAllSparseElements;
	FMassElementBitSet RequiredAnySparseElements;
	FMassElementBitSet RequiredOptionalSparseElements;
	FMassElementBitSet RequiredNoneSparseElements;

	FMassChunkFragmentBitSet RequiredAllChunkFragments;
	FMassChunkFragmentBitSet RequiredOptionalChunkFragments;
	FMassChunkFragmentBitSet RequiredNoneChunkFragments;
	FMassSharedFragmentBitSet RequiredAllSharedFragments;
	FMassSharedFragmentBitSet RequiredOptionalSharedFragments;
	FMassSharedFragmentBitSet RequiredNoneSharedFragments;
	FMassConstSharedFragmentBitSet RequiredAllConstSharedFragments;
	FMassConstSharedFragmentBitSet RequiredOptionalConstSharedFragments;
	FMassConstSharedFragmentBitSet RequiredNoneConstSharedFragments;

	/**
	 * Fragments that will be accessed on other entities along with this query.
	 * These do not affect which entities are discovered by this query, but 
	 * inform the dependency solver to avoid write contention on these fragments.
	 * Used by Linked entity access and Indirect entity access.
	 */
	FMassFragmentBitSet IndirectReadOnlyFragments;
	FMassFragmentBitSet IndirectReadWriteFragments;

	/**
	 * Requirements for fragments on Linked entities.
	 * Queries will only discover entities with a linked entity that meets these requirements.
	 */
	TArray<FMassFragmentRequirementDescription> LinkedEntityFragmentRequirements;

	TSharedPtr<FMassEntityManager> CachedEntityManager;

private:
	MASSENTITY_API void CacheProperties() const;
	mutable uint16 bPropertiesCached : 1 = false;
	mutable uint16 bHasPositiveRequirements : 1 = false;
	mutable uint16 bHasNegativeRequirements : 1 = false;
	/** 
	 * Indicates that the requirements specify only optional elements, which means any composition having any one of 
	 * the optional elements will be accepted. Note that RequiredNone* requirements are handled separately and if specified 
	 * still need to be satisfied.
	 */
	mutable uint16 bHasOptionalRequirements : 1 = false;
	mutable uint16 bHasSparseRequirements : 1 = false;
	
	uint16 bInitialized : 1 = false;
	uint16 IncrementalChangesCount = 0;

	bool bRequiresGameThreadExecution = false;

public:
	UE_DEPRECATED(5.6, "This type of FMassFragmentRequirements is no longer supported. Use one of the other constructors instead.")
	MASSENTITY_API FMassFragmentRequirements(std::initializer_list<UScriptStruct*> InitList);

	UE_DEPRECATED(5.6, "This type of FMassFragmentRequirements is no longer supported. Use one of the other constructors instead.")
	MASSENTITY_API FMassFragmentRequirements(TConstArrayView<const UScriptStruct*> InitList);
};

/**
 * Specifies entity archetypes that can be created.
 * Does NOT imply a filtering constraint; this is purely an access declaration.
 */
struct FMassEntityCreationRequirements
{
public:
	/** Add by exact composition */
	void AddCreatedArchetype(const FMassElementBitSet& InComposition)
	{
		Compositions.Add(InComposition);
	}

	void AddCreatedArchetype(FMassElementBitSet&& InComposition)
	{
		Compositions.Add(MoveTemp(InComposition));
	}

	/** Export as "write" access to all fragment types contained in the declared archetypes */
	MASSENTITY_API void ExportRequirements(FMassExecutionRequirements& OutRequirements) const;

	/** Populates ArchetypeHandles with all created entity archetypes (EntityManager creates those that don't exist) */
	MASSENTITY_API void ResolveArchetypes(const FMassEntityManager& EntityManager);

	/** Gets the cached array of archetype handles allowed */
	const TArray<FMassArchetypeHandle>& GetCreatedArchetypes()
	{
		return ArchetypeHandles;
	}

	/** Returns true if the required composition list containsthe provided composition */
	bool Contains(const FMassElementBitSet& Composition) const
	{
		return Compositions.ContainsByPredicate([&Composition](const FMassElementBitSet& ElementBitSet)
			{
				return ElementBitSet.IsEquivalent(Composition);
			});
	}

	/** Clear the list of created archetype handles to force an update */
	void ClearCreatedArchetypeHandles()
	{
		ArchetypeHandles.Reset();
	}

	void Reset()
	{
		ArchetypeHandles.Reset();
		Compositions.Reset();
	}

	bool IsEmpty() const
	{
		return Compositions.Num() == 0;
	}

	const TArray<FMassElementBitSet>& GetCompositions() const
	{
		return Compositions;
	}

	bool HasAny(const FMassEntityCreationRequirements& Other) const
	{
		if (!Compositions.IsEmpty())
		{
			for (const FMassElementBitSet& Composition : Compositions)
			{
				if (Other.Contains(Composition))
				{
					return true;
				}
			}
		}
		return false;
	}


private:
	TArray<FMassArchetypeHandle> ArchetypeHandles;
	TArray<FMassElementBitSet> Compositions;
};

//-----------------------------------------------------------------------------
// INLINE
//-----------------------------------------------------------------------------
inline FMassFragmentRequirementDescription::FMassFragmentRequirementDescription(const UScriptStruct* InStruct, const EMassFragmentAccess InAccessMode, const EMassFragmentPresence InPresence)
	: StructType(InStruct)
	, AccessMode(InAccessMode)
	, Presence(InPresence)
{
	check(InStruct);
}

inline bool FMassFragmentRequirementDescription::RequiresBinding() const
{
	return (AccessMode != EMassFragmentAccess::None);
}

inline bool FMassFragmentRequirementDescription::IsOptional() const
{
	return (Presence == EMassFragmentPresence::Optional || Presence == EMassFragmentPresence::Any);
}

inline int32 FMassFragmentRequirementDescription::GetStructureSize() const
{
	return StructType->GetStructureSize();
}

inline FName FMassFragmentRequirementDescription::GetFName() const
{
	return StructType->GetFName();
}

inline const FMassExternalSubsystemBitSet& FMassSubsystemRequirements::GetRequiredConstSubsystems() const
{
	return RequiredConstSubsystems;
}

inline const FMassExternalSubsystemBitSet& FMassSubsystemRequirements::GetRequiredMutableSubsystems() const
{
	return RequiredMutableSubsystems;
}

inline bool FMassSubsystemRequirements::IsEmpty() const
{
	return RequiredConstSubsystems.IsEmpty() && RequiredMutableSubsystems.IsEmpty();
}

inline bool FMassSubsystemRequirements::DoesRequireGameThreadExecution() const
{
	return bRequiresGameThreadExecution;
}

inline uint32 GetTypeHash(const FMassSubsystemRequirements& Instance)
{
	return HashCombine(GetTypeHash(Instance.RequiredConstSubsystems), GetTypeHash(Instance.RequiredMutableSubsystems));
}

template<>
inline FMassFragmentRequirements& FMassFragmentRequirements::AddTagRequirements<EMassFragmentPresence::All>(const FMassTagBitSet& TagBitSet)
{
	RequiredAllTags += TagBitSet;
	// force re-caching the next time this query is used or the following CacheArchetypes call.
	IncrementChangeCounter();
	return *this;
}

template<>
inline FMassFragmentRequirements& FMassFragmentRequirements::AddTagRequirements<EMassFragmentPresence::Any>(const FMassTagBitSet& TagBitSet)
{
	RequiredAnyTags += TagBitSet;
	// force re-caching the next time this query is used or the following CacheArchetypes call.
	IncrementChangeCounter();
	return *this;
}

template<>
inline FMassFragmentRequirements& FMassFragmentRequirements::AddTagRequirements<EMassFragmentPresence::None>(const FMassTagBitSet& TagBitSet)
{
	RequiredNoneTags += TagBitSet;
	// force re-caching the next time this query is used or the following CacheArchetypes call.
	IncrementChangeCounter();
	return *this;
}

template<>
inline FMassFragmentRequirements& FMassFragmentRequirements::AddTagRequirements<EMassFragmentPresence::Optional>(const FMassTagBitSet& TagBitSet)
{
	RequiredOptionalTags += TagBitSet;
	// force re-caching the next time this query is used or the following CacheArchetypes call.
	IncrementChangeCounter();
	return *this;
}

inline TConstArrayView<FMassFragmentRequirementDescription> FMassFragmentRequirements::GetFragmentRequirements() const
{ 
	return FragmentRequirements; 
}

inline TConstArrayView<FMassFragmentRequirementDescription> FMassFragmentRequirements::GetChunkFragmentRequirements() const
{ 
	return ChunkFragmentRequirements; 
}

inline TConstArrayView<FMassFragmentRequirementDescription> FMassFragmentRequirements::GetConstSharedFragmentRequirements() const
{ 
	return ConstSharedFragmentRequirements; 
}

inline TConstArrayView<FMassFragmentRequirementDescription> FMassFragmentRequirements::GetSharedFragmentRequirements() const
{ 
	return SharedFragmentRequirements; 
}

inline const FMassFragmentBitSet& FMassFragmentRequirements::GetRequiredAllFragments() const
{ 
	return RequiredAllFragments; 
}

inline const FMassFragmentBitSet& FMassFragmentRequirements::GetRequiredAnyFragments() const
{ 
	return RequiredAnyFragments; 
}

inline const FMassFragmentBitSet& FMassFragmentRequirements::GetRequiredOptionalFragments() const
{ 
	return RequiredOptionalFragments; 
}

inline const FMassFragmentBitSet& FMassFragmentRequirements::GetRequiredNoneFragments() const
{ 
	return RequiredNoneFragments; 
}

inline const FMassTagBitSet& FMassFragmentRequirements::GetRequiredAllTags() const
{ 
	return RequiredAllTags; 
}

inline const FMassTagBitSet& FMassFragmentRequirements::GetRequiredAnyTags() const
{ 
	return RequiredAnyTags; 
}

inline const FMassTagBitSet& FMassFragmentRequirements::GetRequiredNoneTags() const
{ 
	return RequiredNoneTags; 
}

inline const FMassTagBitSet& FMassFragmentRequirements::GetRequiredOptionalTags() const
{ 
	return RequiredOptionalTags; 
}

inline const FMassChunkFragmentBitSet& FMassFragmentRequirements::GetRequiredAllChunkFragments() const
{ 
	return RequiredAllChunkFragments; 
}

inline const FMassChunkFragmentBitSet& FMassFragmentRequirements::GetRequiredOptionalChunkFragments() const
{ 
	return RequiredOptionalChunkFragments; 
}

inline const FMassChunkFragmentBitSet& FMassFragmentRequirements::GetRequiredNoneChunkFragments() const
{ 
	return RequiredNoneChunkFragments; 
}

inline const FMassSharedFragmentBitSet& FMassFragmentRequirements::GetRequiredAllSharedFragments() const
{ 
	return RequiredAllSharedFragments; 
}

inline const FMassSharedFragmentBitSet& FMassFragmentRequirements::GetRequiredOptionalSharedFragments() const
{ 
	return RequiredOptionalSharedFragments; 
}

inline const FMassSharedFragmentBitSet& FMassFragmentRequirements::GetRequiredNoneSharedFragments() const
{ 
	return RequiredNoneSharedFragments; 
}

inline const FMassConstSharedFragmentBitSet& FMassFragmentRequirements::GetRequiredAllConstSharedFragments() const
{ 
	return RequiredAllConstSharedFragments; 
}

inline const FMassConstSharedFragmentBitSet& FMassFragmentRequirements::GetRequiredOptionalConstSharedFragments() const
{ 
	return RequiredOptionalConstSharedFragments; 
}

inline const FMassConstSharedFragmentBitSet& FMassFragmentRequirements::GetRequiredNoneConstSharedFragments() const
{ 
	return RequiredNoneConstSharedFragments; 
}

inline const FMassElementBitSet& FMassFragmentRequirements::GetRequiredAllSparseElements() const 
{ 
	return RequiredAllSparseElements; 
}

inline const FMassElementBitSet& FMassFragmentRequirements::GetRequiredAnySparseElements() const 
{ 
	return RequiredAnySparseElements; 
}

inline const FMassElementBitSet& FMassFragmentRequirements::GetRequiredOptionalSparseElements() const 
{ 
	return RequiredOptionalSparseElements; 
}

inline const FMassElementBitSet& FMassFragmentRequirements::GetRequiredNoneSparseElements() const 
{ 
	return RequiredNoneSparseElements; 
}

inline const FMassFragmentBitSet& FMassFragmentRequirements::GetIndirectReadOnlyFragments() const
{
	return IndirectReadOnlyFragments;
}

inline const FMassFragmentBitSet& FMassFragmentRequirements::GetIndirectReadWriteFragments() const
{
	return IndirectReadWriteFragments;
}

inline bool FMassFragmentRequirements::IsInitialized() const 
{ 
	return bInitialized; 
}

inline bool FMassFragmentRequirements::HasPositiveRequirements() const 
{ 
	return bHasPositiveRequirements; 
}

inline bool FMassFragmentRequirements::HasNegativeRequirements() const 
{ 
	return bHasNegativeRequirements; 
}

inline bool FMassFragmentRequirements::HasOptionalRequirements() const 
{ 
	return bHasOptionalRequirements; 
}

inline bool FMassFragmentRequirements::HasSparseRequirements() const 
{ 
	return bHasSparseRequirements; 
}

inline bool FMassFragmentRequirements::HasLinkedEntityRequirements() const
{ 
	return !LinkedEntityFragmentRequirements.IsEmpty(); 
}

inline bool FMassFragmentRequirements::DoesRequireGameThreadExecution() const
{
	return bRequiresGameThreadExecution;
}

inline void FMassFragmentRequirements::IncrementChangeCounter()
{ 
	++IncrementalChangesCount; 
	bPropertiesCached = false;
}

inline void FMassFragmentRequirements::ConsumeIncrementalChangesCount()
{
	IncrementalChangesCount = 0;
}

inline bool FMassFragmentRequirements::HasIncrementalChanges() const
{
	return IncrementalChangesCount > 0;
}
