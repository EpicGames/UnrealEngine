// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "MassSettings.generated.h"

#define UE_API MASSENTITY_API


/** 
 * A common parent for Mass's per-module settings.
 * Classes extending this class will automatically get registered
 * with the Mass Settings and show under its section in Project Settings.
 */
UCLASS(MinimalAPI, Abstract, config = Mass, defaultconfig, collapseCategories)
class UMassModuleSettings : public UObject
{
	GENERATED_BODY()
protected:
	UE_API virtual void PostInitProperties() override;
};


UCLASS(MinimalAPI, config = Mass, defaultconfig, DisplayName = "Mass", AutoExpandCategories = "Mass")
class UMassSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API void RegisterModuleSettings(UMassModuleSettings& SettingsCDO);

public:
	UPROPERTY(VisibleAnywhere, Category = "Mass", NoClear, EditFixedSize, meta = (EditInline))
	TMap<FName, TObjectPtr<UMassModuleSettings>> ModuleSettings;
};

#undef UE_API
