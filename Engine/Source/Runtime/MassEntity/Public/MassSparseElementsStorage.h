// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformMath.h"
#include "Mass/EntityHandle.h"
#include "MassElement.h"
#include "StructUtils/StructView.h"
#include "MassEntityConcepts.h"
#include "UObject/ObjectKey.h"
#include "StructUtils/StructArrayView.h"

#define UE_API MASSENTITY_API

/**
 * Known limitations:
 * - doesn't support reloading ScriptStruct types
 */

struct FMassEntityManager;

namespace UE::Mass
{

struct FSparseElementIterator;

/**
 * Storage type for hosting instances of sparse elements.
 *
 * Sparse elements are optional tags and fragments that can be added to an entity without causing it to
 * change archetypes. Sparse elements are not part of Archetype's composition - instead, data on which
 * entity has which elements is stored per archetype chunk. Sparse tags live there (since they don't contain
 * any data per instance). Sparse fragment instances live in FSparseElementsStorage.
 *
 * FSparseElementsStorage provides a way iterate through all instances of a given sparse element type,
 * using UE::Mass::FSparseElementIterator. To create one call FMassEntityManager::CreateSparseElementsIterator
 *
 * Sparse fragment instances are stored in per-type pools (FSparseElementsStorage::FTypePool) which store data in
 * chunks (FSparseElementsStorage::FPackedStructDataChunk), and each chunk contains uniform array of instances
 * of the hosted struct type, along with a bitset indicating which elements are valid (i.e. "set").
 *
 * Do not use directly. The storage is to be accessed via FMassEntityManager and other means that forward their
 * calls to the entity manager.
 */
struct FSparseElementsStorage
{	
private:
	friend FSparseElementIterator;
	struct FTypePool;

public:
	struct FTypeConfig
	{
		FTypeConfig(TNotNull<const UScriptStruct*> InElementType)
			: AllocSizePerElement(InElementType->GetStructureSize())
			, ElementType(InElementType)
		{
		}
		FTypeConfig(const FTypeConfig&) = default;

		SIZE_T GetElementSize() const
		{
			return AllocSizePerElement;
		}

		const UScriptStruct* GetType() const 
		{
			return ElementType;
		}

		/**
		 * Converts down to a power of 2 and sets ElementsPerChunk and ChunkIndexShift
		 * @return actual value used for ElementsPerChunk
		 */
		uint32 SetElementsPerChunk(uint32 InElementsPerChunk)
		{
			constexpr uint32 MinElementsPerChunk = 1u << 5;
			constexpr uint32 MaxChunkShift = 32 - 5;
			InElementsPerChunk = FMath::Max(InElementsPerChunk, MinElementsPerChunk);
			ChunkIndexShift = FMath::FloorLog2(InElementsPerChunk);
			check(ChunkIndexShift < MaxChunkShift);
			ElementsPerChunk = (1u << ChunkIndexShift);
			ElementIndexMask = ElementsPerChunk - 1;
			return ElementsPerChunk;
		}

		uint32 GetChunkIndexShift() const 
		{ 
			return ChunkIndexShift; 
		}

		uint32 GetElementsPerChunk() const 
		{ 
			return ElementsPerChunk; 
		}

	private:
		friend FTypePool;
		FTypeConfig();
		static constexpr int32 DefaultChunkIndexShift = 7;
		uint32 ChunkIndexShift = DefaultChunkIndexShift;
		uint32 ElementsPerChunk = 1 << DefaultChunkIndexShift;
		uint32 ElementIndexMask = (1 << DefaultChunkIndexShift) - 1;
		const SIZE_T AllocSizePerElement = 0;
		TNotNull<const UScriptStruct*> ElementType;
	};

private:
	struct FPackedStructDataChunk
	{
		uint8* RawMemory = nullptr;
		TBitArray<> OccupationMask;
		uint32 NumElements = 0;

		FPackedStructDataChunk() = default;

		FPackedStructDataChunk(const FPackedStructDataChunk&) = delete;
		FPackedStructDataChunk& operator=(const FPackedStructDataChunk&) = delete;

		/** Move constructor required since the array holding FPackedStructDataChunk instances will get reallocated */
		FPackedStructDataChunk(FPackedStructDataChunk&& Other)
			: RawMemory(Other.RawMemory)
			, OccupationMask(MoveTemp(Other.OccupationMask))
			, NumElements(Other.NumElements)
		{
			Other.RawMemory = nullptr;
			Other.NumElements = 0;
		}

		FPackedStructDataChunk& operator=(FPackedStructDataChunk&& Other)
		{
			if (this != &Other) 
			{
				if (RawMemory) 
				{
					FMemory::Free(RawMemory);
				}

				RawMemory = Other.RawMemory;
				OccupationMask = MoveTemp(Other.OccupationMask);
				NumElements = Other.NumElements;
			
				Other.RawMemory = nullptr;
				Other.NumElements = 0;
			}
			return *this;
		}

		~FPackedStructDataChunk()
		{
			if (RawMemory != nullptr)
			{
				FMemory::Free(RawMemory);
				RawMemory = nullptr;
			}
		}

		void Init(const int32 MaxElements, const SIZE_T ElementSize)
		{
			OccupationMask.Init(false, MaxElements);
			RawMemory = static_cast<uint8*>(FMemory::Malloc(MaxElements * ElementSize));
		}

		bool IsInitialized() const
		{
			return RawMemory != nullptr;
		}

		bool HasElement(const uint32 ElementIndex)
		{
			checkSlow(OccupationMask.IsValidIndex(ElementIndex));
			return OccupationMask[ElementIndex];
		}

		/** ElementIndex is verified by the caller */
		uint8* GetElement(const uint32 ElementIndex, const SIZE_T ElementSize)
		{
			checkSlow(OccupationMask.IsValidIndex(ElementIndex));
			return OccupationMask[ElementIndex] ? &RawMemory[ElementSize * ElementIndex] : nullptr;
		}

		const uint8* GetElement(const uint32 ElementIndex, const SIZE_T ElementSize) const
		{
			checkSlow(OccupationMask.IsValidIndex(ElementIndex));
			return OccupationMask[ElementIndex] ? &RawMemory[ElementSize * ElementIndex] : nullptr;
		}

		/** returns uninitialized memory, to be initialized by the caller */
		uint8* Add(const uint32 ElementIndex, const SIZE_T ElementSize)
		{
			checkSlow(HasElement(ElementIndex) == false);
			OccupationMask[ElementIndex] = true;
			++NumElements;
			return &RawMemory[ElementSize * ElementIndex];
		}

		uint8* Remove(const uint32 ElementIndex, const SIZE_T ElementSize)
		{
			if (HasElement(ElementIndex))
			{
				OccupationMask[ElementIndex] = false;
				--NumElements;
				return &RawMemory[ElementSize * ElementIndex];
			}
			return nullptr;
		}
	private:
		friend FTypePool;
		uint8* GetElementUnsafe(const uint32 ElementIndex, const SIZE_T ElementSize) const
		{
			return &RawMemory[ElementSize * ElementIndex];
		}
	};

	struct FTypePool
	{
		FTypePool() = default;
		FTypePool(const FTypeConfig& InConfig)
			: Config(InConfig)
		{
		}
		UE_API ~FTypePool();

		TNotNull<uint8*> Add(uint32 EntityIndex);
		TNotNull<uint8*> Add(uint32 EntityIndex, TNotNull<const uint8*> SourceData);
		TNotNull<uint8*> AddMove(uint32 EntityIndex, TNotNull<uint8*> SourceData);
		FStructView Add_GetView(uint32 EntityIndex);
		bool Remove(uint32 EntityIndex);
		FConstStructView Get(uint32 EntityIndex) const;
		FStructView GetMutable(uint32 EntityIndex);
		FStructView GetOrCreateMutable(uint32 EntityIndex);

		uint32 GetNumElements() const
		{
			return NumElements;
		}

		UE_API static FTypePool& GetDummyPool();

	private:
		friend FSparseElementIterator;

		uint32 GetChunkIndex(const uint32 EntityIndex) const
		{
			return EntityIndex >> Config.ChunkIndexShift;
		}

		uint32 GetElementIndex(const uint32 EntityIndex) const
		{
			return EntityIndex & Config.ElementIndexMask;
		}

		FPackedStructDataChunk* GetPackedData(uint32 ChunkIndex);
		const FPackedStructDataChunk* GetPackedData(uint32 ChunkIndex) const;
		FPackedStructDataChunk& GetOrCreatePackedData(uint32 ChunkIndex);

		void GetChunkAndElementIndex(const uint32 EntityIndex, uint32& OutChunkIndex, uint32& OutElementIndex) const
		{
			OutChunkIndex = GetChunkIndex(EntityIndex);
			OutElementIndex = GetElementIndex(EntityIndex);
		}

		TArray<FPackedStructDataChunk> Chunks;
		/** stores total number of instances of the given sparse element type */
		uint32 NumElements = 0;
		const FTypeConfig Config;
	};

public:
	UE_API void Deinitialize();

	/**
	 * Creates and instance of FSparseElementIterator that can be used to iterate over all available
	 * instances of sparse fragment of type ElementType
	 */
	FSparseElementIterator CreateElementIterator(TNotNull<const UScriptStruct*> ElementType);

	UE_API FStructView AddElementToEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType);

	template<CSparse T>
	requires CFragment<T>
	T& AddElementToEntity(FMassEntityHandle EntityHandle)
	{
		return AddElementToEntity(EntityHandle, T::StaticStruct()).template Get<T>();
	}

	FStructView AddElementInstanceToEntity(FMassEntityHandle EntityHandle, const FConstStructView Instance)
	{
		const UScriptStruct* FragmentType = Instance.GetScriptStruct();
		checkfSlow(IsA<FMassFragment>(FragmentType) && IsSparse(FragmentType), TEXT("%s is not a sparse fragment type"), *GetNameSafe(FragmentType));
		FStructView ElementInstance = AddElementToEntity(EntityHandle, Instance.GetScriptStruct());
		FragmentType->CopyScriptStruct(ElementInstance.GetMemory(), Instance.GetMemory());
		return ElementInstance;
	}

	UE_API bool RemoveElementFromEntity(FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType);

	template<CSparse T>
	requires CFragment<T>
	bool RemoveElementFromEntity(FMassEntityHandle EntityHandle)
	{
		return RemoveElementFromEntity(EntityHandle, T::StaticStruct());
	}

	/**
	 * @param InOutEntityHandles may get modified by the function - sorted and invalid handles removed 
	 */
	void BatchAddElementToEntities(TArray<FMassEntityHandle>& InOutEntityHandles, TNotNull<const UScriptStruct*> ElementType, TArray<FStructView>* OutAddedElements = nullptr);
	void BatchAddElementToEntities(TArrayView<const FMassEntityHandle> InEntityHandles, TNotNull<const UScriptStruct*> ElementType, TArray<FStructView>* OutAddedElements = nullptr);

	template<CSparse T>
	requires CFragment<T>
	void BatchAddElementToEntities(TArray<FMassEntityHandle>& InOutEntityHandles, TArray<FStructView>* OutAddedElements = nullptr)
	{
		BatchAddElementToEntities(InOutEntityHandles, T::StaticStruct(), OutAddedElements);
	}

	void BatchAddElementInstancesToEntities(TArrayView<const FMassEntityHandle> InEntityHandles, const FStructArrayView& FragmentPayload);
	/**
	 * Adds FragmentInstance (type and values) to each entity indicated by InEntityHandles
	 */
	void BatchAddElementInstancesToEntities(TArrayView<const FMassEntityHandle> InEntityHandles, FConstStructView FragmentInstance);

	/**
	 * @param InOutEntityHandles may get modified by the function - sorted and invalid handles removed 
	 * @return num element instances actually removed
	 */
	int32 BatchRemoveElementFromEntities(TArray<FMassEntityHandle>& InOutEntityHandles, TNotNull<const UScriptStruct*> ElementType);
	int32 BatchRemoveElementFromEntities(TArrayView<const FMassEntityHandle> InEntityHandles, TNotNull<const UScriptStruct*> ElementType);

	/** @return num element instances actually removed */
	template<CSparse T>
	requires CFragment<T>
	int32 BatchRemoveElementFromEntities(TArray<FMassEntityHandle>& InOutEntityHandles)
	{
		return BatchRemoveElementFromEntities(InOutEntityHandles, T::StaticStruct());
	}

	/** Clears sparse elements instances belonging to entities indicated by EntityHandles */
	UE_API void RemoveEntity(FMassEntityHandle EntityHandle, const FMassElementBitSet& Elements);

	/**
	 * Returns the total number of instances of the given sparse element type
	 */
	uint32 GetNumElementsOfType(TNotNull<const UScriptStruct*> ElementType) const
	{
		const FTypePool* TypePool = GetTypePool(ElementType);
		return TypePool ? TypePool->GetNumElements() : 0;
	}

	template<CSparse T>
	requires CFragment<T>
	uint32 GetNumElementsOfType() const
	{
		return GetNumElementsOfType(T::StaticStruct());
	}

	/**
	 * Checks whether there are any instances of the given sparse element type
	 */
	bool HasAnyElementsOfType(TNotNull<const UScriptStruct*> ElementType) const
	{
		const FTypePool* TypePool = GetTypePool(ElementType);
		return TypePool && (TypePool->GetNumElements() > 0);
	}

	template<CSparse T>
	requires CFragment<T>
	bool HasAnyElementsOfType() const
	{
		return HasAnyElementsOfType(T::StaticStruct());
	}

	UE_API FStructView GetMutableElementDataForEntity(const FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType);
	UE_API FConstStructView GetElementDataForEntity(const FMassEntityHandle EntityHandle, TNotNull<const UScriptStruct*> ElementType) const;

	UE_API void ConfigureType(TNotNull<const UScriptStruct*> ElementType, FTypeConfig& Config);

private:
	template<typename TContainer>
	void BatchAddElementToEntities(TContainer InEntityHandles, TNotNull<const UScriptStruct*> ElementType, TArray<FStructView>* OutAddedElements);

	TSparseArray<FTypePool> TypePools;

	/** @param InTypeIndex if INDEX_NONE then the index will be calculated based on ElementType */
	FTypePool& GetOrCreateTypePool(TNotNull<const UScriptStruct*> ElementType, const int32 InTypeIndex = INDEX_NONE);
	UE_API const FTypePool* GetTypePool(TNotNull<const UScriptStruct*> ElementType) const;
	UE_API FTypePool* GetTypePool(TNotNull<const UScriptStruct*> ElementType);
};

/** To iterate over continuous lists of sparse elements of specific types */
struct FSparseElementIterator 
{
	FSparseElementIterator()
		: Pool(FSparseElementsStorage::FTypePool::GetDummyPool())
	{
	}
private:
	friend FSparseElementsStorage;
	// instances to be created only via FSparseElementsStorage
	FSparseElementIterator(const FSparseElementsStorage::FTypePool& InPool)
		: Pool(InPool), ChunkIndex(0)
	{
		++(*this);
	}
public:
	bool IsValid() const 
	{
		return Pool.Chunks.IsValidIndex(ChunkIndex)
			&& ElementIndex < Pool.Config.GetElementsPerChunk();
	}

	explicit operator bool() const
	{
		return IsValid();
	}

	void operator++()
	{
		while (ChunkIndex < Pool.Chunks.Num()) 
		{
			const FSparseElementsStorage::FPackedStructDataChunk& Chunk = Pool.Chunks[ChunkIndex];
			if (Chunk.IsInitialized())
			{
				checkfSlow(Chunk.OccupationMask.IsEmpty() == false, TEXT("Initialized chunks are expected to have their OccupationMask filled with data."));
				// Find next bit set
				TBitArray<>::FConstIterator It(Chunk.OccupationMask, ElementIndex + 1);
				while (static_cast<uint32>(It.GetIndex()) < Pool.Config.GetElementsPerChunk()) 
				{
					if (It.GetValue()) 
					{
						ElementIndex = It.GetIndex();
						return;
					}
					++It;
				}
			}
			// Move to next chunk
			++ChunkIndex;
			ElementIndex = INDEX_NONE;
		}
	}

	FStructView GetElementView() const 
	{
		checkSlow(IsValid());
		const FSparseElementsStorage::FPackedStructDataChunk& Chunk = Pool.Chunks[ChunkIndex];
		checkSlow(Chunk.OccupationMask.IsValidIndex(ElementIndex) && Chunk.OccupationMask[ElementIndex]);
		uint8* Memory = &Chunk.RawMemory[ElementIndex * Pool.Config.GetElementSize()];
		return FStructView(Pool.Config.GetType(), Memory);
	}

	FStructView operator*() const
	{
		return GetElementView();
	}

	int32 GetEntityIndex() const
	{
		return ChunkIndex * Pool.Config.GetElementsPerChunk() + ElementIndex;
	}

	UE_API FMassEntityHandle GetEntityHandle(const FMassEntityManager& EntityManager) const;

private:
	const FSparseElementsStorage::FTypePool& Pool;
	int32 ChunkIndex = INDEX_NONE;
	uint32 ElementIndex = INDEX_NONE;
};

//-----------------------------------------------------------------------------
// INLINES
//-----------------------------------------------------------------------------
inline FSparseElementIterator FSparseElementsStorage::CreateElementIterator(TNotNull<const UScriptStruct*> ElementType)
{
	FTypePool* TypePool = GetTypePool(ElementType);
	return TypePool ? FSparseElementIterator(*TypePool) : FSparseElementIterator();
}

} // namespace UE::Mass

#undef UE_API
