// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "MassRequirements.h"
#include "Subsystems/EngineSubsystem.h"
#include "Subsystems/GameInstanceSubsystem.h"
#include "Subsystems/LocalPlayerSubsystem.h"
#include "Subsystems/WorldSubsystem.h"
#include "Engine/LocalPlayer.h"
#include "Engine/GameInstance.h"

#if WITH_EDITOR
#include "Editor.h"
#include "EditorSubsystem.h"
#else
#include "Engine/Engine.h"
#endif // WITH_EDITOR

struct FMassEntityManager;

struct FMassSubsystemAccess
{
	MASSENTITY_API explicit FMassSubsystemAccess(FMassEntityManager* InEntityManager = nullptr);

	UE_DEPRECATED(5.8, "Use FMassSubsystemAccess(FMassEntityManager*) constructor instead.")
	MASSENTITY_API explicit FMassSubsystemAccess(UWorld* InWorld);

	//-----------------------------------------------------------------------------
	// Statically-typed subsystems
	//-----------------------------------------------------------------------------
	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	T* GetMutableSubsystem()
	{
		const int32 SystemIndex = FMassExternalSubsystemBitSet::GetTypeIndex<T>();
		if (ensureMsgf(MutableSubsystemsBitSet.IsBitSet(SystemIndex)
			, TEXT("Undeclared read/write access to subsystem %s"), *GetNameSafe(T::StaticClass())))
		{
			return GetSubsystemInternal<T>(SystemIndex);
		}

		return nullptr;
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	T& GetMutableSubsystemChecked()
	{
		T* InstancePtr = GetMutableSubsystem<T>();
		check(InstancePtr);
		return *InstancePtr;
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	const T* GetSubsystem()
	{
		const int32 SystemIndex = FMassExternalSubsystemBitSet::GetTypeIndex<T>();
		if (ensureMsgf(ConstSubsystemsBitSet.IsBitSet(SystemIndex) || MutableSubsystemsBitSet.IsBitSet(SystemIndex)
			, TEXT("Undeclared read-only access to subsystem %s"), *GetNameSafe(T::StaticClass())))
		{
			return GetSubsystemInternal<T>(SystemIndex);
		}
		return nullptr;
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	const T& GetSubsystemChecked()
	{
		const T* InstancePtr = GetSubsystem<T>();
		check(InstancePtr);
		return *InstancePtr;
	}

	//-----------------------------------------------------------------------------
	// UClass-provided subsystems
	//-----------------------------------------------------------------------------
	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	T* GetMutableSubsystem(const TSubclassOf<USubsystem> SubsystemClass)
	{
		check(SubsystemClass);
		const int32 SystemIndex = FMassExternalSubsystemBitSet::GetTypeIndex(*SubsystemClass);
		if (ensureMsgf(MutableSubsystemsBitSet.IsBitSet(SystemIndex)
			, TEXT("Undeclared read/write access to subsystem %s"), *GetNameSafe(SubsystemClass)))
		{
			return GetSubsystemInternal<T>(SystemIndex, SubsystemClass);
		}

		return nullptr;
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	T& GetMutableSubsystemChecked(const TSubclassOf<USubsystem> SubsystemClass)
	{
		check(SubsystemClass);
		T* InstancePtr = GetMutableSubsystem<T>(SubsystemClass);
		check(InstancePtr);
		return *InstancePtr;
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	const T* GetSubsystem(const TSubclassOf<USubsystem> SubsystemClass)
	{
		check(SubsystemClass);
		const int32 SystemIndex = FMassExternalSubsystemBitSet::GetTypeIndex(*SubsystemClass);
		if (ensureMsgf(ConstSubsystemsBitSet.IsBitSet(SystemIndex) || MutableSubsystemsBitSet.IsBitSet(SystemIndex)
			, TEXT("Undeclared read-only access to subsystem %s"), *GetNameSafe(SubsystemClass)))
		{
			return GetSubsystemInternal<T>(SystemIndex, SubsystemClass);
		}
		return nullptr;
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	const T& GetSubsystemChecked(const TSubclassOf<USubsystem> SubsystemClass)
	{
		check(SubsystemClass);
		const T* InstancePtr = GetSubsystem<T>(SubsystemClass);
		check(InstancePtr);
		return *InstancePtr;
	}

	//-----------------------------------------------------------------------------
	// remaining API
	//-----------------------------------------------------------------------------
	MASSENTITY_API bool CacheSubsystemRequirements(const FMassSubsystemRequirements& SubsystemRequirements);
	MASSENTITY_API void SetSubsystemRequirements(const FMassSubsystemRequirements& SubsystemRequirements);

	void GetSubsystemRequirementBits(FMassExternalSubsystemBitSet& OutConstSubsystemsBitSet, FMassExternalSubsystemBitSet& OutMutableSubsystemsBitSet)
	{
		OutConstSubsystemsBitSet = ConstSubsystemsBitSet;
		OutMutableSubsystemsBitSet = MutableSubsystemsBitSet;
	}

	void SetSubsystemRequirementBits(const FMassExternalSubsystemBitSet& InConstSubsystemsBitSet, const FMassExternalSubsystemBitSet& InMutableSubsystemsBitSet)
	{
		ConstSubsystemsBitSet = InConstSubsystemsBitSet;
		MutableSubsystemsBitSet = InMutableSubsystemsBitSet;
	}

	template<typename T>
	static constexpr bool DoesRequireWorld()
	{
		constexpr bool bIsWorldSubsystem = TIsDerivedFrom<T, UWorldSubsystem>::IsDerived;
		constexpr bool bIsGameInstanceSubsystem = TIsDerivedFrom<T, UGameInstanceSubsystem>::IsDerived;
		constexpr bool bIsLocalPlayerSubsystem = TIsDerivedFrom<T, ULocalPlayerSubsystem>::IsDerived;

		return (bIsWorldSubsystem || bIsGameInstanceSubsystem || bIsLocalPlayerSubsystem);
	}

	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	static T* FetchSubsystemInstance(UWorld* World)
	{
		check(World);
		if constexpr (TIsDerivedFrom<T, UWorldSubsystem>::IsDerived)
		{
			return UWorld::GetSubsystem<T>(World);
		}
		else if constexpr (TIsDerivedFrom<T, UGameInstanceSubsystem>::IsDerived)
		{
			return UGameInstance::GetSubsystem<T>(World->GetGameInstance());
		}
		else if constexpr (TIsDerivedFrom<T, ULocalPlayerSubsystem>::IsDerived)
		{
			// note that this default implementation will work only for the first player in a local-coop game
			// to customize this behavior specialize the FetchSubsystemInstance template function for the type you need. 
			return ULocalPlayer::GetSubsystem<T>(World->GetFirstLocalPlayerFromController());
		}
		else
		{
			checkf(false, TEXT("FMassSubsystemAccess::FetchSubsystemInstance: Unhandled world-related USubsystem class %s"), *T::StaticClass()->GetName());
		}
	}
	
	template<typename T, typename = typename TEnableIf<TIsDerivedFrom<T, USubsystem>::IsDerived>::Type>
	static T* FetchSubsystemInstance()
	{
		if constexpr (TIsDerivedFrom<T, UEngineSubsystem>::IsDerived)
		{
			return GEngine->GetEngineSubsystem<T>();
		}
#if WITH_EDITOR
		else if constexpr (TIsDerivedFrom<T, UEditorSubsystem>::IsDerived)
		{
			return GEditor->GetEditorSubsystem<T>();
		}
#endif // WITH_EDITOR
		else
		{
			checkf(false, TEXT("FMassSubsystemAccess::FetchSubsystemInstance: Unhandled world-less USubsystem class %s"), *T::StaticClass()->GetName());
		}
	}

	static MASSENTITY_API USubsystem* FetchSubsystemInstance(UWorld* World, TSubclassOf<USubsystem> SubsystemClass);

protected:
	template<typename T>
	T* GetSubsystemInternal(const int32 SystemIndex)
	{
		checkSlow(SystemIndex != INDEX_NONE);
		if (UNLIKELY(Subsystems.IsValidIndex(SystemIndex) == false))
		{
			Subsystems.AddZeroed(SystemIndex - Subsystems.Num() + 1);
		}

		T* SystemInstance = (T*)Subsystems[SystemIndex];
		if (SystemInstance == nullptr)
		{
			if constexpr (DoesRequireWorld<T>())
			{
				SystemInstance = FetchSubsystemInstance<std::remove_const_t<T>>(GetWorld());
			}
			else
			{
				SystemInstance = FetchSubsystemInstance<std::remove_const_t<T>>();
			}
			Subsystems[SystemIndex] = SystemInstance;
		}
		return SystemInstance;
	}

	template<typename T>
	T* GetSubsystemInternal(const int32 SystemIndex, const TSubclassOf<USubsystem> SubsystemClass)
	{
		checkSlow(SystemIndex != INDEX_NONE);
		if (UNLIKELY(Subsystems.IsValidIndex(SystemIndex) == false))
		{
			Subsystems.AddZeroed(SystemIndex - Subsystems.Num() + 1);
		}

		USubsystem* SystemInstance = (T*)Subsystems[SystemIndex];
		if (SystemInstance == nullptr)
		{
			SystemInstance = FetchSubsystemInstance(GetWorld(), SubsystemClass);
			Subsystems[SystemIndex] = SystemInstance;
		}
		return Cast<T>(SystemInstance);
	}

	MASSENTITY_API UWorld* GetWorld() const;
	MASSENTITY_API bool CacheSubsystem(const uint32 SystemIndex);

	FMassExternalSubsystemBitSet ConstSubsystemsBitSet;
	FMassExternalSubsystemBitSet MutableSubsystemsBitSet;
	TArray<USubsystem*> Subsystems;

	/** Non-owning. Lifetime is guaranteed by the owning FMassExecutionContext which holds
	 *  a TSharedRef<FMassEntityManager>. Raw pointer avoids atomic ref-count overhead on
	 *  copy (FMassSubsystemAccess is deep-copied when FMassExecutionContext is copied). */
	FMassEntityManager* EntityManager = nullptr;
};
