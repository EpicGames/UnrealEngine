// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "MassTypeManager.h"
#include "Subsystems/SubsystemCollection.h"
#include "MassEntityConcepts.h"
#include "MassSubsystemBase.generated.h"

#define UE_API MASSENTITY_API

struct FMassEntityManager;

namespace UE::Mass
{
	namespace Subsystems
	{
		struct FInitializationState
		{
			uint8 bInitializeCalled : 1 = false;
			uint8 bPostInitializeCalled : 1 = false;
			uint8 bOnWorldBeginPlayCalled : 1 = false;
		};

		MASSENTITY_API void RegisterSubsystemType(FSubsystemCollectionBase& Collection, TSubclassOf<USubsystem> SubsystemClass, FSubsystemTypeTraits&& Traits);
		MASSENTITY_API void RegisterSubsystemType(TSharedRef<FMassEntityManager> EntityManager, TSubclassOf<USubsystem> SubsystemClass, FSubsystemTypeTraits&& Traits);
	} // namespace Subsystems
} // namespace UE::Mass

/**
 * The sole responsibility of this world subsystem class is to serve functionality common to all 
 * Mass-related UWorldSubsystem-based subsystems, like whether the subsystems should get created at all. 
 */
UCLASS(Abstract, config = Mass, defaultconfig, MinimalAPI)
class UMassSubsystemBase : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	static UE_API bool AreRuntimeMassSubsystemsAllowed(UObject* Outer);
	UE::Mass::Subsystems::FInitializationState GetInitializationState() const
	{
		return InitializationState;
	}

protected:
	//~USubsystem interface
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void PostInitialize() override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual void OnWorldBeginPlay(UWorld& InWorld) override;
	//~End of USubsystem interface

	/**
	 * Needs to be called in Initialize for subsystems we want to behave properly when dynamically added after UWorld::BeginPlay
	 * (for example via GameplayFeatureActions). This is required for subsystems relying on their PostInitialize and/or OnWorldBeginPlay called.
	 */
	UE_API void HandleLateCreation();

	/**
	 * Registers given subsystem class as part of Mass type information. Needs to be called as part of Initialize override.
	 * Note that calling the function is only required if the registered traits differ from the parent class'.
	 */
	template <UE::Mass::CSubsystem T>
	void OverrideSubsystemTraits(FSubsystemCollectionBase& Collection)
	{
		UE::Mass::Subsystems::RegisterSubsystemType(Collection, T::StaticClass(), UE::Mass::FSubsystemTypeTraits::Make<T>());
	}

	/**
	 * Tracks which initialization function had already been called. Requires the child classes to call Super implementation
	 * for their Initialize, PostInitialize, Deinitialize and OnWorldBeginPlayCalled overrides
	 */
	UE::Mass::Subsystems::FInitializationState InitializationState;
};

/**
 * The sole responsibility of this tickable world subsystem class is to serve functionality common to all
 * Mass-related UTickableWorldSubsystem-based subsystems, like whether the subsystems should get created at all.
 */
UCLASS(Abstract, config = Mass, defaultconfig, MinimalAPI)
class UMassTickableSubsystemBase : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	UE::Mass::Subsystems::FInitializationState GetInitializationState() const
	{
		return InitializationState;
	}

protected:
	//~USubsystem interface
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void PostInitialize() override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual void OnWorldBeginPlay(UWorld& InWorld) override;
	//~End of USubsystem interface

	/**
	 * Registers given subsystem class as part of Mass type information. Needs to be called as part of Initialize override.
	 * Note that calling the function is only required if the registered traits differ from the parent class'.
	 */
	template <UE::Mass::CSubsystem T>
	void OverrideSubsystemTraits(FSubsystemCollectionBase& Collection)
	{
		UE::Mass::Subsystems::RegisterSubsystemType(Collection, T::StaticClass(), UE::Mass::FSubsystemTypeTraits::Make<T>());
	}

	/**
	 * Needs to be called in Initialize for subsystems we want to behave properly when dynamically added after UWorld::BeginPlay
	 * (for example via GameplayFeatureActions). This is required for subsystems relying on their PostInitialize and/or OnWorldBeginPlay called.
	 */
	UE_API void HandleLateCreation();

private:
	/** 
	 * Tracks which initialization function had already been called. Requires the child classes to call Super implementation
	 * for their Initialize, PostInitialize, Deinitialize and OnWorldBeginPlayCalled overrides
	 */
	UE::Mass::Subsystems::FInitializationState InitializationState;
};

#undef UE_API