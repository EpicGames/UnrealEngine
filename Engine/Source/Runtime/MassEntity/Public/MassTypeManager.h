// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"
#include "UObject/ObjectKey.h"
#include "Misc/NotNull.h"
#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"
#include "MassEntityConcepts.h"
#include "Subsystems/Subsystem.h"
#include "MassEntityRelations.h"
#include "Mass/ExternalSubsystemTraits.h"

#define UE_API MASSENTITY_API


struct FMassEntityManager;

namespace UE::Mass
{
	using namespace Relations;
	struct FRelationData;

	/** Handle for identifying and managing types in the type manager */
	struct FTypeHandle
	{
		FTypeHandle() = default;
		FTypeHandle(const FTypeHandle&) = default;

		bool operator==(const FTypeHandle&) const = default;

		bool IsValid() const;

		const UClass* GetClass() const
		{
			return Cast<const UClass>(TypeKey.ResolveObjectPtr());
		}

		const UScriptStruct* GetScriptStruct() const
		{
			return Cast<UScriptStruct>(TypeKey.ResolveObjectPtr());
		}

		friend inline uint32 GetTypeHash(const FTypeHandle& InHandle)
		{
			return GetTypeHash(InHandle.TypeKey);
		}

		FName GetFName() const
		{
			const UStruct* RepresentedType = TypeKey.ResolveObjectPtr();
			return GetFNameSafe(RepresentedType);
		}

		FString ToString() const
		{
			const UStruct* RepresentedType = TypeKey.ResolveObjectPtr();
			return GetNameSafe(RepresentedType);
		}

	private:
		friend struct FTypeManager;
		UE_API explicit FTypeHandle(TObjectKey<const UStruct> InTypeKey);

		TObjectKey<const UStruct> TypeKey;
	};

	/** Placeholder to be used when no traits have been specified nor the type is known */
	struct FEmptyTypeTraits
	{
	};

	/** Traits of USubsystem-based types */
	struct FSubsystemTypeTraits
	{
		FSubsystemTypeTraits() = default;

		/** Factory function for creating traits specific to a given subsystem type */
		template <typename T>
		static FSubsystemTypeTraits Make()
		{
			FSubsystemTypeTraits Traits;
			Traits.bGameThreadOnly = TMassExternalSubsystemTraits<T>::GameThreadOnly;
			Traits.bThreadSafeWrite = TMassExternalSubsystemTraits<T>::ThreadSafeWrite;
			return MoveTemp(Traits);
		}

		/** Whether the subsystem must  be run on the Game Thread */
		bool bGameThreadOnly = true;
		/** Whether the subsystem supports thread-safe write operations */
		bool bThreadSafeWrite = false;
	};

	/** Traits of Shared Fragment types */
	struct FSharedFragmentTypeTraits
	{
		FSharedFragmentTypeTraits() = default;

		/** Factory function for creating traits specific to a given shared fragment type */
		template<CSharedFragment T>
		static FSharedFragmentTypeTraits Make()
		{
			FSharedFragmentTypeTraits Traits;
			Traits.bGameThreadOnly = TMassSharedFragmentTraits<T>::GameThreadOnly;
			return MoveTemp(Traits);
		}

		/** Whether the shared fragment has to be used only on the Game Thread */
		bool bGameThreadOnly = true;
	};

	/**
	 * Wrapper for metadata and traits about specific types. The type is used by the TypeManager
	 * to uniformly store information for all types.
	 */
	struct FTypeInfo
	{
		using FTypeTraits = TVariant<FEmptyTypeTraits
			, FSubsystemTypeTraits
			, FSharedFragmentTypeTraits
			, FRelationTypeTraits>;

		FName TypeName;
		FTypeTraits Traits;

		/** Fetches stored data as FSubsystemTypeTraits, if applicable, or null otherwise */
		const FSubsystemTypeTraits* GetAsSystemTraits() const;

		/** Fetches stored data as FSharedFragmentTypeTraits, if applicable, or null otherwise */
		const FSharedFragmentTypeTraits* GetAsSharedFragmentTraits() const;

		/** Fetches stored data as FRelationTypeTraits, if applicable, or null otherwise */
		const FRelationTypeTraits* GetAsRelationTraits() const;

		/** Fetches stored data as FRelationTypeTraits. Will complain if stored data is not of FRelationTypeTraits type */
		const FRelationTypeTraits& GetAsRelationTraitsChecked() const;
	};

	struct FTypeManager : TSharedFromThis<FTypeManager>
	{
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnRegisterBuiltInTypes, FTypeManager& /*Instance*/);

		/**
		 * Move-only, ref-counted handle for a static type registration. The static registry tracks
		 * how many handles reference each type — the entry is only removed when the last handle
		 * for that type is destroyed. Already-initialized FTypeManager instances are unaffected.
		 */
		struct FStaticTypeRegistrationHandle
		{
			FStaticTypeRegistrationHandle() = default;
			UE_API ~FStaticTypeRegistrationHandle();

			UE_API FStaticTypeRegistrationHandle(FStaticTypeRegistrationHandle&& InOther);
			UE_API FStaticTypeRegistrationHandle& operator=(FStaticTypeRegistrationHandle&& InOther);

			FStaticTypeRegistrationHandle(const FStaticTypeRegistrationHandle&) = delete;
			FStaticTypeRegistrationHandle& operator=(const FStaticTypeRegistrationHandle&) = delete;

			bool IsValid() const;

		private:
			friend struct FTypeManager;
			TObjectKey<const UStruct> RegisteredType;
		};

		UE_API explicit FTypeManager(FMassEntityManager& InEntityManager);

		void RegisterBuiltInTypes();

		/**
		 * @return whether the type manager instance has any types registered at all.
		 */
		bool IsEmpty() const;

		/** Register traits for given subsystem type */
		UE_API FTypeHandle RegisterType(TNotNull<const UStruct*> InType, FSubsystemTypeTraits&&);
		/** Register traits for given shared fragments type */
		UE_API FTypeHandle RegisterType(TNotNull<const UStruct*> InType, FSharedFragmentTypeTraits&&);
		/** Register traits for given relation type */
		UE_API FTypeHandle RegisterType(FRelationTypeTraits&&);

		/** Registration helper for shared fragments */
		template<CSharedFragment T>
		FTypeHandle RegisterType();

		/** Registration helper for subsystems */
		template<typename T> requires TIsDerivedFrom<typename TRemoveReference<T>::Type, USubsystem>::Value
		FTypeHandle RegisterType();

		/**
		 * @return stored traits for the type represented by TypeHandle, or null if the type is unknown
		 */
		const FTypeInfo* GetTypeInfo(FTypeHandle TypeHandle) const;

		/**
		 * @return stored traits for the type represented by TypeKey, or null if the type is unknown
		 */
		const FTypeInfo* GetTypeInfo(TObjectKey<const UStruct> TypeKey) const;

		const FRelationTypeTraits& GetRelationTypeChecked(const FTypeHandle TypeHandle) const;
		const FRelationTypeTraits& GetRelationTypeChecked(const TNotNull<const UScriptStruct*> RelationOrElementType) const;
		UE_API FTypeHandle GetRelationTypeHandle(const TNotNull<const UScriptStruct*> RelationOrElementType) const;
		UE_API bool IsValidRelationType(const TNotNull<const UScriptStruct*> RelationOrElementType) const;

		/** Alias for the iterator type that can be used to iterate all of stored type traits information */
		using FTypeInfoConstIterator = TMap<FTypeHandle, FTypeInfo>::TConstIterator;

		/**
		 * Alias for the iterator type that can be used to iterate over stored subsystem types.
		 * Note that this iterator will only point to types, not type trait data. Contents of this iterator
		 * need to be used with GetTypeInfo to get actual traits data. 
		 */
		using FSubsystemTypeConstIterator = TSet<FTypeHandle>::TConstIterator;

		FTypeInfoConstIterator MakeIterator() const;
		FSubsystemTypeConstIterator MakeSubsystemIterator() const;

		FMassEntityManager& GetEntityManager();

		static FTypeHandle MakeTypeHandle(TNotNull<const UStruct*> InTypeKey);

		/**
		 * Broadcasts as part of FTypeManager::RegisterBuiltInTypes call, giving a chance to the external code to
		 * register additional types that are supposed to be available from the very start.
		 */
		UE_API static FOnRegisterBuiltInTypes OnRegisterBuiltInTypes;

		//---------------------------------------------------------------------
		// Static type registration
		// Types registered here are copied into every FTypeManager instance
		// during RegisterBuiltInTypes(). Use for types whose lifetime spans
		// multiple entity managers (engine subsystems, game instance subsystems).
		//---------------------------------------------------------------------

		/**
		 * Register a type statically with an RAII handle. Destroying the returned handle removes
		 * the type from the static registry (future TypeManager instances won't see it).
		 */
		UE_API static FStaticTypeRegistrationHandle RegisterStaticType(TNotNull<const UStruct*> InType, FSubsystemTypeTraits&& TypeTraits);
		UE_API static FStaticTypeRegistrationHandle RegisterStaticType(TNotNull<const UStruct*> InType, FSharedFragmentTypeTraits&& TypeTraits);

		/** RAII static registration for subsystem types. T must derive from USubsystem. */
		template<CSubsystem T> 
		static FStaticTypeRegistrationHandle RegisterStaticType();

		/** RAII static registration for shared fragment types. T must satisfy CSharedFragment. */
		template<CSharedFragment T>
		static FStaticTypeRegistrationHandle RegisterStaticType();

	private:
		/** Register traits for given type */
		UE_API FTypeHandle RegisterTypeInternal(TNotNull<const UStruct*> InType, FTypeInfo&&);

		FMassEntityManager& OuterEntityManager; 

		/** Mapping of types to their info */
		TMap<FTypeHandle, FTypeInfo> TypeDataMap;

		/** Contains all registered subsystem types. Can be used to filter access to TypeDataMap */
		TSet<FTypeHandle> SubsystemTypes;

		bool bBuiltInTypesRegistered = false;
	};

	//----------------------------------------------------------------------------
	// INLINES
	//----------------------------------------------------------------------------
	inline bool FTypeHandle::IsValid() const
	{
		// using this strange contraption since TObjectKey doesn't supply a way to check
		// whether it's set, while comparison and construction is trivial.
		// Note: we don't care to what the key's been set, we don't expect types to go away
		return TypeKey != TObjectKey<const UStruct>();
	}

	inline const FSubsystemTypeTraits* FTypeInfo::GetAsSystemTraits() const
	{
		return Traits.TryGet<FSubsystemTypeTraits>();
	}

	inline const FSharedFragmentTypeTraits* FTypeInfo::GetAsSharedFragmentTraits() const
	{
		return Traits.TryGet<FSharedFragmentTypeTraits>();
	}

	inline const FRelationTypeTraits* FTypeInfo::GetAsRelationTraits() const
	{
		return Traits.TryGet<FRelationTypeTraits>();
	}

	inline const FRelationTypeTraits& FTypeInfo::GetAsRelationTraitsChecked() const
	{
		return Traits.Get<FRelationTypeTraits>();
	}

	inline bool FTypeManager::IsEmpty() const
	{
		return TypeDataMap.IsEmpty();
	}

	inline FTypeManager::FTypeInfoConstIterator FTypeManager::MakeIterator() const
	{
		return FTypeManager::FTypeInfoConstIterator(TypeDataMap);
	}

	inline FTypeManager::FSubsystemTypeConstIterator FTypeManager::MakeSubsystemIterator() const
	{
		return FTypeManager::FSubsystemTypeConstIterator(SubsystemTypes);
	}

	template<CSharedFragment T>
	FTypeHandle FTypeManager::RegisterType()
	{
		return RegisterType(T::StaticStruct(), FSharedFragmentTypeTraits::Make<T>());
	}

	template<typename T>
	requires TIsDerivedFrom<typename TRemoveReference<T>::Type, USubsystem>::Value
	FTypeHandle FTypeManager::RegisterType()
	{
		return RegisterType(T::StaticClass(), FSubsystemTypeTraits::Make<T>());
	}

	inline const FTypeInfo* FTypeManager::GetTypeInfo(FTypeHandle TypeHandle) const
	{
		return TypeDataMap.Find(TypeHandle);
	}

	inline const FTypeInfo* FTypeManager::GetTypeInfo(TObjectKey<const UStruct> TypeKey) const
	{
		return TypeDataMap.Find(FTypeHandle(TypeKey));
	}

	inline const FRelationTypeTraits& FTypeManager::GetRelationTypeChecked(const FTypeHandle RelationHandle) const
	{
		const FTypeInfo* RelationInfo = GetTypeInfo(RelationHandle);
		check(RelationInfo);
		return RelationInfo->GetAsRelationTraitsChecked();
	}

	inline const FRelationTypeTraits& FTypeManager::GetRelationTypeChecked(const TNotNull<const UScriptStruct*> RelationOrElementType) const
	{
		return GetRelationTypeChecked(GetRelationTypeHandle(RelationOrElementType));
	}

	inline FMassEntityManager& FTypeManager::GetEntityManager()
	{
		return OuterEntityManager;
	}

	inline bool FTypeManager::FStaticTypeRegistrationHandle::IsValid() const
	{
		return RegisteredType != TObjectKey<const UStruct>();
	}

	inline FTypeHandle FTypeManager::MakeTypeHandle(TNotNull<const UStruct*> InTypeKey)
	{
		return FTypeHandle(InTypeKey);
	}

	//---------------------------------------------------------------------
	// Static registration template inlines
	//---------------------------------------------------------------------
	template<CSubsystem T>
	FTypeManager::FStaticTypeRegistrationHandle FTypeManager::RegisterStaticType()
	{
		return RegisterStaticType(T::StaticClass(), FSubsystemTypeTraits::Make<T>());
	}

	template<CSharedFragment T>
	FTypeManager::FStaticTypeRegistrationHandle FTypeManager::RegisterStaticType()
	{
		return RegisterStaticType(T::StaticStruct(), FSharedFragmentTypeTraits::Make<T>());
	}

} // namespace UE::Mass

#undef UE_API
