// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityRelations.h"
#include "MassTypeManager.h"
#include "MassRelationObservers.h"
#include "MassChildOf.generated.h"

#define UE_API MASSENTITY_API
 
namespace UE::Mass::Relations
{
	extern UE_API FTypeHandle ChildOfHandle;
	UE_API void RegisterChildOfRelation();
} // namespace UE::Mass::Relations

USTRUCT()
struct FMassChildOfRelation : public FMassRelation
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassChildOfFragment : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Mass)
	FMassEntityHandle Parent;
};

UCLASS(MinimalAPI)
class UMassChildOfRelationEntityCreation : public UMassRelationEntityCreation
{
	GENERATED_BODY()

protected:
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};

#undef UE_API
