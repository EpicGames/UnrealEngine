// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructTypeBitSet.h"
#include "Templates/UnrealTypeTraits.h"

/**
 * Traits configuring finer details of TTypeBitSetBuilder behavior.
 * Provides a way to enforce compile-time checks and validations.
 */
template <typename TBaseType>
struct TTypeBitSetTraits final
{
	/** 
	 * Static constexpr boolean to verify if a tested type is valid.
	 * Ensures that the tested type is derived from the base type.
	 */
	template <typename TTestedType>
	static constexpr bool IsValidType = TIsDerivedFrom<TTestedType, TBaseType>::IsDerived;
};

/**
 * TTypeBitSetBuilder is a template class for building and managing type-specific bitsets.
 * It extends TTypeBitSetBase to provide functionalities specific to bitset building.
 *
 * @param TBaseStruct the base struct type that all stored types must derive from.
 * @param TUStructType Unreal's struct type, typically UScriptStruct or UClass.
 * @param bTestInheritanceAtRuntime flag to enable runtime inheritance checks.
 * @param TContainer the container type for storing bitsets (default is FStructTypeBitSet::FBitSetContainer).
 */
template<typename TBaseStruct, typename TUStructType = UScriptStruct, bool bTestInheritanceAtRuntime=WITH_STRUCTUTILS_DEBUG, typename TContainer=FStructTypeBitSet::FBitSetContainer>
struct TTypeBitSetBuilder : TTypeBitSetBase<TTypeBitSetBuilder<TBaseStruct, TUStructType, bTestInheritanceAtRuntime>, TBaseStruct, TUStructType, TContainer&, bTestInheritanceAtRuntime>
{
	/** Define the base class for easier reference */
	using Super = TTypeBitSetBase<TTypeBitSetBuilder, TBaseStruct, TUStructType, TContainer&, bTestInheritanceAtRuntime>;
	/** Alias for the base struct type */
	using FBaseStruct = TBaseStruct;
	/** Traits for compile-time checks */
	using FTraits = TTypeBitSetTraits<TBaseStruct>;

private:
	/**
	 * Internal bitset class extending TContainer.
	 * Used to ensure proper comparison of bitsets.
	 */
	struct FBitSet : TContainer
	{
		/**
		 * Equality operator to compare bitsets.
		 * Uses CompareSetBits to compare the set bits, ignoring missing bits.
		 */
		inline bool operator==(const FBitSet& Other) const
		{
			return TContainer::CompareSetBits(Other, /*bMissingBitValue=*/false);
		}
	};

public:
	/** Alias for a const bitset, this is the type to use to represent the stored bit set */
	using FConstBitSet = const FBitSet;

	/** Friend declaration to allow base class access */
	friend Super;

	/** Bring base class methods into scope */
	using Super::StructTypesBitArray;
	using Super::GetBaseUStruct;
	using Super::Add;
	using Super::Remove;
	using Super::Contains;
	using Super::operator+=;
	using Super::operator-=;
	using Super::operator+;
	using Super::operator-;
	using Super::ExportTypes;

	/**
	 * Constructor that initializes the builder with a struct tracker and a source bitset.
	 * @param InStructTracker the struct tracker to use.
	 * @param Source the source bitset to initialize from.
	 */
	TTypeBitSetBuilder(FStructTracker& InStructTracker, FConstBitSet& Source)
		: Super(const_cast<FBitSet&>(Source))
		, StructTracker(InStructTracker)
	{
	}

private:	
	/**
	 * Private constructor used internally to initialize with a specific bit index.
	 * @param InStructTracker the struct tracker to use.
	 * @param Source the source bitset to initialize from.
	 * @param BitToSet the bit index to set.
	 */
	TTypeBitSetBuilder(FStructTracker& InStructTracker, FConstBitSet& Source, const int32 BitToSet)
		: Super(const_cast<FBitSet&>(Source))
		, StructTracker(InStructTracker)
	{
		StructTypesBitArray.AddAtIndex(BitToSet); 
	}

public:
	/**
	 * Assignment operator to copy the bitset from another builder.
	 * Ensures that both builders use the same struct tracker.
	 * @param Source the other builder to copy from.
	 * @return Reference to this builder.
	 */
	TTypeBitSetBuilder& operator=(const TTypeBitSetBuilder& Source)
	{
		ensureMsgf(&Source.StructTracker == &StructTracker, TEXT("Assignment is only allowed between two instances created with the same StructTracker."));
		StructTypesBitArray = Source.StructTypesBitArray; 
		return *this;
	}

	/**
	 * Retrieves the index of a struct type within the struct tracker.
	 * If the type is not registered, it will be added.
	 * @param InStructType the struct type to get the index for.
	 * @return The index of the struct type.
	 */
	int32 GetTypeIndex(TNotNull<const TUStructType*> InStructType) const
	{
		if constexpr (bTestInheritanceAtRuntime)
		{
			// Ensure the struct type derives from the base struct
			if (UNLIKELY(!ensureMsgf(InStructType->IsChildOf(GetBaseUStruct())
				, TEXT("Creating index for '%s' while it doesn't derive from the expected struct type %s")
				, *InStructType->GetPathName(), *GetBaseUStruct()->GetName())))
			{
				return INDEX_NONE;
			}
		}

		// Find or add the struct type index in the tracker
		return GetStructTracker().FindOrAddStructTypeIndex(InStructType);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	int32 GetTypeIndex(const TUStructType& InStructType) const
	{
		return GetTypeIndex(&InStructType);
	}

	/**
	 * Static method to get the type index for a given struct type.
	 * @param InStructTracker the struct tracker to use.
	 * @param InStructType the struct type to get the index for.
	 * @return The index of the struct type.
	 */
	static int32 GetTypeIndex(const FStructTracker& InStructTracker, TNotNull<const TUStructType*> InStructType)
	{
		if constexpr (bTestInheritanceAtRuntime)
		{
			// Ensure the struct type derives from the base struct
			if (UNLIKELY(!ensureMsgf(InStructType->IsChildOf(GetBaseUStruct())
				, TEXT("Creating index for '%s' while it doesn't derive from the expected struct type %s")
				, *InStructType->GetPathName(), *GetBaseUStruct()->GetName())))
			{
				return INDEX_NONE;
			}
		}

		// Find or add the struct type index in the tracker
		return InStructTracker.FindOrAddStructTypeIndex(InStructType);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	int32 GetTypeIndex(const FStructTracker& InStructTracker, const TUStructType& InStructType) const
	{
		return GetTypeIndex(InStructTracker, &InStructType);
	}

	/**
	 * Template method to get the type index for a specific C++ type.
	 * Ensures at compile-time that the type is valid.
	 * @param T the C++ type to get the index for.
	 * @return The index of the struct type.
	 */
	template<typename T>
	static int32 GetTypeIndex(FStructTracker& InStructTracker)
	{
		static_assert(FTraits::template IsValidType<T>, "Given struct is not a valid type for this TypeBitSetBuilder.");
		static const int32 TypeIndex = GetTypeIndex(InStructTracker, UE::StructUtils::GetAsUStruct<T>());
		return TypeIndex;
	}

	/**
	 * Template method to get the type index for a specific C++ type.
	 * Ensures at compile-time that the type is valid.
	 * @param T the C++ type to get the index for.
	 * @return The index of the struct type.
	 */
	template<typename T>
	int32 GetTypeIndex() const
	{
		static_assert(FTraits::template IsValidType<T>, "Given struct is not a valid type for this TypeBitSetBuilder.");
		static const int32 TypeIndex = GetTypeIndex(UE::StructUtils::GetAsUStruct<T>());
		return TypeIndex;
	}
	
	/**
	 * Template method to create a bitset for a specific C++ type.
	 * @param T the C++ type.
	 * @param InStructType the Unreal struct type.
	 * @return A new TTypeBitSetBuilder instance with the specified type.
	 */
	template<typename T>
	static TTypeBitSetBuilder GetTypeBitSet(const TUStructType& InStructType)
	{
		static_assert(FTraits::template IsValidType<T>, "Given struct is not a valid type for this TypeBitSetBuilder.");
		return TTypeBitSetBuilder(GetTypeIndex<T>(InStructType));
	}

	/**
	 * Retrieves the struct type at a given index from the struct tracker.
	 * @param Index the index of the struct type.
	 * @return Pointer to the struct type.
	 */
	const TUStructType* GetTypeAtIndex(const int32 Index)
	{
		return Cast<const TUStructType>(StructTracker.GetStructType(Index));
	}
	
	/**
	 * Template method to add a struct type to the bitset.
	 * @param T the C++ type to add.
	 * @return The index of the added struct type.
	 */
	template<typename T>
	inline int32 Add()
	{
		static_assert(FTraits::template IsValidType<T>, "Given struct is not a valid type for this TypeBitSetBuilder.");
		const int32 StructTypeIndex = GetTypeIndex<T>();
		StructTypesBitArray.AddAtIndex(StructTypeIndex); 
		return StructTypeIndex;
	}

	/**
	 * Template method to remove a struct type from the bitset.
	 * @param T the C++ type to remove.
	 * @return The index of the removed struct type.
	 */
	template<typename T>
	inline int32 Remove()
	{
		static_assert(FTraits::template IsValidType<T>, "Given struct is not a valid type for this TypeBitSetBuilder.");
		const int32 StructTypeIndex = GetTypeIndex<T>();
		StructTypesBitArray.RemoveAtIndex(StructTypeIndex); 
		return StructTypeIndex;
	}

	/**
	 * Removes all bits set in another builder's bitset from this builder's bitset.
	 * @param Other the other builder whose bits to remove.
	 */
	inline void Remove(const TTypeBitSetBuilder& Other)
	{
		StructTypesBitArray -= Other.StructTypesBitArray;
	}

	/**
	 * Template method to check if a struct type is contained in the bitset.
	 * @param T the C++ type to check.
	 * @return True if the type is contained; false otherwise.
	 */
	template<typename T>
	inline bool Contains() const
	{
		static_assert(FTraits::template IsValidType<T>, "Given struct is not a valid type for this TypeBitSetBuilder.");
		const int32 StructTypeIndex = GetTypeIndex<T>();
		return StructTypesBitArray.Contains(StructTypeIndex);
	}

	/**
	 * Subtracts another builder's bitset from this builder's bitset.
	 * @param Other the other builder.
	 * @return A new builder with the result.
	 */
	inline TTypeBitSetBuilder operator-(const TTypeBitSetBuilder& Other) const
	{
		TTypeBitSetBuilder Result = *this;
		Result -= Other;
		return MoveTemp(Result);
	}

	/**
	 * Adds another builder's bitset to this builder's bitset.
	 * @param Other the other builder.
	 * @return A new builder with the result.
	 */
	inline TTypeBitSetBuilder operator+(const TTypeBitSetBuilder& Other) const
	{
		TTypeBitSetBuilder Result(StructTracker);
		Result.StructTypesBitArray = TBitArray<>::BitwiseOR(StructTypesBitArray, Other.StructTypesBitArray, EBitwiseOperatorFlags::MaxSize);
		return MoveTemp(Result);
	}

	/**
	 * Performs a bitwise AND with another builder's bitset.
	 * @param Other the other builder.
	 * @return A new builder with the result.
	 */
	inline TTypeBitSetBuilder operator&(const TTypeBitSetBuilder& Other) const
	{
		return TTypeBitSetBuilder(StructTracker, TBitArray<>::BitwiseAND(StructTypesBitArray, Other.StructTypesBitArray, EBitwiseOperatorFlags::MinSize));
	}

	/**
	 * Performs a bitwise OR with another builder's bitset.
	 * @param Other the other builder.
	 * @return A new builder with the result.
	 */
	inline TTypeBitSetBuilder operator|(const TTypeBitSetBuilder& Other) const
	{
		return TTypeBitSetBuilder(StructTracker, TBitArray<>::BitwiseOR(StructTypesBitArray, Other.StructTypesBitArray, EBitwiseOperatorFlags::MaxSize));
	}

	/**
	 * Gets the overlap between this builder's bitset and another's.
	 * @param Other the other builder.
	 * @return A new builder representing the overlap.
	 */
	inline TTypeBitSetBuilder GetOverlap(const TTypeBitSetBuilder& Other) const
	{
		return *this & Other;
	}

	/**
	 * Checks if this builder's bitset is equivalent to another's.
	 * @param Other the other builder.
	 * @return True if equivalent; false otherwise.
	 */
	inline bool IsEquivalent(const TTypeBitSetBuilder& Other) const
	{
		return StructTypesBitArray.CompareSetBits(Other.StructTypesBitArray, /*bMissingBitValue=*/false);
	}

	/**
	 * Checks if this builder's bitset has all bits set in another's.
	 * @param Other the other builder.
	 * @return True if all bits are set; false otherwise.
	 */
	inline bool HasAll(const TTypeBitSetBuilder& Other) const
	{
		return StructTypesBitArray.HasAll(Other.StructTypesBitArray);
	}

	/**
	 * Checks if this builder's bitset has any bits set in another's.
	 * @param Other the other builder.
	 * @return True if any bits are set; false otherwise.
	 */
	inline bool HasAny(const TTypeBitSetBuilder& Other) const
	{
		return StructTypesBitArray.HasAny(Other.StructTypesBitArray);
	}

	/**
	 * Checks if this builder's bitset has none of the bits set in another's.
	 * @param Other the other builder.
	 * @return True if no bits are set; false otherwise.
	 */
	inline bool HasNone(const TTypeBitSetBuilder& Other) const
	{
		return !StructTypesBitArray.HasAny(Other.StructTypesBitArray);
	}

	/**
	 * Checks if the bitset is empty.
	 * @return True if empty; false otherwise.
	 */
	bool IsEmpty() const 
	{ 
		return StructTypesBitArray.IsEmpty();
	}

	/**
	 * Checks if a specific bit is set in the bitset.
	 * @param BitIndex the index of the bit to check.
	 * @return True if the bit is set; false otherwise.
	 */
	inline bool IsBitSet(const int32 BitIndex) const
	{
		return StructTypesBitArray.Contains(BitIndex);
	}

	/**
	 * Counts the number of set bits in the bitset.
	 * @return The number of set bits.
	 */
	int32 CountStoredTypes() const
	{
		return StructTypesBitArray.CountSetBits();
	}

	/**
	 * Retrieves the maximum number of types tracked by the struct tracker.
	 * @return The maximum number of types.
	 */
	int32 GetMaxNum() const
	{		
		return StructTracker.Num();
	}

	/**
	 * Static method to get the maximum number of types from a struct tracker.
	 * @param StructTracker the struct tracker to query.
	 * @return The maximum number of types.
	 */
	static int32 GetMaxNum(const FStructTracker& StructTracker)
	{		
		return StructTracker.Num();
	}

	/**
	 * Equality operator to compare two builders.
	 * @param Other the other builder to compare.
	 * @return True if equal; false otherwise.
	 */
	inline bool operator==(const TTypeBitSetBuilder& Other) const
	{
		return StructTypesBitArray.CompareSetBits(Other.StructTypesBitArray, /*bMissingBitValue=*/false);
	}

	/**
	 * Inequality operator to compare two builders.
	 * @param Other the other builder to compare.
	 * @return True if not equal; false otherwise.
	 */
	inline bool operator!=(const TTypeBitSetBuilder& Other) const
	{
		return !(*this == Other);
	}

	/**
	 * Conversion operator to a const bitset.
	 * @return A const reference to the bitset.
	 */
	operator FConstBitSet() const
	{
		return static_cast<FConstBitSet&>(StructTypesBitArray);
	}

	/**
	 * Exports the types stored in the bitset to an output array.
	 * Note: This method can be slow due to the use of weak pointers in the struct tracker.
	 * @param TOutStructType the output struct type.
	 * @param Allocator the allocator for the output array.
	 * @param OutTypes the array to populate with struct types.
	 */
	template<typename TOutStructType, typename Allocator>
	void ExportTypes(TArray<const TOutStructType*, Allocator>& OutTypes) const
	{
		TBitArray<>::FConstIterator It(StructTypesBitArray);
		while (It)
		{
			if (It.GetValue())
			{
				// Get the struct type from the tracker and add to the output array
				OutTypes.Add(Cast<TOutStructType>(StructTracker.GetStructType(It.GetIndex())));
			}
			++It;
		}
	}

	/**
	 * Lists all types used by this bitset, calling the provided callback for each one.
	 * Returning false from the callback will early-out of iterating over the types.
	 * Note: This method can be slow due to the use of weak pointers in the struct tracker.
	 * @param Callback the callback function to call for each type.
	 */
	void ExportTypes(TFunctionRef<bool(const TUStructType*)> Callback) const
	{
		TBitArray<>::FConstIterator It(StructTypesBitArray);
		bool bKeepGoing = true;
		while (bKeepGoing && It)
		{
			if (It.GetValue())
			{
				bKeepGoing = Callback(GetTypeAtIndex(It.GetIndex()));
			}
			++It;
		}
	}

	/**
	 * Retrieves the allocated size of the bitset.
	 * @return The allocated size in bytes.
	 */
	SIZE_T GetAllocatedSize() const
	{
		return StructTypesBitArray.GetAllocatedSize();
	}

	/**
	 * Provides a debug string description of the bitset contents.
	 * @return A string describing the contents of the bitset.
	 */
	FString DebugGetStringDesc() const
	{
	#if WITH_STRUCTUTILS_DEBUG
		FStringOutputDevice Ar;
		DebugGetStringDesc(Ar);
		return static_cast<FString>(Ar);
	#else
		return TEXT("DEBUG INFO COMPILED OUT");
	#endif //WITH_STRUCTUTILS_DEBUG
	}

#if WITH_STRUCTUTILS_DEBUG
	/**
	 * Provides a debug string description of the bitset contents, via the provided FOutputDevice.
	 * @param Ar the output device to write to.
	 */
	void DebugGetStringDesc(FOutputDevice& Ar) const
	{
		for (int32 Index = 0; Index < StructTypesBitArray.Num(); ++Index)
		{
			if (StructTypesBitArray[Index])
			{
				// Get the name of the struct type and log it
				Ar.Logf(TEXT("%s, "), *StructTracker.DebugGetStructTypeName(Index).ToString());
			}
		}
	}

	/**
	 * Gets the names of individual struct types in the bitset.
	 * @param OutFNames Array to populate with struct type names.
	 */
	void DebugGetIndividualNames(TArray<FName>& OutFNames) const
	{
		for (int32 Index = 0; Index < StructTypesBitArray.Num(); ++Index)
		{
			if (StructTypesBitArray[Index])
			{
				OutFNames.Add(StructTracker.DebugGetStructTypeName(Index));
			}
		}
	}

	/**
	 * Retrieves the name of a struct type at a given index.
	 * @param StructTypeIndex the index of the struct type.
	 * @return The name of the struct type.
	 */
	FName DebugGetStructTypeName(const int32 StructTypeIndex) const
	{
		return StructTracker.DebugGetStructTypeName(StructTypeIndex);
	}

	/**
	 * Retrieves all registered struct types as a view.
	 * @return Array view of weak pointers to struct types.
	 */
	TConstArrayView<TWeakObjectPtr<const TUStructType>> DebugGetAllStructTypes() const
	{
		return StructTracker.DebugGetAllStructTypes<TUStructType>();
	}

	/**
	 * Static method to retrieve the name of a struct type at a given index from a struct tracker.
	 * @param StructTracker the struct tracker to use.
	 * @param StructTypeIndex the index of the struct type.
	 * @return The name of the struct type.
	 */
	static FName DebugGetStructTypeName(const FStructTracker& StructTracker, const int32 StructTypeIndex)
	{
		return StructTracker.DebugGetStructTypeName(StructTypeIndex);
	}

	/**
	 * Static method to retrieve all registered struct types from a struct tracker.
	 * @param StructTracker the struct tracker to use.
	 * @return Array view of weak pointers to struct types.
	 */
	static TConstArrayView<TWeakObjectPtr<const TUStructType>> DebugGetAllStructTypes(const FStructTracker& StructTracker)
	{
		return StructTracker.DebugGetAllStructTypes<TUStructType>();
	}

protected:
	/** For unit testing purposes only */
	const TBitArray<>& DebugGetStructTypesBitArray() const 
	{ 
		return StructTypesBitArray; 
	}

	TBitArray<>& DebugGetMutableStructTypesBitArray() 
	{ 
		return StructTypesBitArray; 
	}
#endif // WITH_STRUCTUTILS_DEBUG

protected:
	/**
	 * Retrieves the struct tracker used by this builder.
	 * @return Reference to the struct tracker.
	 */
	FStructTracker& GetStructTracker() const
	{
		return StructTracker;
	}

private:
	/** 
	 * Reference to the struct tracker used. It's TTypeBitSetBuilder's instance creator responsibility 
	 * to ensure that the instance doesn't outlive the referenced object.
	 */
	FStructTracker& StructTracker; 
};
